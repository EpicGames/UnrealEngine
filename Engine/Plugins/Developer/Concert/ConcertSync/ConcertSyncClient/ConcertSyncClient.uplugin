{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.2",
	"FriendlyName": "Concert Sync - Client",
	"Description": "Client plugin to enables multi-users editor sessions when connecting to a Concert Server",
	"Category": "Networking",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"Hidden": true,
	"IsBetaVersion": true,
	"Installed": false,
	"SupportedPrograms": [ "LiveLinkHub" ],
	"Modules": [
		{
			"Name": "ConcertSyncClient",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "ConcertSyncCore",
			"Enabled": true
		},
		{
			"Name": "ConcertMain",
			"Enabled": true
		},
		{
			"Name": "FileSandbox",
			"Enabled": true
		}
	]
}
