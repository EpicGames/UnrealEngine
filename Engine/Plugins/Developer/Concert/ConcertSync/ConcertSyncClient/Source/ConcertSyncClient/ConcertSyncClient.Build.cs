// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ConcertSyncClient : ModuleRules
	{
		public ConcertSyncClient(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"Concert",
					"ConcertClient",
					"ConcertSyncCore"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetRegistry",
					"ConcertTransport",
					"FileSandboxCore",
					"HeadMountedDisplay",
					"InputCore",
					"JsonUtilities",
					"LevelSequence",
					"MovieScene",
					"RenderCore",
					"TimeManagement",
					"SlateCore",
					"Slate",
					"SourceControl",
					"Serialization",
					"TraceLog"
                }
            );

			if (Target.bBuildEditor)
			{
				PrivateIncludePathModuleNames.AddRange(
					new string[]
					{
						"DirectoryWatcher",
					}
				);

				DynamicallyLoadedModuleNames.AddRange(
					new string[]
					{
						"DirectoryWatcher",
					}
				);

				PrivateDependencyModuleNames.AddRange(
					new string[] {
						"EditorFramework",
						"EditorStyle",
						"EngineSettings",
						"LevelEditor",
						"Sequencer",
						"UnrealEd",
						"TypedElementRuntime",
						"TypedElementFramework",
						"ViewportInteraction",
						"VREditor",
					}
				);
			}
		}
	}
}
