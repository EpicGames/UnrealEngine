// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertActivityStream.h"

#include "IConcertClientWorkspace.h"
#include "IConcertClient.h"
#include "IConcertModule.h"
#include "IConcertClientModule.h"

#define LOCTEXT_NAMESPACE "ConcertActivityStream"

FConcertActivityStream::FConcertActivityStream(TWeakPtr<IConcertClient, ESPMode::ThreadSafe> InConcertClient, const FGuid& InAdminEndpointId, const FGuid& InSessionId, bool bInIncludeActivityDetails, uint32 InBufferSize)
: WeakConcertClient(MoveTemp(InConcertClient))
, AdminEndpointId(InAdminEndpointId)
, SessionId(InSessionId)
, FetchRequestContinuationToken(MakeShared<uint8, ESPMode::ThreadSafe>(0))
, BufferSize(FMath::Max(InBufferSize, 1u))
, bIncludeActivityDetails(bInIncludeActivityDetails)
{
	// Fire the initial request and find out how many activities are available.
	Fetch(-GetRequestBatchSize()); // Negative -> request the N last activities (the most recent ones), up to 1024.
}

bool FConcertActivityStream::Read(TArray<TSharedPtr<FConcertSessionActivity>>& InOutArray, int32& OutReadCount, FText& OutErrorMsg)
{
	OutErrorMsg = LastErrorMsg;
	if (!OutErrorMsg.IsEmpty())
	{
		OutReadCount = 0;
		bAllActivitiesFetched = true;
		return true; // Done? When an error occurs, the stream is in a unknown state, consider the streaming done.
	}

	if (!bAllActivitiesFetched)
	{
		// Fire a new request to buffer some more.
		Fetch(GetRequestBatchSize());
	}

	// Returns the cached activities to the caller.
	OutReadCount = AvailableActivities.Num();
	if (OutReadCount)
	{
		InOutArray.Append(MoveTemp(AvailableActivities));
		check(AvailableActivities.Num() == 0);
	}

	return bAllActivitiesFetched; // Done?
}

void FConcertActivityStream::Fetch(int64 RequestFetchCount)
{
	check(RequestFetchCount != 0); // Can be negative (tail query) or positive, but not zero.

	if (bFetching) // Don't stack a request over an ongoing one.
	{
		return;
	}

	IConcertClientPtr ConcertClient = WeakConcertClient.Pin();
	if (!ConcertClient)
	{
		LastErrorMsg = LOCTEXT("InvalidClient", "Failed to fetch activities from the server. Concert client pointer was invalidated.");
		return;
	}

	check((RequestFetchCount < 0 && LowestFetchedActivityId == 0) || (RequestFetchCount > 0 && LowestFetchedActivityId > 1));
	bFetching = true;

	// Compute the starting point of the next request. (We fetch backward, down to activity ID 1)
	int64 FetchFrom  = FMath::Max(1ll, LowestFetchedActivityId - RequestFetchCount); // Ignored by the server if 'RequestFetchCount' is negative (Tail query).
	int64 FetchCount = RequestFetchCount < 0 ? RequestFetchCount : FMath::Min(RequestFetchCount, LowestFetchedActivityId - 1);

	TWeakPtr<uint8, ESPMode::ThreadSafe> ContinuationExecutionToken = FetchRequestContinuationToken;
	ConcertClient->GetSessionActivities(AdminEndpointId, SessionId, FetchFrom, FetchCount, bIncludeActivityDetails).Next([this, RequestFetchCount, ContinuationExecutionToken](const FConcertAdmin_GetSessionActivitiesResponse& Response)
	{
		if (TSharedPtr<uint8, ESPMode::ThreadSafe> ExecutionToken = ContinuationExecutionToken.Pin()) // Ensures the 'this' object still exists.
		{
			if (Response.ResponseCode == EConcertResponseCode::Success)
			{
				// Merge the previous map with the latest one.
				EndpointClientInfoMap.Append(Response.EndpointClientInfoMap);

				// Received activities are in ascending order (older first), but the stream requirement is to store them in opposite order. (most recent first)
				for (int Index = Response.Activities.Num() - 1; Index >= 0; --Index)
				{
					FConcertSyncActivity* SyncActivity;
					FStructOnScope ActivityPayload;
					TUniquePtr<FConcertSessionSerializedPayload> EventPayload;

					Response.Activities[Index].GetPayload(ActivityPayload);
					if (ActivityPayload.GetStruct()->IsChildOf(FConcertSyncTransactionActivity::StaticStruct()))
					{
						FConcertSyncTransactionActivity* SyncTransactionActivity = (FConcertSyncTransactionActivity*)ActivityPayload.GetStructMemory();
						SyncActivity = SyncTransactionActivity;
						EventPayload = MakeUnique<FConcertSessionSerializedPayload>();
						EventPayload->SetTypedPayload(SyncTransactionActivity->EventData);
					}
					else if (ActivityPayload.GetStruct()->IsChildOf(FConcertSyncPackageActivity::StaticStruct()))
					{
						FConcertSyncPackageActivity* SyncPackageActivity = (FConcertSyncPackageActivity*)ActivityPayload.GetStructMemory();
						SyncActivity = SyncPackageActivity;
						EventPayload = MakeUnique<FConcertSessionSerializedPayload>();
						EventPayload->SetTypedPayload(SyncPackageActivity->EventData);
						checkf(!SyncPackageActivity->EventData.Package.HasPackageData(), TEXT("The concert activity stream should only stream the package meta data. The package data itself should not be required where ConcertActivityStream is used. (The package data can be very large)"));
					}
					else // bIncludeActivityDetails was false/Other type of activities (lock/connection) don't have interesting data to inspect other than the one already provided by the generic activity.
					{
						check(ActivityPayload.GetStruct()->IsChildOf(FConcertSyncActivity::StaticStruct()));
						SyncActivity = (FConcertSyncActivity*)ActivityPayload.GetStructMemory();
					}

					// Get the Activity summary.
					FStructOnScope EventSummaryPayload;
					SyncActivity->EventSummary.GetPayload(EventSummaryPayload);
					
					// Note: Moving the 'SyncActivity' slices the derived type on purpose when this is a transaction/package activity. Details are decoupled in EventPayload.
					AvailableActivities.Add(MakeShared<FConcertSessionActivity>(MoveTemp(*SyncActivity), MoveTemp(EventSummaryPayload), MoveTemp(EventPayload)));
				}

				if (RequestFetchCount > 0) // Response to second and subsequent requests.
				{
					check(Response.Activities.Num()); // We received 0 activities, but expected some. We fired a request because we did not reach to activity ID 1 yet.
					LowestFetchedActivityId = AvailableActivities.Last()->Activity.ActivityId;
					bAllActivitiesFetched = LowestFetchedActivityId == 1; // Activity ID are 1-based (1 being the very first activity)
				}
				else if (AvailableActivities.Num()) // Response to the first request, the tail query, used to discovery the max activity id.
				{
					TotalActivityCount = AvailableActivities[0]->Activity.ActivityId; // This is last activity recorded and has the max id which is also the total count (activity ids are 1-based)
					LowestFetchedActivityId = AvailableActivities.Last()->Activity.ActivityId; // The lowest activity ID fetched until now.
					bAllActivitiesFetched = LowestFetchedActivityId == 1; // Activity ID are 1-based (1 being the very first activity)
				}
				else // Response to the first request but no session activities reported. The session doesn't have any activities.
				{
					TotalActivityCount = 0;
					LowestFetchedActivityId = 0;
					bAllActivitiesFetched = true;
				}
			}
			else
			{
				LastErrorMsg = FText::Format(LOCTEXT("ActivityRequestFailed", "Failed to retrieve {0} activities from session '{1}'. Reason: {2}"), RequestFetchCount, FText::AsCultureInvariant(SessionId.ToString()), Response.Reason);
			}

			bFetching = false; // Clear the flag, allowing a new request.
		}
	});
}

#undef LOCTEXT_NAMESPACE
