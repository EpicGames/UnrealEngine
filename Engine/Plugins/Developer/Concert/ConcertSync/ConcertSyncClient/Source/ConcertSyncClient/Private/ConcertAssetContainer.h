// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataAsset.h"
#include "ConcertAssetContainer.generated.h"


// Forward declarations
class USoundBase;
class USoundCue;
class UStaticMesh;
class UMaterial;
class UMaterialInterface;
class UMaterialInstance;
class UFont;

/**
 * Asset container for VREditor.
 */
UCLASS(MinimalAPI)
class UConcertAssetContainer : public UDataAsset
{
	GENERATED_BODY()

public:

	//
	// Meshes
	//

	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> GenericDesktopMesh;

	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> GenericHMDMesh;

	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> VivePreControllerMesh;

	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> OculusControllerMesh;

	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> GenericControllerMesh;

	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> LaserPointerMesh;

	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> LaserPointerEndMesh;

	UPROPERTY(EditAnywhere, Category = Mesh)
	TObjectPtr<UStaticMesh> LaserPointerStartMesh;

	//
	// Materials
	//

	UPROPERTY(EditAnywhere, Category = Material)
	TObjectPtr<UMaterialInterface> PresenceMaterial;

	UPROPERTY(EditAnywhere, Category = Material)
	TObjectPtr<UMaterialInterface> TextMaterial;

	UPROPERTY(EditAnywhere, Category = Desktop)
	TObjectPtr<UMaterialInterface> HeadMaterial;

	UPROPERTY(EditAnywhere, Category = Material)
	TObjectPtr<UMaterialInterface> LaserCoreMaterial;

	UPROPERTY(EditAnywhere, Category = Material)
	TObjectPtr<UMaterialInterface> LaserMaterial;

	UPROPERTY(EditAnywhere, Category = Material)
	TObjectPtr<UMaterialInterface> PresenceFadeMaterial;
};
