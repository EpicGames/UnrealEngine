// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class IConcertClientPackageBridge;
class IConcertClientReplicationBridge;
class IConcertClientTransactionBridge;

namespace UE::ConcertSyncClient
{
	struct FConcertBridges
	{
		/** Package bridge used by sessions of this client */
		IConcertClientPackageBridge* PackageBridge;
		/** Transaction bridge used by sessions of this client */
		IConcertClientTransactionBridge* TransactionBridge;
		/** Replication bridge used by sessions of this client */
		IConcertClientReplicationBridge* ReplicationBridge;

		bool IsValid() const { return PackageBridge && TransactionBridge && ReplicationBridge; }
	};
}

