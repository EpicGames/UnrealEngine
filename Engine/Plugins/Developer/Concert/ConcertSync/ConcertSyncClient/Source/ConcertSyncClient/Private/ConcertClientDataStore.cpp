// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertClientDataStore.h"
#include "ConcertMessageData.h"
#include "IConcertSession.h"
#include "ConcertSyncClientLiveSession.h"

//------------------------------------------------------------------------------
// Utilities for internal usage.
//------------------------------------------------------------------------------

namespace ConcertDataStoreUtils
{
/** When the compare exchange payload gets bigger than this value, in bytes, the client will try to send the value version if available. */
static const uint32 CompareExchangePayloadSizeOptimizationThreshold = 64;

} // namespace ConcertDataStoreUtils

//------------------------------------------------------------------------------
// FConcertClientDataStore implementation.
//------------------------------------------------------------------------------

FConcertClientDataStore::FConcertClientDataStore(TSharedRef<FConcertSyncClientLiveSession> InLiveSession)
	: LiveSession(MoveTemp(InLiveSession))
{
	LiveSession->GetSession().RegisterCustomEventHandler<FConcertDataStore_ReplicateEvent>(this, &FConcertClientDataStore::OnReplicationEvent);
}

FConcertClientDataStore::~FConcertClientDataStore()
{
	LiveSession->GetSession().UnregisterCustomEventHandler<FConcertDataStore_ReplicateEvent>(this);
}

TFuture<FConcertDataStoreResult> FConcertClientDataStore::InternalFetchOrAdd(const FName& Key, const UScriptStruct* Type, const FName& TypeName, const void* Value)
{
	FScopeLock ScopeLock(&CritSection);

	// Check if the key/value is already cached.
	FConcertDataStoreResult Result = LocalCache.Fetch(Key, TypeName);
	if (Result.Code == EConcertDataStoreResultCode::TypeMismatch || Result.Code == EConcertDataStoreResultCode::Fetched)
	{
		// If type did not match, no need to call the server, just report the error. If value was cached, no need to fetch
		// the latest value, use the cached one. See the comment in InternalFetchAs().
		return MakeFulfilledPromise<FConcertDataStoreResult>(MoveTemp(Result)).GetFuture();
	}

	// Create the request
	FConcertDataStore_FetchOrAddRequest FetchOrAddRequest;
	FetchOrAddRequest.Key = Key;
	FetchOrAddRequest.TypeName = TypeName;
	FetchOrAddRequest.SerializedValue.SetPayload(Type, Value);

	return LiveSession->GetSession().SendCustomRequest<FConcertDataStore_FetchOrAddRequest, FConcertDataStore_Response>(FetchOrAddRequest, LiveSession->GetSession().GetSessionServerEndpointId()).Next(
		[this, SentKey = Key, SentTypeName = TypeName, SentValue = FetchOrAddRequest.SerializedValue](const FConcertDataStore_Response& Response)
	{
		return HandleResponse(SentKey, SentTypeName, SentValue, Response);
	});
}

TFuture<FConcertDataStoreResult> FConcertClientDataStore::InternalFetchAs(const FName& Key, const UScriptStruct* Type, const FName& TypeName) const
{
	FScopeLock ScopeLock(&CritSection);

	// Fetch is always run from the local cache. The server is expected to push new values to the client. If you are concern
	// that the client may use out of date values, know that even if the client reached the server and fetched the up to date
	// value, by the time the client receives it, the server may already have changed it again. So the client is never sure to
	// have the latest value.
	return MakeFulfilledPromise<FConcertDataStoreResult>(LocalCache.Fetch(Key, TypeName)).GetFuture();
}

TFuture<FConcertDataStoreResult> FConcertClientDataStore::InternalCompareExchange(const FName& Key, const UScriptStruct* Type, const FName& TypeName, const void* Expected, const void* Desired)
{
	FScopeLock ScopeLock(&CritSection);

	// If the key is already cached, ensure the type matches.
	FConcertDataStoreResult Result = LocalCache.Fetch(Key, TypeName);
	if (Result.Code == EConcertDataStoreResultCode::TypeMismatch || Result.Code == EConcertDataStoreResultCode::NotFound)
	{
		// No need to call the server, the types don't match, the user never fetched it or the server did not push the new key yet.
		return MakeFulfilledPromise<FConcertDataStoreResult>(MoveTemp(Result)).GetFuture();
	}

	// The key value was fetched from the cache.
	check(Result.Code == EConcertDataStoreResultCode::Fetched);

	FConcertDataStore_CompareExchangeRequest CompareExchangeRequest;
	CompareExchangeRequest.Key = Key;
	CompareExchangeRequest.TypeName = TypeName;

	// Serialize the expected value.
	CompareExchangeRequest.Expected.SetPayload(Type, Expected);

	// If the expected value matches the cached value.
	if (Result.Value->SerializedValue.PayloadBytes.Bytes == CompareExchangeRequest.Expected.PayloadBytes.Bytes)
	{
		// If the 'expected' payload is rather large.
		if (CompareExchangeRequest.Expected.PayloadSize > ConcertDataStoreUtils::CompareExchangePayloadSizeOptimizationThreshold)
		{
			// Send the 'version' rather than 'expected' to save bandwidth.
			CompareExchangeRequest.ExpectedVersion = Result.Value->Version;
			CompareExchangeRequest.Expected.PayloadTypeName = TEXT("");
			CompareExchangeRequest.Expected.PayloadCompressionMethod = EConcertCompressionMethod::Uncompressed;
			CompareExchangeRequest.Expected.PayloadCompressionBias = EConcertCompressionBias::None;
			CompareExchangeRequest.Expected.PayloadSize = 0;
			CompareExchangeRequest.Expected.PayloadBytes.Bytes.Reset();
		}
	}
	else
	{
		// No need to call the server, the expected value doesn't match the one in cache. This means the server
		// pushed a newer value to this client or the client did not use the latest value it fetched. Return
		// the latest value cached.
		return MakeFulfilledPromise<FConcertDataStoreResult>(MoveTemp(Result)).GetFuture();
	}

	// Serialize the desired value in the request.
	CompareExchangeRequest.Desired.SetPayload(Type, Desired);

	return LiveSession->GetSession().SendCustomRequest<FConcertDataStore_CompareExchangeRequest, FConcertDataStore_Response>(CompareExchangeRequest, LiveSession->GetSession().GetSessionServerEndpointId()).Next(
		[this, SentKey = Key, SentTypeName = TypeName, SentValue = CompareExchangeRequest.Desired](const FConcertDataStore_Response& Response)
	{
		return HandleResponse(SentKey, SentTypeName, SentValue, Response);
	});
}

void FConcertClientDataStore::InternalRegisterChangeNotificationHandler(const FName& Key, const FName& TypeName, const FChangeNotificationHandler& Handler, EConcertDataStoreChangeNotificationOptions Options)
{
	FScopeLock ScopeLock(&CritSection);

	ChangeNotificationHandlers.Add(Key, Handler);

	// If the caller want so get the initial value immediately.
	if ((Options & EConcertDataStoreChangeNotificationOptions::NotifyOnInitialValue) == EConcertDataStoreChangeNotificationOptions::NotifyOnInitialValue)
	{
		FConcertDataStoreResult Result = LocalCache.Fetch(Key, TypeName);
		if (Result.Code == EConcertDataStoreResultCode::Fetched)
		{
			// The key exists and the types match.
			Handler(Key, Result.Value.Get());
		}
		else if (Result.Code == EConcertDataStoreResultCode::TypeMismatch)
		{
			// The key exists, but types don't match. Still call the delegate. This call our wrapper delegate in IConcertClientDataStore and
			// the wrapper will see the value is null and interpret it as a type mismatch, then it will honor the user options to report
			// or not the key/value in such case.
			Handler(Key, nullptr);
		}
	}
}

void FConcertClientDataStore::InternalUnregisterChangeNotificationHandler(const FName& Key)
{
	FScopeLock ScopeLock(&CritSection);
	ChangeNotificationHandlers.Remove(Key);
}

void FConcertClientDataStore::OnReplicationEvent(const FConcertSessionContext& Context, const FConcertDataStore_ReplicateEvent& Event)
{
	FScopeLock ScopeLock(&CritSection);

	for (const FConcertDataStore_KeyValuePair& Pair : Event.Values)
	{
		// Cache the value. The client cache is designed to ensure the client always has an older value than the one pushed by
		// the server. The client is only allowed to cache values received from the server, it is forbidden to cache a value before
		// it gets acknowledged. For the same reason, we should ever get any type mismatch error.
		FConcertDataStoreResult Result = LocalCache.Store(Pair.Key, Pair.Value.TypeName, Pair.Value.SerializedValue);
		check(Result.Code != EConcertDataStoreResultCode::TypeMismatch);

		// Notify the observer(s) of this key about the change. As explained above, through this notification, the received value is
		// expected to always be more recent than the one cached. Also, we don't notify the client about its own changes. This assumes
		// the server will never send an update to the client that initiated the update.
		if (FChangeNotificationHandler* Handler = ChangeNotificationHandlers.Find(Pair.Key))
		{
			(*Handler)(Pair.Key, Result.Value.Get());
		}
	}
}

FConcertDataStoreResult FConcertClientDataStore::HandleResponse(const FName& SentKey, const FName& SentValueTypeName, const FConcertSessionSerializedPayload& SentValue, const FConcertDataStore_Response& Response)
{
	FConcertDataStoreResult Result;

	// When a value is added or exchanged, the server doesn't send back the value payload to save bandwidth. We need to cache the value we sent.
	if (Response.ResultCode == EConcertDataStoreResultCode::Added || Response.ResultCode == EConcertDataStoreResultCode::Exchanged)
	{
		// Ensure the server doesn't send data when the client initiated the operation (The client knows the value it sent).
		check(Response.Value.TypeName.IsNone());
		check(Response.Value.SerializedValue.PayloadTypeName.IsNone());
		check(Response.Value.SerializedValue.PayloadCompressionMethod == EConcertCompressionMethod::Uncompressed);
		check(Response.Value.SerializedValue.PayloadCompressionBias == EConcertCompressionBias::None);
		check(Response.Value.SerializedValue.PayloadBytes.Bytes.Num() == 0);

		// Ensure the server sent back a valid version in case it has exchanged the value. (Successfully added value is always version 1)
		check(Response.ResultCode == EConcertDataStoreResultCode::Added || (Response.ResultCode == EConcertDataStoreResultCode::Exchanged && Response.Value.Version > 0));

		// Ensure FetchOrAdd()/CompareExchange() called the FConcertDataStoreResponseHandler constructor that recorded the value sent.
		check(SentValue.PayloadBytes.Bytes.Num() > 0);

		// Add the value or update it in the cache, using the value we previously sent.
		FScopeLock ScopeLock(&CritSection);
		Result = LocalCache.Store(SentKey, SentValueTypeName, SentValue, Response.ResultCode == EConcertDataStoreResultCode::Added ? 1 : Response.Value.Version);
		Result.Code = Response.ResultCode;
	}
	else if (Response.ResultCode == EConcertDataStoreResultCode::Fetched)
	{
		// Populate or update the cache.
		check(Response.Value.TypeName == SentValueTypeName);
		FScopeLock ScopeLock(&CritSection);
		Result = LocalCache.Store(SentKey, Response.Value.TypeName, Response.Value.SerializedValue, Response.Value.Version);
		Result.Code = Response.ResultCode;
	}
	else
	{
		// TypeMismatch/NotFound/UnexpectedError -> Don't need to cache anything. Ensure the server did not send back a payload for it.
		// UnexpectedError -> This is Response.ResultCode default value. It is expected when a request time out because the Concert framework sends a default-constructed response.
		check(Response.ResultCode == EConcertDataStoreResultCode::UnexpectedError || Response.ResultCode == EConcertDataStoreResultCode::TypeMismatch || Response.ResultCode == EConcertDataStoreResultCode::NotFound);
		check(Response.Value.SerializedValue.PayloadTypeName.IsNone());
		check(Response.Value.SerializedValue.PayloadBytes.Bytes.Num() == 0);
		Result.Code = Response.ResultCode;
	}

	return Result;
}

FConcertDataStoreValueConstPtr FConcertClientDataStore::GetCachedValue(const FName& Key, const FName& TypeName) const
{
	FScopeLock ScopeLock(&CritSection);
	return LocalCache.Fetch(Key, TypeName).Value;
}

int32 FConcertClientDataStore::GetCacheSize() const
{
	FScopeLock ScopeLock(&CritSection);
	return LocalCache.GetSize();
}

//------------------------------------------------------------------------------
// Utilities made available for testing purpose. Many of these functions are
// declared "extern" in ConcertSyncTest module because I did not want to expose
// ConcertClientDataStore.h publicly just for testing sake as this is not
// required for normal usage. The code below provides a way to instantiate
// a FConcertClientDataStore and look into its cache.
//------------------------------------------------------------------------------
namespace UE::ConcertSyncTests::DatastoreTests
{

// This class is private and is meant to be indirectly used by the ConcertSyncTest module only.
// It enables looking up the content of the client cache.
class FConcertClientDataStoreTest : public FConcertClientDataStore
{
public:
	FConcertClientDataStoreTest(TSharedRef<IConcertClientSession> Session) : FConcertClientDataStore(MakeShared<FConcertSyncClientLiveSession>(MoveTemp(Session), EConcertSyncSessionFlags::None)) {}

	// Change the accessibility of those methods from protected to public.
	using FConcertClientDataStore::GetCachedValue;
	using FConcertClientDataStore::GetCacheSize;
};

// Creates an instance of FConcertClientDataStoreTest for testing purpose only. Expected to be used by ConcertSyncTest module only.
CONCERTSYNCCLIENT_API TSharedRef<IConcertClientDataStore> MakeConcertClientDataStoreForTest(TSharedRef<IConcertClientSession> Session)
{
	return MakeShared<FConcertClientDataStoreTest>(MoveTemp(Session));
}

// Returns the size of the client data store cache. Expected to be used by ConcertSyncTest module only.
CONCERTSYNCCLIENT_API int32 GetConcertClientDataStoreCacheSize(const IConcertClientDataStore& ClientStore)
{
	// We expecte the client to pass the instance returned by MakeConcertClientDataStoreForTest()
	const FConcertClientDataStoreTest& DataStoreClientTest = static_cast<const FConcertClientDataStoreTest&>(ClientStore);
	return DataStoreClientTest.GetCacheSize();
}

// Returns the value (if any) cached in the client data store. Expected to be used by ConcertSyncTest module only.
CONCERTSYNCCLIENT_API FConcertDataStoreValueConstPtr GetConcertClientDataStoreCachedValue(const IConcertClientDataStore& ClientStore, const FName& Key, const FName& TypeName)
{
	// We expecte the client to pass the instance returned by MakeConcertClientDataStoreForTest()
	const FConcertClientDataStoreTest& DataStoreClientTest = static_cast<const FConcertClientDataStoreTest&>(ClientStore);
	return DataStoreClientTest.GetCachedValue(Key, TypeName);
}

// Returns the threshold at which the client will send the version of a value rather than its payload because it will be cheaper. Expected to be used by ConcertSyncTest module only.
CONCERTSYNCCLIENT_API uint32 GetCompareExchangePayloadOptimizationThreshold()
{
	return ConcertDataStoreUtils::CompareExchangePayloadSizeOptimizationThreshold;
}

} // namespace ConcertDataStoreTestUtils
