// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IConcertClientDataStore.h"
#include "ConcertDataStore.h"
#include "ConcertDataStoreMessages.h"

class FConcertSyncClientLiveSession;
struct FConcertSessionContext;

/**
 * Stores key/value pairs in a data structure shared by all clients in the same
 * Concert session.
 *
 * @remarks The implementation is thread safe, enabling clients to call from any thread.
 */
class FConcertClientDataStore : public IConcertClientDataStore
{
public:
	/**
	 * Constructs a data store on the client side.
	 * @param InSession The session owning this store and used to send requests and receive responses or events.
	 */
	FConcertClientDataStore(TSharedRef<FConcertSyncClientLiveSession> InLiveSession);
	virtual ~FConcertClientDataStore();

protected:
	// Begin IConcertClientDataStore interface.
	virtual TFuture<FConcertDataStoreResult> InternalFetchOrAdd(const FName& Key, const UScriptStruct* Type, const FName& TypeName, const void* Payload) override;
	virtual TFuture<FConcertDataStoreResult> InternalFetchAs(const FName& Key, const UScriptStruct* Type, const FName& TypeName) const override;
	virtual TFuture<FConcertDataStoreResult> InternalCompareExchange(const FName& Key, const UScriptStruct* Type, const FName& TypeName, const void* Expected, const void* Desired) override;
	virtual void InternalRegisterChangeNotificationHandler(const FName& Key, const FName& TypeName, const FChangeNotificationHandler& Handler, EConcertDataStoreChangeNotificationOptions Options) override;
	virtual void InternalUnregisterChangeNotificationHandler(const FName& Key) override;
	// End IConcertClientDataStore interface.

	FConcertDataStoreResult HandleResponse(const FName& SentKey, const FName& SentValueTypeName, const FConcertSessionSerializedPayload& SentValue, const FConcertDataStore_Response& Response);

	// Returns the data store cache, enabling derived classes to read it. Useful for testing purpose.
	FConcertDataStoreValueConstPtr GetCachedValue(const FName& Key, const FName& TypeName) const;
	int32 GetCacheSize() const;

private:
	/** Handle replication events sent by the Concert data store server. */
	void OnReplicationEvent(const FConcertSessionContext& Context, const FConcertDataStore_ReplicateEvent& Event);

	/** The session used to dispatch requests. */
	TSharedRef<FConcertSyncClientLiveSession> LiveSession;

	/** Critical section to support clients calling from multiple threads. */
	mutable FCriticalSection CritSection;

	/** A replicated cache of the server store. Not thread safe by itself. Ensure to lock CritSection before usage. */
	FConcertDataStore LocalCache;

	/** Map the keys observed by the client to its change notification handlers. Ensure to lock CritSection before usage. */
	TMap<FName, FChangeNotificationHandler> ChangeNotificationHandlers;
};
