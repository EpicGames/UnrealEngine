// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertClientDesktopPresenceActor.h"
#include "Components/StaticMeshComponent.h"
#include "Components/SplineMeshComponent.h"
#include "Engine/EngineTypes.h"
#include "IConcertSession.h"
#include "ConcertPresenceEvents.h"
#include "ConcertClientPresenceManager.h"

#if WITH_EDITOR
#include "EditorViewportClient.h"
//#include "ConcertClientPresenceMode.h"
#include "Editor.h"
#endif
#include "TimerManager.h"
#include "ConcertAssetContainer.h"
#include "Materials/MaterialInstanceDynamic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConcertClientDesktopPresenceActor)


AConcertClientDesktopPresenceActor::AConcertClientDesktopPresenceActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DesktopMeshComponent = CreateDefaultSubobject<UStaticMeshComponent>(TEXT("Desktop"));
	AddOwnedComponent(DesktopMeshComponent);
	DesktopMeshComponent->SetMobility(EComponentMobility::Movable);
	DesktopMeshComponent->SetupAttachment(RootComponent);

	LaserPointer = CreateDefaultSubobject<USplineMeshComponent>(TEXT("Pointer"));
	AddOwnedComponent(LaserPointer);
	LaserPointer->SetMobility(EComponentMobility::Movable);
	LaserPointer->SetupAttachment(RootComponent);
	LaserPointer->SetCollisionEnabled(ECollisionEnabled::NoCollision);
	LaserPointer->SetbNeverNeedsCookedCollisionData(true);

	LastEndPoint = FVector::ZeroVector;
	bIsLaserVisible = true;

	bLastKnownMovingCamera = false;
	bMovingCamera = false;
}

void AConcertClientDesktopPresenceActor::BeginDestroy()
{
	Super::BeginDestroy();
}

void AConcertClientDesktopPresenceActor::SetPresenceColor(const FLinearColor& InColor)
{
	static const FName LaserColorParam(TEXT("UserColor"));
	static const FName BodyColorParam(TEXT("Color"));
	static const FName ColorParamName(TEXT("Color"));

	LaserMid->SetVectorParameterValue(LaserColorParam, InColor);
	LaserCoreMid->SetVectorParameterValue(LaserColorParam, InColor);
	PresenceMID->SetVectorParameterValue(BodyColorParam, InColor);
	TextMID->SetVectorParameterValue(ColorParamName, InColor);
}

void AConcertClientDesktopPresenceActor::InitPresence(const class UConcertAssetContainer& InAssetContainer, FName DeviceType)
{
	Super::InitPresence(InAssetContainer, DeviceType);

	UMaterialInterface* PresenceMaterial = InAssetContainer.PresenceFadeMaterial;

	PresenceMID = UMaterialInstanceDynamic::Create(PresenceMaterial, this);
	DesktopMeshComponent->SetMaterial(0, PresenceMID);

	UMaterialInterface* LaserMaterial = InAssetContainer.LaserMaterial;
	LaserMid = UMaterialInstanceDynamic::Create(LaserMaterial, this);

	UMaterialInterface* LaserCoreMaterial = InAssetContainer.LaserCoreMaterial;
	LaserCoreMid = UMaterialInstanceDynamic::Create(LaserCoreMaterial, this);

	LaserPointer->SetMaterial(0, LaserCoreMid);
	LaserPointer->SetMaterial(1, LaserMid);
}

void AConcertClientDesktopPresenceActor::HandleEvent(const FStructOnScope& InEvent)
{
	if (InEvent.GetStruct() == FConcertClientDesktopPresenceUpdateEvent::StaticStruct())
	{
		if (const FConcertClientDesktopPresenceUpdateEvent* Event = (const FConcertClientDesktopPresenceUpdateEvent*)InEvent.GetStructMemory())
		{
			const double TimestampSeconds = FPlatformTime::Seconds();
			const double LocationUpdateFrequency = FConcertClientPresenceManager::GetLocationUpdateFrequency();

			auto UpdateLastKnownLocation = [&TimestampSeconds, &LocationUpdateFrequency](TOptional<FConcertClientMovement>& MovementObject, const FVector& Position)
			{
				if (!MovementObject.IsSet())
				{
					MovementObject = FConcertClientMovement(LocationUpdateFrequency, TimestampSeconds, Position);
				}
				else
				{
					MovementObject->UpdateLastKnownLocation(TimestampSeconds, Position);
				}
			};

			UpdateLastKnownLocation(LaserStartMovement, Event->TraceStart);
			UpdateLastKnownLocation(LaserEndMovement, Event->TraceEnd);

			bLastKnownMovingCamera = Event->bMovingCamera;
		}
	}
	else
	{
		Super::HandleEvent(InEvent);
	}
}

void AConcertClientDesktopPresenceActor::SetLaserTimer(FTimerManager& InTimerManager, bool bLaserShouldShow)
{
	if (bIsLaserVisible && !bLaserShouldShow && !InTimerManager.IsTimerActive(LaserTimerHandle))
	{
		InTimerManager.SetTimer(LaserTimerHandle, this, &AConcertClientDesktopPresenceActor::HideLaser, 5.0f);
	}
	else if (bLaserShouldShow)
	{
		InTimerManager.ClearTimer(LaserTimerHandle);
	}
}

void AConcertClientDesktopPresenceActor::Tick(float DeltaSeconds)
{
	Super::Tick(DeltaSeconds);

	if (LaserStartMovement.IsSet() && LaserEndMovement.IsSet())
	{
		FVector LaserStartPosition;
		FVector LaserEndPosition;
		LaserStartMovement->MoveSmooth(DeltaSeconds, LaserStartPosition);
		LaserEndMovement->MoveSmooth(DeltaSeconds, LaserEndPosition);

		// Transform the trace data from world space to component space
		FTransform WorldToComponent = LaserPointer->GetComponentToWorld().Inverse();

		FVector LocalTraceStart = WorldToComponent.TransformPosition(LaserStartPosition);
		// zero out local x so it aligns with the desktop mesh
		LocalTraceStart.X = 0;
		FVector LocalTraceEnd = WorldToComponent.TransformPosition(LaserEndPosition);

		bool bLaserShouldShow = bMovingCamera || FVector::DistSquared(LastEndPoint, LaserEndPosition) > 5.0f;

#if WITH_EDITOR
		if (GEditor)
		{
			TSharedRef<FTimerManager> WorldTimerManager = GEditor->GetTimerManager();
			SetLaserTimer(*WorldTimerManager, bLaserShouldShow);
		}
		else
#endif
		{
			FTimerManager& WorldTimerManager = GetWorldTimerManager();
			SetLaserTimer(WorldTimerManager, bLaserShouldShow);
		}

		if (!bIsLaserVisible && bLaserShouldShow)
		{
			ShowLaser();
		}

		LastEndPoint = LaserEndPosition;

		if (bMovingCamera != bLastKnownMovingCamera)
		{
			bMovingCamera = bLastKnownMovingCamera;
			LaserPointer->SetStartAndEnd(LocalTraceStart, FVector(1, 0, 0), LocalTraceEnd, FVector(1, 0, 0));
		}
		else if (!bMovingCamera)
		{
			LaserPointer->SetStartAndEnd(LocalTraceStart, FVector(1, 0, 0), LocalTraceEnd, FVector(1, 0, 0));
		}
	}
}

void AConcertClientDesktopPresenceActor::HideLaser()
{
	bIsLaserVisible = false;
	LaserPointer->SetVisibility(false);

	if (LaserStartMovement.IsSet())
	{
		LaserStartMovement.Reset();
	}

	if (LaserEndMovement.IsSet())
	{
		LaserEndMovement.Reset();
	}
}

void AConcertClientDesktopPresenceActor::ShowLaser()
{
	bIsLaserVisible = true;
	LaserPointer->SetVisibility(true);
}
