// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertClientLiveTransactionAuthors.h"
#include "ConcertSyncClientLiveSession.h"
#include "ConcertSyncSessionDatabase.h"
#include "ConcertSyncSessionTypes.h"
#include "IConcertSession.h"

FConcertClientLiveTransactionAuthors::FConcertClientLiveTransactionAuthors(TSharedRef<FConcertSyncClientLiveSession> InLiveSession)
	: LiveSession(MoveTemp(InLiveSession))
{
	ResolveLiveTransactionAuthors();
}

FConcertClientLiveTransactionAuthors::~FConcertClientLiveTransactionAuthors()
{
}

void FConcertClientLiveTransactionAuthors::ResolveLiveTransactionAuthors()
{
	OtherEndpointsWithLiveTransactionsMap.Reset();

	TArray<int64> LiveTransactionEventIds;
	if (LiveSession->GetSessionDatabase().GetLiveTransactionEventIds(LiveTransactionEventIds))
	{
		for (const int64 LiveTransactionEventId : LiveTransactionEventIds)
		{
			FConcertSyncTransactionActivity TransactionActivity;
			if (LiveSession->GetSessionDatabase().GetTransactionActivityForEvent(LiveTransactionEventId, TransactionActivity))
			{
				AddLiveTransactionActivity(TransactionActivity.EndpointId, TransactionActivity.EventData.Transaction.ModifiedPackages);
			}
		}
	}
}

void FConcertClientLiveTransactionAuthors::ResolveLiveTransactionAuthorsForPackage(const FName& PackageName)
{
	OtherEndpointsWithLiveTransactionsMap.Remove(PackageName);

	TArray<int64> LiveTransactionEventIds;
	if (LiveSession->GetSessionDatabase().GetLiveTransactionEventIdsForPackage(PackageName, LiveTransactionEventIds))
	{
		for (const int64 LiveTransactionEventId : LiveTransactionEventIds)
		{
			FConcertSyncTransactionActivity TransactionActivity;
			if (LiveSession->GetSessionDatabase().GetTransactionActivityForEvent(LiveTransactionEventId, TransactionActivity))
			{
				AddLiveTransactionActivity(TransactionActivity.EndpointId, TArrayView<const FName>(&PackageName, 1));
			}
		}
	}
}

void FConcertClientLiveTransactionAuthors::AddLiveTransactionActivity(const FGuid& EndpointId, TArrayView<const FName> ModifiedPackages)
{
	// Ignore this transaction if we generated it
	if (EndpointId == LiveSession->GetSession().GetSessionClientEndpointId())
	{
		return;
	}

	// Skip this transaction if its endpoint is already in the list of endpoints that have made changes to this package
	TArray<FName, TInlineAllocator<2>> PackagesToProcess;
	for (const FName& ModifiedPackage : ModifiedPackages)
	{
		const TArray<FGuid>* OtherEndpointsWithLiveTransactions = OtherEndpointsWithLiveTransactionsMap.Find(ModifiedPackage);
		if (!OtherEndpointsWithLiveTransactions || !OtherEndpointsWithLiveTransactions->Contains(EndpointId))
		{
			PackagesToProcess.Add(ModifiedPackage);
		}
	}
	if (PackagesToProcess.Num() == 0)
	{
		return;
	}

	// Check to see if the other client is in our list of "other" clients
	// If so then it cannot possibly be us
	bool bClientIsConnected = false;
	{
		FConcertSessionClientInfo ConnectedClientInfo;
		bClientIsConnected = LiveSession->GetSession().FindSessionClient(EndpointId, ConnectedClientInfo);
	}
	if (!bClientIsConnected)
	{
		// If the client isn't connected, get the data for the endpoint and see if it looks like a previous version of us
		// Ignore this transaction if so
		FConcertSyncEndpointData EndpointData;
		if (!LiveSession->GetSessionDatabase().GetEndpoint(EndpointId, EndpointData))
		{
			return;
		}
		const FConcertClientInfo& ThisClient = LiveSession->GetSession().GetLocalClientInfo();
		if (EndpointData.ClientInfo.UserName == ThisClient.UserName &&
			EndpointData.ClientInfo.DeviceName == ThisClient.DeviceName &&
			EndpointData.ClientInfo.PlatformName == ThisClient.PlatformName &&
			EndpointData.ClientInfo.DisplayName == ThisClient.DisplayName
			)
		{
			return;
		}
	}

	// Otherwise, this change must be from another endpoint so track it here
	for (const FName& PackageToProcess : PackagesToProcess)
	{
		TArray<FGuid>& OtherEndpointsWithLiveTransactions = OtherEndpointsWithLiveTransactionsMap.FindOrAdd(PackageToProcess);
		OtherEndpointsWithLiveTransactions.Add(EndpointId);
	}
}

bool FConcertClientLiveTransactionAuthors::IsPackageAuthoredByOtherClients(const FName& PackageName, int32* OutOtherClientsWithModifNum, TArray<FConcertClientInfo>* OutOtherClientsWithModifInfo, int32 OtherClientsWithModifMaxFetchNum) const
{
	int32 OtherClientWithModifNum = 0;

	if (const TArray<FGuid>* OtherEndpointsWithLiveTransactions = OtherEndpointsWithLiveTransactionsMap.Find(PackageName))
	{
		OtherClientWithModifNum = OtherEndpointsWithLiveTransactions->Num();

		// The caller wants to know which other client(s) modified the specified package.
		if (OutOtherClientsWithModifInfo && OtherClientsWithModifMaxFetchNum > 0 && OtherClientWithModifNum > 0)
		{
			for (const FGuid& EndpointId : *OtherEndpointsWithLiveTransactions)
			{
				FConcertSyncEndpointData EndpointData;
				LiveSession->GetSessionDatabase().GetEndpoint(EndpointId, EndpointData);
				OutOtherClientsWithModifInfo->Emplace(MoveTemp(EndpointData.ClientInfo));
				if (--OtherClientsWithModifMaxFetchNum == 0)
				{
					break;
				}
			}
		}
	}

	// The caller wants to know how many other client(s) modified the specified package.
	if (OutOtherClientsWithModifNum)
	{
		*OutOtherClientsWithModifNum = OtherClientWithModifNum;
	}

	// Returns if the specified package was modified by other clients.
	return OtherClientWithModifNum > 0;
}
