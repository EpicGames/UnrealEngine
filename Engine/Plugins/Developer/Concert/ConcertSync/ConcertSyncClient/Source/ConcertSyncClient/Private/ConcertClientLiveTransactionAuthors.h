// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConcertMessages.h"
#include "ConcertMessageData.h"

class FConcertSyncClientLiveSession;
struct FConcertSyncTransactionActivity;

/**
 * Tracks which client authored which package using the set of transactions that
 * haven't been saved to disk yet, also known as 'Live Transactions'. The purpose
 * of this class is to answer the question "has anybody, other than this client, modified
 * a package?". This is useful when multiple clients are concurrently editing the
 * same package in the same session. When someone is about to save, it might be
 * important to know if somebody else has modified the package and if so, review the
 * other people changes. This feature is integrated in the editor to visually mark
 * the assets modified by other clients.
 *
 * @par User authentication
 * The UE Editor doesn't use a mechanism like login/password to authenticate
 * a user. To uniquely identify a user, Concert generates a unique GUID
 * for each UE Editor instance. The same person may open/close the editor
 * several times or run multiple instances in parallel. Each editor instance
 * will get a new unique GUID. When the same user runs the editor in
 * parallel, the user will be recognized as two different people. When a user
 * exits the editor (or crash), then rejoins a session from a new editor instance,
 * the implementation will try to match its new identity to its previous one
 * and then assign all live transactions performed using the previous identity
 * to the new identity, if the user name, display name, machine name, ... match.
 *
 * @par Thread-safety
 * This class is currently called form the UI and Concert network layer, both
 * running in the game thread. For this reason, the class doesn't implement
 * internal synchronization.
 *
 * @par Design considerations
 * The transaction ledger doesn't track the users performing the transactions.
 * The functionality provided by this class could arguably be moved in the
 * transaction ledger, but this class could easily be implemented client
 * side only using the information already available in the transaction and
 * activity ledger.
 *
 * @note
 * For completeness, the functions below could be implemented, but they were not because they were not required for the actual use case.
 *     - bool IsPackageAuthoredByThisClient(const FName& PackageName) const;
 *     - const FConcertClientInfo& GetThisClientInfo() const;
 *     - TArray<FName> GetPackagesAuthoredBy(const FConcertClientInfo& ClientInfo) const;
 *     - TArray<FName> GetAuthoredPackages() const;
 *     - const FConcertClientInfo& GetLastPackageAuthor(const FName& PackageName) const;
 */
class FConcertClientLiveTransactionAuthors
{
public:
	/** Constructor.
	 * @param Session This local client session, used to identify this client against other clients connected to the session.
	 */
	FConcertClientLiveTransactionAuthors(TSharedRef<FConcertSyncClientLiveSession> InLiveSession);

	/** Destructor. */
	~FConcertClientLiveTransactionAuthors();

	/**
	 * Adds live transaction activity
	 * @param EndpointId The ID of the endpoint that made the transaction.
	 * @param ModifiedPackages The packages that were modified by the transaction.
	 */
	void AddLiveTransactionActivity(const FGuid& EndpointId, TArrayView<const FName> ModifiedPackages);

	/**
	 * Gets all live transactions for the given package and processes the author of each transaction to see if they were made by other clients
	 */
	void ResolveLiveTransactionAuthorsForPackage(const FName& PackageName);

	/**
	 * Returns true if the specified packages has live transaction(s) from any other client(s) than the one corresponding
	 * to the client session passed at construction and possibly returns information about the other clients.
	 * @param[in] PackageName The package name.
	 * @param[out] OutOtherClientsWithModifNum If not null, will contain how many other client(s) have modified the specified package.
	 * @param[out] OutOtherClientsWithModifInfo If not null, will contain the other client who modified the packages, up to OtherClientsWithModifMaxFetchNum.
	 * @param[in] OtherClientsWithModifMaxFetchNum The maximum number of client info to store in OutOtherClientsWithModifInfo if the latter is not null.
	 */
	bool IsPackageAuthoredByOtherClients(const FName& PackageName, int32* OutOtherClientsWithModifNum = nullptr, TArray<FConcertClientInfo>* OutOtherClientsWithModifInfo = nullptr, int32 OtherClientsWithModifMaxFetchNum = 0) const;

private:
	/**
	 * Gets all live transactions and processes the author of each transaction to see if they were made by other clients
	 */
	void ResolveLiveTransactionAuthors();

	/** The client session. */
	TSharedRef<FConcertSyncClientLiveSession> LiveSession;

	/** Maps package names to the list of endpoints (other than this one) that have live transactions on a package. */
	TMap<FName, TArray<FGuid>> OtherEndpointsWithLiveTransactionsMap;
};
