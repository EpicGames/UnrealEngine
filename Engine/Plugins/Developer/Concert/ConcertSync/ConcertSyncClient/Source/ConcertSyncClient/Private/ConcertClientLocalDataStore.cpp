// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertClientLocalDataStore.h"
#include "ConcertDataStore.h"

FConcertClientLocalDataStore::FConcertClientLocalDataStore()
	: DataStore(MakeUnique<FConcertDataStore>())
{
}

FConcertDataStoreResult FConcertClientLocalDataStore::InternalFetch(const FName& Key, const FName& TypeName) const
{
	return DataStore->Fetch(Key, TypeName);
}

FConcertDataStoreResult FConcertClientLocalDataStore::InternalStore(const FName& Key, const FName& Typename, const FConcertSessionSerializedPayload& Value)
{
	return DataStore->Store(Key, Typename, Value);
}
