// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertClientLockManager.h"
#include "IConcertSession.h"
#include "ConcertSyncClientLiveSession.h"
#include "ConcertLogGlobal.h"

#include "Engine/Engine.h"
#include "UObject/Package.h"

#if WITH_EDITOR
	#include "Editor.h"
#endif

#define LOCTEXT_NAMESPACE "ConcertClientLockManager"

FConcertClientLockManager::FConcertClientLockManager(TSharedRef<FConcertSyncClientLiveSession> InLiveSession)
	: LiveSession(InLiveSession)
{
#if WITH_EDITOR
	if (GIsEditor)
	{
		// Back up 'package ok to save delegate' and install ours
		OkToSaveBackupDelegate = FCoreUObjectDelegates::IsPackageOKToSaveDelegate;
		FCoreUObjectDelegates::IsPackageOKToSaveDelegate.BindRaw(this, &FConcertClientLockManager::CanSavePackage);

		// Install delegate for the OnAssetsCanDelete call
		FEditorDelegates::OnAssetsCanDelete.AddRaw(this, &FConcertClientLockManager::CanDeleteAssets);
	}
#endif	// WITH_EDITOR

	LiveSession->GetSession().RegisterCustomEventHandler<FConcertResourceLockEvent>(this, &FConcertClientLockManager::HandleResourceLockEvent);
}

FConcertClientLockManager::~FConcertClientLockManager()
{
#if WITH_EDITOR
	// Restore 'is ok to save package' delegate
	if (OkToSaveBackupDelegate.IsBound())
	{
		FCoreUObjectDelegates::IsPackageOKToSaveDelegate = OkToSaveBackupDelegate;
		OkToSaveBackupDelegate.Unbind();
	}

	FEditorDelegates::OnAssetsCanDelete.RemoveAll(this);
#endif	// WITH_EDITOR

	LiveSession->GetSession().UnregisterCustomEventHandler<FConcertResourceLockEvent>(this);
}

FGuid FConcertClientLockManager::GetWorkspaceLockId() const
{
	return LiveSession->GetSession().GetSessionClientEndpointId();
}

FGuid FConcertClientLockManager::GetResourceLockId(const FName InResourceName) const
{
	return LockedResources.FindRef(InResourceName);
}

bool FConcertClientLockManager::AreResourcesLockedBy(TArrayView<const FName> ResourceNames, const FGuid& ClientId)
{
	for (const FName& ResourceName : ResourceNames)
	{
		if (LockedResources.FindRef(ResourceName) != ClientId)
		{
			return false;
		}
	}
	return true;
}

TFuture<FConcertResourceLockResponse> FConcertClientLockManager::LockResources(TArray<FName> InResourceNames)
{
	FConcertResourceLockRequest Request{ LiveSession->GetSession().GetSessionClientEndpointId(), MoveTemp(InResourceNames), EConcertResourceLockType::Lock };
	return LiveSession->GetSession().SendCustomRequest<FConcertResourceLockRequest, FConcertResourceLockResponse>(Request, LiveSession->GetSession().GetSessionServerEndpointId());
}

TFuture<FConcertResourceLockResponse> FConcertClientLockManager::UnlockResources(TArray<FName> InResourceNames)
{
	FConcertResourceLockRequest Request{ LiveSession->GetSession().GetSessionClientEndpointId(), MoveTemp(InResourceNames), EConcertResourceLockType::Unlock };
	return LiveSession->GetSession().SendCustomRequest<FConcertResourceLockRequest, FConcertResourceLockResponse>(Request, LiveSession->GetSession().GetSessionServerEndpointId());
}

void FConcertClientLockManager::SetLockedResources(const TMap<FName, FGuid>& InLockedResources)
{
	LockedResources = InLockedResources;
}

bool FConcertClientLockManager::CanSavePackage(UPackage* InPackage, const FString& InFilename, FOutputDevice* ErrorLog)
{
	if (!GEngine->IsAutosaving())
	{
		FGuid LockOwner = LockedResources.FindRef(InPackage->GetFName());
		if (LockOwner.IsValid() && LockOwner != GetWorkspaceLockId())
		{
			ErrorLog->Log(TEXT("LogConcert"), ELogVerbosity::Warning, FString::Printf(TEXT("Package %s currently locked by another user."), *InPackage->GetFName().ToString()));
			return false;
		}
	}
#if WITH_EDITOR
	if (OkToSaveBackupDelegate.IsBound())
	{
		return OkToSaveBackupDelegate.Execute(InPackage, InFilename, ErrorLog);
	}
#endif
	return true;
}

void FConcertClientLockManager::CanDeleteAssets(const TArray<UObject*>& InAssetsToDelete, FCanDeleteAssetResult& CanDeleteResult)
{
#if WITH_EDITOR
	for (UObject* Obj : InAssetsToDelete)
	{
		// For asset we lock the package path, so validate against the package path
		FGuid LockOwner = LockedResources.FindRef(Obj->GetOutermost()->GetFName());
		if (LockOwner.IsValid() && LockOwner != GetWorkspaceLockId())
		{
			UE_LOGF(LogConcert, Warning, "Asset %ls can't be deleted because it is currently locked by another user.", *Obj->GetPathName());
			CanDeleteResult.Set(false);
		}
	}
#endif
}

void FConcertClientLockManager::HandleResourceLockEvent(const FConcertSessionContext& Context, const FConcertResourceLockEvent& Event)
{
	switch (Event.LockType)
	{
	case EConcertResourceLockType::Lock:
		for (const FName& ResourceName : Event.ResourceNames)
		{
			LockedResources.FindOrAdd(ResourceName) = Event.ClientId;
		}
		break;
	case EConcertResourceLockType::Unlock:
		for (const FName& ResourceName : Event.ResourceNames)
		{
			LockedResources.Remove(ResourceName);
		}
		break;
	default:
		// no-op
		break;
	}
}

#undef LOCTEXT_NAMESPACE
