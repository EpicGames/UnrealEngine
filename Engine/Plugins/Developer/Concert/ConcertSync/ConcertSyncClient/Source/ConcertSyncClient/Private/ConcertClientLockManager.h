// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConcertWorkspaceMessages.h"
#include "Async/Future.h"

class FConcertSyncClientLiveSession;
struct FConcertSessionContext;

class UObject;
class UPackage;
class FOutputDevice;
struct FCanDeleteAssetResult;

class FConcertClientLockManager
{
public:
	explicit FConcertClientLockManager(TSharedRef<FConcertSyncClientLiveSession> InLiveSession);
	~FConcertClientLockManager();

	/**
	 * @return the client id this workspace uses to lock resources.
	 */
	FGuid GetWorkspaceLockId() const;

	/**
	 * @return a valid client id of the owner of this resource lock or an invalid id if unlocked
	 */
	FGuid GetResourceLockId(const FName InResourceName) const;
	
	/**
	 * Verify if resources are locked by a particular client
	 * @param ResourceNames list of resources path to verify
	 * @param ClientId the client id to verify
	 * @return true if all resources in ResourceNames are locked by ClientId
	 * @note passing an invalid client id will return true if all resources are unlocked
	 */
	bool AreResourcesLockedBy(TArrayView<const FName> ResourceNames, const FGuid& ClientId);
	
	/**
	 * Attempt to lock the given resource.
	 * @note Passing force will always assign the lock to the given endpoint, even if currently locked by another.
	 * @return True if the resource was locked (or already locked by the given endpoint), false otherwise.
	 */
	TFuture<FConcertResourceLockResponse> LockResources(TArray<FName> InResourceNames);
	
	/**
	 * Attempt to unlock the given resource.
	 * @note Passing force will always clear, even if currently locked by another endpoint.
	 * @return True if the resource was unlocked, false otherwise.
	 */
	TFuture<FConcertResourceLockResponse> UnlockResources(TArray<FName> InResourceNames);

	/**
	 * Set the set of locked resources (resource ID -> endpoint ID).
	 */
	void SetLockedResources(const TMap<FName, FGuid>& InLockedResources);

private:
	/**
	 * Called prior to saving a package to see whether the save should be allowed to proceed.
	 */
	bool CanSavePackage(UPackage* InPackage, const FString& InFilename, FOutputDevice* ErrorLog);

	/**
	 * Called prior to deleting a package asset to see whether the delete should be allowed to proceed.
	 */
	void CanDeleteAssets(const TArray<UObject*>& InAssetsToDelete, FCanDeleteAssetResult& CanDeleteResult);

	/**
	 * Called to handle a resource being locked or unlocked by the server.
	 */
	void HandleResourceLockEvent(const FConcertSessionContext& Context, const FConcertResourceLockEvent& Event);

	/**
	 * Session instance this package manager was created for.
	 */
	TSharedPtr<FConcertSyncClientLiveSession> LiveSession;

	/**
	 * Tracks locked resources from the server (resource ID -> endpoint ID).
	 */
	TMap<FName, FGuid> LockedResources;

#if WITH_EDITOR
	/** */
	FCoreUObjectDelegates::FIsPackageOKToSaveDelegate OkToSaveBackupDelegate;
#endif	// WITH_EDITOR
};
