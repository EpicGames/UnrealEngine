// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertClientMovement.h"
#include "CoreMinimal.h"
#include "EngineGlobals.h"
#include "UObject/ConstructorHelpers.h"
#include "EngineGlobals.h"
#include "Components/StaticMeshComponent.h"
#include "Components/TextRenderComponent.h"
#include "ConcertAssetContainer.h"
#include "ConcertClientPresenceMode.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "ConcertClientPresenceManager.h"
#include "ConcertLogGlobal.h"

/** Constructor */
FConcertClientMovement::FConcertClientMovement(double InUpdateFrequencyTimeSeconds, double InCurrentTimestamp, const FVector& InPosition)
	: PreviousPosition(InPosition)
	, PreviousOrientation(FQuat::Identity)
	, LastKnownPosition(InPosition)
	, LastKnownOrientation(FQuat::Identity)
	, CurrentPosition(InPosition)
	, CurrentOrientation(FQuat::Identity)
	, TimeSecondsSinceLastKnownUpdate(0.0)
	, LastKnownUpdateTimestampSeconds(InCurrentTimestamp)
	, UpdateFrequencyTimeSeconds(InUpdateFrequencyTimeSeconds)
	, bHasOrientation(false)
{
}

/** Constructor */
FConcertClientMovement::FConcertClientMovement(double InUpdateFrequencyTimeSeconds, double InCurrentTimestamp, const FVector& InPosition, const FQuat& InOrientation)
	: PreviousPosition(InPosition)
	, PreviousOrientation(InOrientation)
	, LastKnownPosition(InPosition)
	, LastKnownOrientation(InOrientation)
	, CurrentPosition(InPosition)
	, CurrentOrientation(InOrientation)
	, TimeSecondsSinceLastKnownUpdate(0.0)
	, LastKnownUpdateTimestampSeconds(InCurrentTimestamp)
	, UpdateFrequencyTimeSeconds(InUpdateFrequencyTimeSeconds)
	, bHasOrientation(true)
{
}
	
FConcertClientMovement::~FConcertClientMovement()
{
}

void FConcertClientMovement::UpdateLastKnownLocation(double InUpdateTimestampSeconds, const FVector& InPosition, const FQuat* InOrientation)
{
	float DeltaTimeSeconds = StaticCast<float>(InUpdateTimestampSeconds - LastKnownUpdateTimestampSeconds);
	check(DeltaTimeSeconds > 0.0);

	// Reset the previous position to the last position/orientation computed by MoveSmooth()
	PreviousPosition = CurrentPosition;
		
	// Update last known position;
	LastKnownPosition = InPosition;

	if (bHasOrientation && InOrientation)
	{
		PreviousOrientation = CurrentOrientation;
		LastKnownOrientation = *InOrientation;
	}

	TimeSecondsSinceLastKnownUpdate = 0.0f;
	LastKnownUpdateTimestampSeconds = InUpdateTimestampSeconds;
}

void FConcertClientMovement::MoveSmooth(double InDeltaTimeSeconds, FVector& OutPosition, FQuat* OutOrientation)
{
	check(UpdateFrequencyTimeSeconds > 0.0f);
	check(InDeltaTimeSeconds > 0.0f);

	// This implements a time-offset technique. It lerps from the previous actual position to the 
	// last known (update) position using the update frequency in seconds as the basis for the lerp. 
	// This results in a remote lag equivalent to the update frequency.
	// @todo: Should this computation project the movement beyond the last known position if the time since 
	// last known update exceeds the update frequency?

	TimeSecondsSinceLastKnownUpdate += InDeltaTimeSeconds;

	// Calculate normalized T [0, 1] based on update frequency
	float Tnorm = FMath::Clamp(TimeSecondsSinceLastKnownUpdate / UpdateFrequencyTimeSeconds, 0.0, 1.0);

	CurrentPosition = FMath::Lerp(PreviousPosition, LastKnownPosition, Tnorm);
	OutPosition = CurrentPosition;

	if (bHasOrientation && OutOrientation)
	{
		CurrentOrientation = FMath::Lerp(PreviousOrientation, LastKnownOrientation, Tnorm);		
		*OutOrientation = CurrentOrientation;
	}
}

