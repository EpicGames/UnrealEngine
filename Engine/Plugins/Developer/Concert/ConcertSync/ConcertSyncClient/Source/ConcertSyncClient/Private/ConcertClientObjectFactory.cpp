// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertClientObjectFactory.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConcertClientObjectFactory)

class FConcertClientObjectFactoryCache
{
public:
	static FConcertClientObjectFactoryCache& Get()
	{
		static FConcertClientObjectFactoryCache Instance;
		return Instance;
	}

	void RegisterFactory(const UConcertClientObjectFactory* Factory)
	{
		checkf(!Factory->GetClass()->HasAnyClassFlags(CLASS_Abstract), TEXT("Factory '%s' is abstract and cannot be registered!"), *Factory->GetPathName());
		checkf(!Factories.Contains(Factory), TEXT("Factory '%s' was already registered!"), *Factory->GetPathName());
		Factories.Add(Factory);
	}

	void UnregisterFactory(const UConcertClientObjectFactory* Factory)
	{
		Factories.Remove(Factory);
	}

	const UConcertClientObjectFactory* FindFactoryForClass(const UClass* Class)
	{
		for (const UConcertClientObjectFactory* Factory : Factories)
		{
			if (Factory->SupportsClass(Class))
			{
				return Factory;
			}
		}

		return nullptr;
	}

private:
	TArray<const UConcertClientObjectFactory*> Factories;
};

void UConcertClientObjectFactory::PostInitProperties()
{
	Super::PostInitProperties();

	checkf(HasAnyFlags(RF_ClassDefaultObject), TEXT("Factories are assumed to be CDOs! If you need to support multiple instances of the same factory class then this code will need updating accordingly."));

	if (!GetClass()->HasAnyClassFlags(CLASS_Abstract))
	{
		FConcertClientObjectFactoryCache::Get().RegisterFactory(this);
	}
}

void UConcertClientObjectFactory::BeginDestroy()
{
	Super::BeginDestroy();

	if (!GetClass()->HasAnyClassFlags(CLASS_Abstract))
	{
		FConcertClientObjectFactoryCache::Get().UnregisterFactory(this);
	}
}

const UConcertClientObjectFactory* UConcertClientObjectFactory::FindFactoryForClass(const UClass* Class)
{
	return FConcertClientObjectFactoryCache::Get().FindFactoryForClass(Class);
}
