// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertClientPackageBridge.h"

#include "ConcertLogGlobal.h"
#include "ConcertWorkspaceData.h"
#include "ConcertSyncClientUtil.h"

#include "Algo/AllOf.h"
#include "Engine/World.h"
#include "Engine/Engine.h"
#include "IConcertClientPackageBridge.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/Package.h"
#include "UObject/PackageReload.h"
#include "UObject/SavePackage.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "Misc/PackageName.h"
#include "Misc/FeedbackContext.h"
#include "HAL/FileManager.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Modules/ModuleManager.h"

#if WITH_EDITOR
	#include "LevelEditor.h"
	#include "UnrealEdMisc.h"
#endif

LLM_DEFINE_TAG(Concert_ConcertClientPackageBridge);
#define LOCTEXT_NAMESPACE "ConcertClientPackageBridge"

namespace ConcertClientPackageBridgeUtil
{

bool ShouldIgnorePackage(const UPackage* InPackage)
{
	// Ignore transient packages and objects, compiled in package are not considered Multi-user content.
	if (!InPackage || InPackage == GetTransientPackage() || InPackage->HasAnyFlags(RF_Transient) || InPackage->HasAnyPackageFlags(PKG_CompiledIn))
	{
		return true;
	}

	// Ignore packages outside of known root paths (we ignore read-only roots here to skip things like unsaved worlds)
	if (!FPackageName::IsValidLongPackageName(InPackage->GetName()))
	{
		return true;
	}

	return false;
}

} // namespace ConcertClientPackageBridgeUtil

FConcertClientPackageBridge::FConcertClientPackageBridge()
	: bIgnoreLocalSave(false)
	, bIgnoreLocalDiscard(false)
{
	LLM_SCOPE_BYTAG(Concert_ConcertClientPackageBridge);
#if WITH_EDITOR
	// Previously these event handlers were wrapped in the GIsEditor below.  We moved them out to support take recording
	// on nodes running in -game mode. This is a very focused use case and it is not safe to rely on these working
	// correctly outside of take recorder.  There is a lot of code in MultiUser that assumes that in -game mode is
	// "receive" only.
	//
	UPackage::PreSavePackageWithContextEvent.AddRaw(this, &FConcertClientPackageBridge::HandlePackagePreSave);
	UPackage::PackageSavedWithContextEvent.AddRaw(this, &FConcertClientPackageBridge::HandlePackageSaved);
	FCoreUObjectDelegates::OnPackageReloaded.AddRaw(this, &FConcertClientPackageBridge::HandleAssetReload);

	FCoreDelegates::OnEndFrame.AddRaw(this, &FConcertClientPackageBridge::OnEndFrame);

	if (GIsEditor)
	{
		// Register Asset Registry Events
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		AssetRegistryModule.Get().OnInMemoryAssetCreated().AddRaw(this, &FConcertClientPackageBridge::HandleAssetAdded);
		AssetRegistryModule.Get().OnInMemoryAssetDeleted().AddRaw(this, &FConcertClientPackageBridge::HandleAssetDeleted);
		AssetRegistryModule.Get().OnAssetRenamed().AddRaw(this, &FConcertClientPackageBridge::HandleAssetRenamed);

		// Register Map Change Events
		FLevelEditorModule& LevelEditor = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		LevelEditor.OnMapChanged().AddRaw(this, &FConcertClientPackageBridge::HandleMapChanged);
	}
#endif	// WITH_EDITOR
}

FConcertClientPackageBridge::~FConcertClientPackageBridge()
{
#if WITH_EDITOR
	// Unregister Package Events
	UPackage::PreSavePackageWithContextEvent.RemoveAll(this);
	UPackage::PackageSavedWithContextEvent.RemoveAll(this);
	FCoreUObjectDelegates::OnPackageReloaded.RemoveAll(this);
	FCoreDelegates::OnEndFrame.RemoveAll(this);

	if (GIsEditor)
	{
		// Unregister Asset Registry Events
		if (FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>("AssetRegistry"))
		{
			IAssetRegistry* AssetRegistry = AssetRegistryModule->TryGet();
			if (AssetRegistry)
			{
				AssetRegistry->OnInMemoryAssetCreated().RemoveAll(this);
				AssetRegistry->OnInMemoryAssetDeleted().RemoveAll(this);
				AssetRegistry->OnAssetRenamed().RemoveAll(this);
			}
		}

		// Unregister Map Change Events
		if (FLevelEditorModule* LevelEditor = FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor"))
		{
			LevelEditor->OnMapChanged().RemoveAll(this);
		}
	}
#endif	// WITH_EDITOR
}

FOnConcertClientLocalPackageEvent& FConcertClientPackageBridge::OnLocalPackageEvent()
{
	return OnLocalPackageEventDelegate;
}

FOnConcertClientLocalPackageDiscarded& FConcertClientPackageBridge::OnLocalPackageDiscarded()
{
	return OnLocalPackageDiscardedDelegate;
}

bool& FConcertClientPackageBridge::GetIgnoreLocalSaveRef()
{
	return bIgnoreLocalSave;
}

bool& FConcertClientPackageBridge::GetIgnoreLocalDiscardRef()
{
	return bIgnoreLocalDiscard;
}

#if WITH_EDITOR
void FConcertClientPackageBridge::HandlePackagePreSave(UPackage* Package, FObjectPreSaveContext ObjectSaveContext)
{
	// Only execute if this is a user save
	if (ObjectSaveContext.IsProceduralSave())
	{
		return;
	}

	// Ignore unwanted saves
	if (bIgnoreLocalSave || ConcertClientPackageBridgeUtil::ShouldIgnorePackage(Package))
	{
		return;
	}

	// Early out if the delegate is unbound
	if (!OnLocalPackageEventDelegate.IsBound())
	{
		return;
	}

	UObject* Asset = Package->FindAssetInPackage();

	FString PackageFilename;
	if (FPackageName::TryConvertLongPackageNameToFilename(Package->GetFName().ToString(), PackageFilename, Asset && Asset->IsA<UWorld>() ? FPackageName::GetMapPackageExtension() : FPackageName::GetAssetPackageExtension()))
	{
		if (IFileManager::Get().FileExists(*PackageFilename))
		{
			FConcertPackageInfo PackageInfo = ConcertSyncClientUtil::FillPackageInfo(Package, Asset, EConcertPackageUpdateType::Saved);
			PackageInfo.bPreSave = true;
			PackageInfo.bAutoSave = GEngine->IsAutosaving();

			OnLocalPackageEventDelegate.Broadcast(PackageInfo, PackageFilename);
		}
	}

	UE_LOGF(LogConcert, Verbose, "Asset Pre-Saved: %ls", *Package->GetName());
}

void FConcertClientPackageBridge::OnEndFrame()
{
	if (PendingPackageInfos.Num() == 0)
	{
		return;
	}

	UPackage::WaitForAsyncFileWrites();
	for (const FConcertPackageInfoTuple& PackageInfoTuple : PendingPackageInfos)
	{
		const FString& PackageFilename = PackageInfoTuple.Get<1>();
		if (IFileManager::Get().FileExists(*PackageFilename))
		{
			OnLocalPackageEventDelegate.Broadcast(PackageInfoTuple.Get<0>(), PackageFilename);
		}
	}
	PendingPackageInfos.Reset();
}

void FConcertClientPackageBridge::HandlePackageSaved(const FString& PackageFilename, UPackage* Package, FObjectPostSaveContext ObjectSaveContext)
{
	// Only execute if this is a user save
	if (ObjectSaveContext.IsProceduralSave())
	{
		return;
	}

	// Ignore unwanted saves
	if (bIgnoreLocalSave || ConcertClientPackageBridgeUtil::ShouldIgnorePackage(Package))
	{
		return;
	}

	// Early out if the delegate is unbound
	if (!OnLocalPackageEventDelegate.IsBound())
	{
		return;
	}

	// if we end up here, the package should be either unlocked or locked by this client, the server will resend the latest revision if it wasn't the case.
	FName NewPackageName;
	PackagesBeingRenamed.RemoveAndCopyValue(Package->GetFName(), NewPackageName);

	FConcertPackageInfo PackageInfo = ConcertSyncClientUtil::FillPackageInfo(Package, nullptr, NewPackageName.IsNone() ? EConcertPackageUpdateType::Saved : EConcertPackageUpdateType::Renamed);
	PackageInfo.NewPackageName = NewPackageName;
	PackageInfo.bPreSave = false;
	PackageInfo.bAutoSave = GEngine->IsAutosaving();
	PackageInfo.bCanSkipHotReload = CanSkipHotReload(PackageInfo);
	PendingPackageInfos.Add(FConcertPackageInfoTuple(MoveTemp(PackageInfo), PackageFilename));

	UE_LOGF(LogConcert, Verbose, "Asset Saved: %ls", *Package->GetName());
}

void FConcertClientPackageBridge::HandleAssetAdded(UObject *Object)
{
	// Early out if the delegate is unbound
	if (!OnLocalPackageEventDelegate.IsBound())
	{
		return;
	}

	UPackage* Package = Object->GetOutermost();

	// Skip packages that are in the process of being renamed as they are always saved after being added
	if (PackagesBeingRenamed.Contains(Package->GetFName()))
	{
		return;
	}

	// Save this package to disk so that we can send its contents immediately
	{
		SCOPED_CONCERT_TRACE(FConcertClientPackageBridge_SavePackage);
		FScopedIgnoreLocalSave IgnorePackageSaveScope(*this);
		UObject* Asset = Package->FindAssetInPackage();
		// @todo FH: Pass the Asset instead of the World to save package when the incidental IsFullyLoaded is fixed
		UWorld* World = Cast<UWorld>(Asset);

		const FString PackageFilename = FPaths::ProjectIntermediateDir() / TEXT("Concert") / TEXT("Temp") / FGuid::NewGuid().ToString() + (Asset && Asset->IsA<UWorld>() ? FPackageName::GetMapPackageExtension() : FPackageName::GetAssetPackageExtension());
		uint32 PackageFlags = Package->GetPackageFlags();
		FSavePackageArgs SaveArgs;
		SaveArgs.TopLevelFlags = RF_Standalone;
		SaveArgs.Error = GWarn;
		SaveArgs.bWarnOfLongFilename = false;
		// Identify this save as an autosave since we currently don't have a better alternative, but we want to distinguish it from a save triggered by the user and want to allow save callbacks to handle it as such
		SaveArgs.SaveFlags = SAVE_Async | SAVE_NoError | SAVE_KeepDirty | SAVE_FromAutosave;
		if (UPackage::SavePackage(Package, World, *PackageFilename, SaveArgs))
		{
			UPackage::WaitForAsyncFileWrites();
			// Saving the newly added asset here shouldn't modify any of its package flags since it's a 'dummy' save i.e. PKG_NewlyCreated
			Package->SetPackageFlagsTo(PackageFlags);

			if (IFileManager::Get().FileExists(*PackageFilename))
			{
				FConcertPackageInfo PackageInfo = ConcertSyncClientUtil::FillPackageInfo(Package, Asset, EConcertPackageUpdateType::Added);
				PackageInfo.bCanSkipHotReload = CanSkipHotReload(PackageInfo);
				OnLocalPackageEventDelegate.Broadcast(PackageInfo, PackageFilename);
				IFileManager::Get().Delete(*PackageFilename);
			}
		}
		else if (Object->IsPackageExternal() && Package->HasAnyPackageFlags(PKG_NewlyCreated))
		{
			// Save can fail if package was not fully loaded.  If we are in a one-file per actor case then we should
			// clear flags to allow users to save it. We leave the package in a dirty state because we want the user to
			// issue the save (if desired) and it is valid for save to fail on asset create when applied from a
			// transaction.
			//
			Package->ClearPackageFlags(PKG_NewlyCreated);
			Package->MarkAsFullyLoaded();
		}
	}

	UE_LOGF(LogConcert, Verbose, "Asset Added: %ls", *Package->GetName());
}

void FConcertClientPackageBridge::HandleAssetDeleted(UObject *Object)
{
	// Early out if the delegate is unbound
	if (!OnLocalPackageEventDelegate.IsBound())
	{
		return;
	}

	UPackage* Package = Object->GetOutermost();

	const FConcertPackageInfo PackageInfo = ConcertSyncClientUtil::FillPackageInfo(Package, nullptr, EConcertPackageUpdateType::Deleted);
	OnLocalPackageEventDelegate.Broadcast(PackageInfo, FString());

	UE_LOGF(LogConcert, Verbose, "Asset Deleted: %ls", *Package->GetName());
}

void FConcertClientPackageBridge::HandleAssetRenamed(const FAssetData& Data, const FString& OldName)
{
	// A rename operation comes through as:
	//	1) Asset renamed (this notification)
	//	2) Asset added (old asset, which we'll ignore)
	//	3) Asset saved (new asset)
	//	4) Asset saved (old asset, as a redirector)
	const FName OldPackageName = *FPackageName::ObjectPathToPackageName(OldName);
	PackagesBeingRenamed.Add(OldPackageName, Data.PackageName);

	UE_LOGF(LogConcert, Verbose, "Asset Renamed: %ls -> %ls", *OldPackageName.ToString(), *Data.PackageName.ToString());
}

void FConcertClientPackageBridge::HandleAssetReload(const EPackageReloadPhase InPackageReloadPhase, FPackageReloadedEvent* InPackageReloadedEvent)
{
	// Early out if the delegate is unbound
	if (!OnLocalPackageDiscardedDelegate.IsBound())
	{
		return;
	}

	if (InPackageReloadPhase == EPackageReloadPhase::PrePackageLoad)
	{
		UPackage* Package = const_cast<UPackage*>(InPackageReloadedEvent->GetOldPackage());
		if (!ConcertClientPackageBridgeUtil::ShouldIgnorePackage(Package))
		{
			OnLocalPackageDiscardedDelegate.Broadcast(Package);

			UE_LOGF(LogConcert, Verbose, "Asset Discarded: %ls", *Package->GetName());
		}
	}
}

void FConcertClientPackageBridge::HandleMapChanged(UWorld* InWorld, EMapChangeType InMapChangeType)
{
	// Early out if the delegate is unbound
	if (!OnLocalPackageDiscardedDelegate.IsBound())
	{
		return;
	}

	if (InMapChangeType == EMapChangeType::TearDownWorld)
	{
		UPackage* Package = InWorld->GetOutermost();
		if (!ConcertClientPackageBridgeUtil::ShouldIgnorePackage(Package))
		{
			OnLocalPackageDiscardedDelegate.Broadcast(Package);

			UE_LOGF(LogConcert, Verbose, "Asset Discarded: %ls", *Package->GetName());
		}
	}
}
#endif

void FConcertClientPackageBridge::RegisterPackageHotReloadHint(FName PackageReloadHintName, FPackageHotReloadHintDelegate FilterHandle)
{
	check(HotReloadHints.Find(PackageReloadHintName) == nullptr);
	check(FilterHandle.IsBound());
	HotReloadHints.Add(PackageReloadHintName) = MoveTemp(FilterHandle);
}

bool FConcertClientPackageBridge::CanSkipHotReload(const FConcertPackageInfo& PackageInfo) const
{
	return Algo::AllOf(HotReloadHints, [&PackageInfo](const TPair<FName,FPackageHotReloadHintDelegate>& Item)
	{
		return Item.Value.Execute(PackageInfo);
	});
}

void FConcertClientPackageBridge::UnregisterPackageHotReloadHint(FName PackageReloadHintName)
{
	HotReloadHints.FindAndRemoveChecked(PackageReloadHintName);
}

void FConcertClientPackageBridge::RegisterPackageFilter(FName FilterName, FPackageFilterDelegate FilterHandle)
{
	check(PackageFilters.Find(FilterName) == nullptr);
	PackageFilters.Add(FilterName) = MoveTemp(FilterHandle);
}

EPackageFilterResult FConcertClientPackageBridge::IsPackageFiltered(const FConcertPackageInfo& PackageInfo) const
{
	for (const auto& Item : PackageFilters)
	{
		if(Item.Value.IsBound())
		{
			EPackageFilterResult Result = Item.Value.Execute(PackageInfo);
			if (Result != EPackageFilterResult::UseDefault)
			{
				return Result;
			}
		}
	}
	return EPackageFilterResult::UseDefault;
}

void FConcertClientPackageBridge::UnregisterPackageFilter(FName FilterName)
{
	PackageFilters.FindAndRemoveChecked(FilterName);
}

#undef LOCTEXT_NAMESPACE
