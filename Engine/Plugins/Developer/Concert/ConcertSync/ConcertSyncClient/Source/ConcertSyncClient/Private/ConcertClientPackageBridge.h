// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertWorkspaceData.h"
#include "IConcertClientPackageBridge.h"

class FObjectPostSaveContext;
class FObjectPreSaveContext;
class UPackage;
class FPackageReloadedEvent;

struct FAssetData;

enum class EMapChangeType : uint8;
enum class EPackageReloadPhase : uint8;

class FConcertClientPackageBridge : public IConcertClientPackageBridge
{
public:
	FConcertClientPackageBridge();
	virtual ~FConcertClientPackageBridge();

	//~ IConcertClientPackageBridge interface
	virtual FOnConcertClientLocalPackageEvent& OnLocalPackageEvent() override;
	virtual FOnConcertClientLocalPackageDiscarded& OnLocalPackageDiscarded() override;
	virtual bool& GetIgnoreLocalSaveRef() override;
	virtual bool& GetIgnoreLocalDiscardRef() override;

	virtual void RegisterPackageHotReloadHint(FName PackageReloadHintName, FPackageHotReloadHintDelegate FilterHandle) override;
	virtual bool CanSkipHotReload(const FConcertPackageInfo& PackageInfo) const override;
	virtual void UnregisterPackageHotReloadHint(FName PackageReloadHintName) override;

	virtual void RegisterPackageFilter(FName FilterName, FPackageFilterDelegate FilterHandle) override;
	virtual void UnregisterPackageFilter(FName FilterName) override;
	virtual EPackageFilterResult IsPackageFiltered(const FConcertPackageInfo& PackageInfo) const override;

private:
	
#if WITH_EDITOR
	/** Handle any deferred tasks that should be run on the game thread. */
	void OnEndFrame();

	/** Called prior to a package being saved to disk */
	void HandlePackagePreSave(UPackage* Package, FObjectPreSaveContext ObjectSavecontext);

	/** Called after a package has been saved to disk */
	void HandlePackageSaved(const FString& PackageFilename, UPackage* Package, FObjectPostSaveContext ObjectSaveContext);

	/** Called when a new asset is added */
	void HandleAssetAdded(UObject *Object);

	/** Called when an existing asset is deleted */
	void HandleAssetDeleted(UObject *Object);

	/** Called when an existing asset is renamed */
	void HandleAssetRenamed(const FAssetData& Data, const FString& OldName);

	/** Called when an asset is hot-reloaded */
	void HandleAssetReload(const EPackageReloadPhase InPackageReloadPhase, FPackageReloadedEvent* InPackageReloadedEvent);

	/** Called when the editor map is changed */
	void HandleMapChanged(UWorld* InWorld, EMapChangeType InMapChangeType);
#endif

	/** Called when a local package event happens */
	FOnConcertClientLocalPackageEvent OnLocalPackageEventDelegate;

	/** Called when a local package discard happens */
	FOnConcertClientLocalPackageDiscarded OnLocalPackageDiscardedDelegate;

	/** Map of named packages filters that can override what is included / excluded by package bridge*/
	TMap<FName, FPackageFilterDelegate> PackageFilters;

	/** Map of named hot reload delegates that can provide hints on if packages can skip hot reload, e.g. take records. */
	TMap<FName, FPackageHotReloadHintDelegate> HotReloadHints;

	/** Flag to ignore package change events, used when we do not want to record package changes we generate ourselves */
	bool bIgnoreLocalSave;

	/** Flag to ignore package discards, used when we do not want to record package changes we generate ourselves */
	bool bIgnoreLocalDiscard;

	/** Map of packages that are in the process of being renamed */
	TMap<FName, FName> PackagesBeingRenamed;

	using FConcertPackageInfoTuple = TTuple<FConcertPackageInfo, FString>;
	TArray<FConcertPackageInfoTuple> PendingPackageInfos;
};
