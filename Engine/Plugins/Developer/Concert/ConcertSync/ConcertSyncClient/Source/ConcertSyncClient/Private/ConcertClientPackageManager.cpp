// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertClientPackageManager.h"

#include "IConcertClientPackageBridge.h"
#include "IConcertSession.h"
#include "IConcertFileSharingService.h"
#include "ConcertSyncClientLiveSession.h"
#include "ConcertSyncSessionDatabase.h"
#include "ConcertLogGlobal.h"
#include "ConcertWorkspaceData.h"
#include "ConcertWorkspaceMessages.h"
#include "ConcertSyncClientUtil.h"
#include "ConcertSyncSettings.h"
#include "ConcertUtil.h"

#include "Algo/Find.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "Engine/Engine.h"
#include "UObject/Package.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "Misc/PackageName.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"

#include "ISourceControlModule.h"
#include "Sandbox/ConcertPackageSandbox.h"
#include "Sandbox/IConcertSandbox.h"
#include "UObject/UObjectHash.h"

#if WITH_EDITOR
	#include "Editor.h"
	#include "Editor/EditorEngine.h"
	#include "FileHelpers.h"
#endif

#define LOCTEXT_NAMESPACE "ConcertClientPackageManager"

namespace ConcertClientPackageManagerUtil
{

bool RunPackageFilters(const TArray<FPackageClassFilter>& InFilters, const FConcertPackageInfo& InPackageInfo)
{
	bool bMatchFilter = false;
	FString PackageName = InPackageInfo.PackageName.ToString();
	UClass* AssetClass = LoadClass<UObject>(nullptr, *InPackageInfo.AssetClass);
	for (const FPackageClassFilter& PackageFilter : InFilters)
	{
		UClass* PackageAssetClass = PackageFilter.AssetClass.TryLoadClass<UObject>();
		if (!PackageAssetClass || (AssetClass && AssetClass->IsChildOf(PackageAssetClass)))
		{
			for (const FString& ContentPath : PackageFilter.ContentPaths)
			{
				if (PackageName.MatchesWildcard(ContentPath))
				{
					bMatchFilter = true;
					break;
				}
			}
		}
	}
	return bMatchFilter;
}
} // end namespace ConcertClientPackageManagerUtil

FConcertClientPackageManager::FConcertClientPackageManager(
	const TSharedRef<FConcertSyncClientLiveSession>& InLiveSession,
	IConcertClientPackageBridge* InPackageBridge, 
	TSharedPtr<IConcertFileSharingService> InFileSharingService,
	const FString& InClientRole
	)
	: LiveSession(InLiveSession)
	, PackageBridge(InPackageBridge)
	, bIgnorePackageDirtyEvent(false)
	, FileSharingService(MoveTemp(InFileSharingService))

{
	using namespace UE::ConcertSyncClient;
	
	check(LiveSession->IsValidSession());
	check(EnumHasAnyFlags(LiveSession->GetSessionFlags(), EConcertSyncSessionFlags::EnablePackages));
	check(PackageBridge);

#if WITH_EDITOR
	if (EnumHasAnyFlags(LiveSession->GetSessionFlags(), EConcertSyncSessionFlags::ShouldUsePackageSandbox))
	{
		// Create Sandbox
		SandboxManager = FConcertPackageSandbox::CreateOrLoad(*this, LiveSession.ToSharedRef(), InClientRole);
	}

	// Previously these event handlers were wrapped in the GIsEditor below.  We moved them out to support take recording
	// on nodes running in -game mode. This is a very focused use case and it is not safe to rely on these working
	// correctly outside of take recorder.  There is a lot of code in MultiUser that assumes that in -game mode is
	// "receive" only.
	//
	UPackage::PackageDirtyStateChangedEvent.AddRaw(this, &FConcertClientPackageManager::HandlePackageDirtyStateChanged);
	PackageBridge->OnLocalPackageEvent().AddRaw(this, &FConcertClientPackageManager::HandleLocalPackageEvent);

	FEditorDelegates::OnDuplicateActorsBegin.AddRaw(this, &FConcertClientPackageManager::HandleDuplicateActorsBegin);
	FEditorDelegates::OnDuplicateActorsEnd.AddRaw(this, &FConcertClientPackageManager::HandleDuplicateActorsEnd);
#endif	// WITH_EDITOR

	LiveSession->GetSession().RegisterCustomEventHandler<FConcertPackageRejectedEvent>(this, &FConcertClientPackageManager::HandlePackageRejectedEvent);
}

FConcertClientPackageManager::~FConcertClientPackageManager()
{
#if WITH_EDITOR
	// Unregister Package Events
	UPackage::PackageDirtyStateChangedEvent.RemoveAll(this);
	PackageBridge->OnLocalPackageEvent().RemoveAll(this);

	if (SandboxManager)
	{
		// Shutdown will cause FConcertPackageReloadHandler to call EnqueueHotReload and EnqueuePurge.
		SandboxManager.Reset();
	}
	FEditorDelegates::OnDuplicateActorsBegin.RemoveAll(this);
	FEditorDelegates::OnDuplicateActorsEnd.RemoveAll(this);
#endif	// WITH_EDITOR

	LiveSession->GetSession().UnregisterCustomEventHandler<FConcertPackageRejectedEvent>(this);

	// Add dirty packages that aren't for purging to the list of hot reload, overlaps with the sandbox are filtered directly in ReloadPackages
	if (EnumHasAnyFlags(LiveSession->GetSessionFlags(), EConcertSyncSessionFlags::ShouldUsePackageSandbox))
	{
		for (const FName& DirtyPackageName : DirtyPackages)
		{
			if (!PackagesPendingPurge.Contains(DirtyPackageName))
			{
				PackagesPendingHotReload.Add(DirtyPackageName);
			}
		}
	}

	// If the persistent level uses external objects, its package should be
	// reloaded as long as the package hasn't been marked for purge. If it has
	// been marked for purge, then it must have only existed in the sandbox
	// that was just discarded, so it should not be reloaded.
	ULevel* PersistentLevel = ConcertSyncClientUtil::GetExternalPersistentWorld();
	if (PersistentLevel)
	{
		const FName PackageName = PersistentLevel->GetPackage()->GetFName();
		if (!PackagesPendingPurge.Contains(PackageName) && !PackagesPendingHotReload.Contains(PackageName))
		{
			PackagesPendingHotReload.Add(PackageName);
		}
	}

	if (!IsEngineExitRequested())
	{
		// Hot reload after unregistering from most delegates to prevent events triggered by hot-reloading (such as asset deleted) to be recorded as transaction.
		SynchronizeInMemoryPackages();
	}
}

bool PackageContainsExternalActors(UPackage* InPackage)
{
	bool bContainsExternalActors = false;
	ForEachObjectWithPackage(
		InPackage, [&bContainsExternalActors](UObject* InObject) mutable -> bool {
			if (InObject->IsPackageExternal()) {
				bContainsExternalActors = true;
				// Break from our loop.
				return false;
			}
			return true;
		});
	return bContainsExternalActors;
}

bool FConcertClientPackageManager::ShouldIgnorePackageDirtyEvent(class UPackage* InPackage) const
{
	return InPackage == GetTransientPackage()
		|| PackageContainsExternalActors(InPackage)
		|| InPackage->HasAnyFlags(RF_Transient)
		|| InPackage->HasAnyPackageFlags(PKG_PlayInEditor | PKG_CompiledIn) // CompiledIn packages are not considered content for MU. (ex when changing some plugin settings like /Script/DisasterRecoveryClient)
		|| bIgnorePackageDirtyEvent
		|| (!PassesPackageFilters(InPackage));
}

void FConcertClientPackageManager::QueueDirtyPackagesForReload()
{
	TArray<UPackage*> DirtyPkgs;
#if WITH_EDITOR
	{
		UEditorLoadingAndSavingUtils::GetDirtyMapPackages(DirtyPkgs);
		// strip the current world from the dirty list if it doesn't have a file on disk counterpart
		UWorld* CurrentWorld = ConcertSyncClientUtil::GetCurrentWorld();
		UPackage* WorldPackage = CurrentWorld ? CurrentWorld->GetOutermost() : nullptr;
		if (WorldPackage && WorldPackage->IsDirty() &&
			(WorldPackage->HasAnyPackageFlags(PKG_PlayInEditor | PKG_InMemoryOnly) ||
			WorldPackage->HasAnyFlags(RF_Transient) ||
			WorldPackage->GetLoadedPath().GetPackageFName() != WorldPackage->GetFName()))
		{
			DirtyPkgs.Remove(WorldPackage);
		}
		UEditorLoadingAndSavingUtils::GetDirtyContentPackages(DirtyPkgs);
	}
#endif
	for (UPackage* DirtyPkg : DirtyPkgs)
	{
		FName PackageName = DirtyPkg->GetFName();
		PackagesPendingHotReload.Add(PackageName);
		PackagesPendingPurge.Remove(PackageName);
	}
}

void FConcertClientPackageManager::SynchronizeInMemoryPackages()
{
	// Purge pending packages first, since hot reloading can prompt on them before we clear their dirty flags
	TGuardValue<bool> IgnoreDirtyEventScope(bIgnorePackageDirtyEvent, true);
	IConcertClientPackageBridge::FScopedIgnoreLocalDiscard IgnorePackageDiscardScope(*PackageBridge);
	PurgePendingPackages();
	HotReloadPendingPackages();
}

#if WITH_EDITOR
void FConcertClientPackageManager::HandleDuplicateActorsBegin()
{
	UWorld* CurrentWorld = ConcertSyncClientUtil::GetCurrentWorld();
	check(CurrentWorld);

	ULevel* Level = CurrentWorld->GetCurrentLevel();
	bPromptWhenAddingToLevelBeforeCheckout = Level->bPromptWhenAddingToLevelBeforeCheckout;
	Level->bPromptWhenAddingToLevelBeforeCheckout = false;
}

void FConcertClientPackageManager::HandleDuplicateActorsEnd()
{
	UWorld* CurrentWorld = ConcertSyncClientUtil::GetCurrentWorld();
	check(CurrentWorld);

	ULevel* Level = CurrentWorld->GetCurrentLevel();
	Level->bPromptWhenAddingToLevelBeforeCheckout = bPromptWhenAddingToLevelBeforeCheckout;
}
#endif

void FConcertClientPackageManager::HandlePackageDiscarded(UPackage* InPackage)
{
	FConcertPackageUpdateEvent Event;
	Event.Package.Info.PackageName = InPackage->GetFName();
	Event.Package.Info.PackageFileExtension = UWorld::FindWorldInPackage(InPackage) ? FPackageName::GetMapPackageExtension() : FPackageName::GetAssetPackageExtension();
	Event.Package.Info.PackageUpdateType = EConcertPackageUpdateType::Dummy;
	LiveSession->GetSessionDatabase().GetTransactionMaxEventId(Event.Package.Info.TransactionEventIdAtSave);
	LiveSession->GetSession().SendCustomEvent(Event, LiveSession->GetSession().GetSessionServerEndpointId(), EConcertMessageFlags::ReliableOrdered);
}

void FConcertClientPackageManager::HandleRemotePackage(const FGuid& InSourceEndpointId, const int64 InPackageEventId, const bool bApply)
{
	// Ignore this package if we generated it
	if (InSourceEndpointId == LiveSession->GetSession().GetSessionClientEndpointId())
	{
		return;
	}

	if (!bApply)
	{
		return;
	}

	LiveSession->GetSessionDatabase().GetPackageEvent(InPackageEventId, [this](FConcertSyncPackageEventData& PackageEvent)
	{
		ApplyPackageUpdate(PackageEvent.MetaData.PackageInfo, PackageEvent.PackageDataStream);
	});
}

void FConcertClientPackageManager::ApplyAllHeadPackageData()
{
	LiveSession->GetSessionDatabase().EnumerateHeadRevisionPackageData([this](const FConcertPackageInfo& InPackageInfo, FConcertPackageDataStream& InPackageDataStream)
	{
		ApplyPackageUpdate(InPackageInfo, InPackageDataStream);
		return true;
	});
}

bool FConcertClientPackageManager::PassesPackageFilters(UPackage* InPackage) const
{
	FConcertPackageInfo DummyPackageInfo = ConcertSyncClientUtil::FillPackageInfo(InPackage, nullptr, EConcertPackageUpdateType::Saved);
	return ApplyPackageFilters(DummyPackageInfo);
}

bool FConcertClientPackageManager::HasSessionChanges() const
{
	bool bHasSessionChanges = false;
	LiveSession->GetSessionDatabase().EnumeratePackageNamesWithHeadRevision([&bHasSessionChanges](const FName PackageName)
	{
		bHasSessionChanges = true;
		return false; // Stop enumeration
	}, /*IgnorePersisted*/true);
	return bHasSessionChanges;
}

TOptional<FString> FConcertClientPackageManager::GetValidPackageSessionPath(FName PackageName) const
{
#if WITH_EDITOR
	FString Filename;
	if (FPackageName::DoesPackageExist(PackageName.ToString(), &Filename))
	{
		return Filename;
	}
	return GetDeletedPackagePath(PackageName);
#else
	return {};
#endif
}

TOptional<FString> FConcertClientPackageManager::GetDeletedPackagePath(FName PackageName) const
{
#if WITH_EDITOR
	check(SandboxManager.IsValid());
	UE::ConcertSyncClient::IConcertSandbox* Sandbox = SandboxManager.Get();
	auto ShouldPersistPackageWithExtension = [Sandbox](const FString& PackageName, const FString& Extension) -> TOptional<FString>
	{
		FString FullPath;
		if (FPackageName::TryConvertLongPackageNameToFilename(PackageName, FullPath, Extension))
		{
			if (Sandbox->DeletedPackageExistsInNonSandbox(FullPath))
			{
				return MoveTemp(FullPath);
			}
		}
		return {};
	};
	FString PackageNameAsString = PackageName.ToString();
	if (TOptional<FString> AsMap = ShouldPersistPackageWithExtension(PackageNameAsString, FPackageName::GetMapPackageExtension()))
	{
		return AsMap;
	}
	if (TOptional<FString> AsAsset = ShouldPersistPackageWithExtension(PackageNameAsString, FPackageName::GetAssetPackageExtension()))
	{
		return AsAsset;
	}
#endif
	return {};
}

FPersistResult FConcertClientPackageManager::PersistSessionChanges(FPersistParameters InParam)
{
#if WITH_EDITOR
	if (SandboxManager)
	{
		// Transform all the package names into actual filenames
		TArray<FString, TInlineAllocator<8>> FilesToPersist;
		for (const FName& PackageName : InParam.PackagesToPersist)
		{
			if (TOptional<FString> ValidPath = GetValidPackageSessionPath(PackageName))
			{
				FilesToPersist.Add(MoveTemp(ValidPath.GetValue()));
			}
		}
		return SandboxManager->PersistSandbox(FilesToPersist, MoveTemp(InParam));
	}
#endif
	return {};
}

bool FConcertClientPackageManager::ApplyPackageFilters(const FConcertPackageInfo& InPackageInfo) const
{
	// Only run the package filters if we are using a sandbox 
	// (We do not want to run package filtering on Disaster Recovery session and we identify it by not having the ShouldUsePackageSandbox flag for now)
	if (EnumHasAnyFlags(LiveSession->GetSessionFlags(), EConcertSyncSessionFlags::ShouldUsePackageSandbox))
	{
		EPackageFilterResult Result = PackageBridge->IsPackageFiltered(InPackageInfo);
		if (Result != EPackageFilterResult::UseDefault)
		{
			return Result == EPackageFilterResult::Include ? true : false;
		}

		const UConcertSyncConfig* SyncConfig = GetDefault<UConcertSyncConfig>();
		// Ignore packages that passes the ExcludePackageClassFilters
		if (SyncConfig->ExcludePackageClassFilters.Num() > 0 && ConcertClientPackageManagerUtil::RunPackageFilters(SyncConfig->ExcludePackageClassFilters, InPackageInfo))
		{
			return false;
		}

	}
	return true;
}

void FConcertClientPackageManager::ApplyPackageUpdate(const FConcertPackageInfo& InPackageInfo, FConcertPackageDataStream& InPackageDataStream)
{
	switch (InPackageInfo.PackageUpdateType)
	{
	case EConcertPackageUpdateType::Dummy:
	case EConcertPackageUpdateType::Added:
	case EConcertPackageUpdateType::Saved:
		SavePackageFile(InPackageInfo, InPackageDataStream);
		break;

	case EConcertPackageUpdateType::Renamed:
		DeletePackageFile(InPackageInfo);
		SavePackageFile(InPackageInfo, InPackageDataStream);
		break;

	case EConcertPackageUpdateType::Deleted:
		DeletePackageFile(InPackageInfo);
		break;

	default:
		break;
	}
}

void FConcertClientPackageManager::HandlePackageRejectedEvent(const FConcertSessionContext& InEventContext, const FConcertPackageRejectedEvent& InEvent)
{
	// Package update was rejected, restore the head-revision of the package
	LiveSession->GetSessionDatabase().GetPackageDataForRevision(InEvent.PackageName, [this](const FConcertPackageInfo& InPackageInfo, FConcertPackageDataStream& InPackageDataStream)
	{
		ApplyPackageUpdate(InPackageInfo, InPackageDataStream);
	});
}

void FConcertClientPackageManager::HandlePackageDirtyStateChanged(UPackage* InPackage)
{
	check(!InPackage->HasAnyFlags(RF_Transient) || InPackage != GetTransientPackage());

	// Dirty packages are tracked for purge/reload, but 'compiled in',
	// 'in memory', or temporary packages cannot be hot purged/reloaded.
	//
	if (InPackage->IsDirty() &&
			!InPackage->HasAnyPackageFlags(PKG_CompiledIn | PKG_InMemoryOnly) &&
			!FPackageName::IsTempPackage(InPackage->GetName()))
	{
		DirtyPackages.Add(InPackage->GetFName());
	}
	else
	{
		DirtyPackages.Remove(InPackage->GetFName());
	}
}

void FConcertClientPackageManager::AddPendingReloadForNewExternalMaps(const FConcertPackageInfo& PackageInfo, const FString& PackagePathname)
{
	// Saved world partitioned main levels depend on loading other external assets. We need to schedule a reload of the map after initial
	// save to ensure we have all of the content to fully load the p-level.
	//
	ULevel* PersistentLevel = ConcertSyncClientUtil::GetExternalPersistentWorld();
	if (PersistentLevel)
	{
		const FName PackageName = PersistentLevel->GetPackage()->GetFName();
		if (PackageName == PackageInfo.PackageName
			&& !PackagesPendingPurge.Contains(PackageName)
			&& !PackagesPendingHotReload.Contains(PackageName))
		{
			UE_LOGF(LogConcert, Display, "Scheduling reloading for world partition persistent level %ls.", *PackagePathname);

			PackagesPendingHotReload.Add(PackageName);
		}
	}
}

void FConcertClientPackageManager::HandleLocalPackageEvent(const FConcertPackageInfo& PackageInfo, const FString& PackagePathname)
{
	// Ignore unwanted saves
	const bool bIsValidPackageName = FPackageName::IsValidLongPackageName(PackageInfo.PackageName.ToString());
	if (!bIsValidPackageName)
	{
		return;
	}
	if (PackageInfo.PackageUpdateType == EConcertPackageUpdateType::Saved)
	{
		if (PackageInfo.bAutoSave && !EnumHasAnyFlags(LiveSession->GetSessionFlags(), EConcertSyncSessionFlags::ShouldSendPackageAutoSaves))
		{
			return;
		}
		else if (PackageInfo.bPreSave)
		{
			// Pre-save events are used to send the pristine package state of a package (if enabled), so make sure we don't already have a history for this package
			FConcertPackageInfo ExistingPackageInfo;
			if (!EnumHasAnyFlags(LiveSession->GetSessionFlags(), EConcertSyncSessionFlags::ShouldSendPackagePristineState) || LiveSession->GetSessionDatabase().GetPackageInfoForRevision(PackageInfo.PackageName, ExistingPackageInfo))
			{
				return;
			}
			// Without live sync feature, the local database is not maintainted after the original 'sync on join'. Package that were not in the original sync don't have a revision from the client point of view.
			else if (!EnumHasAnyFlags(LiveSession->GetSessionFlags(), EConcertSyncSessionFlags::EnableLiveSync) && EmittedPristinePackages.Contains(PackageInfo.PackageName))
			{
				return; // Prevent capturing the original package state at every pre-save.
			}
		}
		AddPendingReloadForNewExternalMaps(PackageInfo, PackagePathname);
	}

	if (PackageInfo.PackageUpdateType == EConcertPackageUpdateType::Added && EnumHasAnyFlags(LiveSession->GetSessionFlags(), EConcertSyncSessionFlags::ShouldUsePackageSandbox))
	{
		// If this package was locally added and we're using a sandbox, also write it to the correct location on disk (which will be placed into the sandbox directory)
		FString DstPackagePathname;
		if (FPackageName::TryConvertLongPackageNameToFilename(PackageInfo.PackageName.ToString(), DstPackagePathname, PackageInfo.PackageFileExtension))
		{
			if (IFileManager::Get().Copy(*DstPackagePathname, *PackagePathname) != ECopyResult::COPY_OK)
			{
				UE_LOGF(LogConcert, Error, "Failed to copy package file '%ls' to the sandbox", *PackagePathname);
			}
		}
	}

	if (PackageInfo.bPreSave && EnumHasAnyFlags(LiveSession->GetSessionFlags(), EConcertSyncSessionFlags::ShouldSendPackagePristineState) && !EnumHasAnyFlags(LiveSession->GetSessionFlags(), EConcertSyncSessionFlags::EnableLiveSync))
	{
		EmittedPristinePackages.Add(PackageInfo.PackageName); // Prevent sending the original package state at every pre-save.
	}

	// if the package filter passes, send the event
	if (ApplyPackageFilters(PackageInfo))
	{
		FConcertPackageUpdateEvent PackageEvent;
		PackageEvent.TransmissionId = FGuid::NewGuid();
		PackageEvent.Package.Info = PackageInfo;

		// Copy or link the package data to the Concert event.
		int64 PackageFileSize = PackagePathname.IsEmpty() ? -1 : IFileManager::Get().FileSize(*PackagePathname); // EConcertPackageUpdateType::Delete is emitted with an empty pathname.
		if (PackageFileSize > 0)
		{
			if (CanExchangePackageDataAsByteArray(static_cast<uint64>(PackageFileSize)))
			{
				// Embed the package data directly in the event.
				if (!FFileHelper::LoadFileToArray(PackageEvent.Package.PackageData.Bytes, *PackagePathname))
				{
					UE_LOGF(LogConcert, Error, "Failed to load file data '%ls' in memory", *PackagePathname);
					return;
				}
			}
			else if (IsFileSharingEnabled())
			{
				// Publish a copy of the package data in the sharing service and set the corresponding file ID in the event.
				if (!FileSharingService->Publish(PackagePathname, PackageEvent.Package.FileId))
				{
					UE_LOGF(LogConcert, Error, "Failed to share a copy of package file '%ls'", *PackagePathname);
					return;
				}
			}
			else
			{
				// Notify the client about the file being too large to be emitted.
				UE_LOGF(LogConcert, Error, "Failed to handle local package file '%ls'. The file is too big to be sent over the network.", *PackagePathname);
				OnConcertClientPackageTooLargeError().Broadcast(PackageEvent.Package.Info, PackageFileSize, FConcertPackage::GetMaxPackageDataSizeEmbeddableAsByteArray());
			}
		}

		LiveSession->GetSessionDatabase().GetTransactionMaxEventId(PackageEvent.Package.Info.TransactionEventIdAtSave);
		if (PackageEvent.Package.HasPackageData())
		{
			const FConcertPackageTransmissionStartEvent AnnouncementEvent{ PackageEvent.TransmissionId, PackageEvent.Package.Info, static_cast<uint64>(PackageFileSize) };
			LiveSession->GetSession().SendCustomEvent(AnnouncementEvent, LiveSession->GetSession().GetSessionServerEndpointId(), EConcertMessageFlags::ReliableOrdered);
		}
		LiveSession->GetSession().SendCustomEvent(PackageEvent, LiveSession->GetSession().GetSessionServerEndpointId(), EConcertMessageFlags::ReliableOrdered);
	}
	// if the package data has been filtered out of the session persist it immediately from the sandbox
	else
	{
		PersistSessionChanges({MakeArrayView(&PackageInfo.PackageName, 1) });
	}
}

void FConcertClientPackageManager::SavePackageFile(const FConcertPackageInfo& PackageInfo, FConcertPackageDataStream& InPackageDataStream)
{
	// This path should only be taken for non-cooked targets for now
	check(!FPlatformProperties::RequiresCookedData());

	if (!InPackageDataStream.DataAr || InPackageDataStream.DataSize == 0)
	{
		// If we have no package data set, then this was from a meta-data
		// only package sync, so we have no new contents to write to disk
		return;
	}

	SCOPED_CONCERT_TRACE(FConcertClientPackageManager_SavePackageFile);
	FString PackageName = PackageInfo.PackageName.ToString();
	ConcertSyncClientUtil::FlushPackageLoading(PackageName);

	// Convert long package name to filename
	FString PackageFilename;
	bool bSuccess = FPackageName::TryConvertLongPackageNameToFilename(PackageName, PackageFilename, PackageInfo.PackageFileExtension);
	if (bSuccess)
	{
		// Overwrite the file on disk
		TUniquePtr<FArchive> DstAr(IFileManager::Get().CreateFileWriter(*PackageFilename, FILEWRITE_EvenIfReadOnly));
		bSuccess = DstAr && ConcertUtil::Copy(*DstAr, *InPackageDataStream.DataAr, InPackageDataStream.DataSize);
	}

	if (bSuccess)
	{
		if (!PackageInfo.bCanSkipHotReload)
		{
			// We skip hot reloading assets that are hinted not to need it. @see FConcertPackageInfo::bCanSkipHotReload)
			PackagesPendingHotReload.Add(PackageInfo.PackageName);
		}
		PackagesPendingPurge.Remove(PackageInfo.PackageName);
	}
}

void FConcertClientPackageManager::DeletePackageFile(const FConcertPackageInfo& PackageInfo)
{
	// This path should only be taken for non-cooked targets for now
	check(!FPlatformProperties::RequiresCookedData());

	FString PackageName = PackageInfo.PackageName.ToString();
	ConcertSyncClientUtil::FlushPackageLoading(PackageName);

	// Convert long package name to filename
	FString PackageFilenameWildcard;
	bool bSuccess = FPackageName::TryConvertLongPackageNameToFilename(PackageName, PackageFilenameWildcard, TEXT(".*"));
	if (bSuccess)
	{
		// Delete the file on disk
		// We delete any files associated with this package as it may have changed extension type during the session
		TArray<FString> FoundPackageFilenames;
		IFileManager::Get().FindFiles(FoundPackageFilenames, *PackageFilenameWildcard, /*Files*/true, /*Directories*/false);
		const FString PackageDirectory = FPaths::GetPath(PackageFilenameWildcard);
		for (const FString& FoundPackageFilename : FoundPackageFilenames)
		{
			bSuccess |= IFileManager::Get().Delete(*(PackageDirectory / FoundPackageFilename), false, true, true);
		}
	}

	if (bSuccess)
	{
		PackagesPendingPurge.Add(PackageInfo.PackageName);
		PackagesPendingHotReload.Remove(PackageInfo.PackageName);
	}
}

bool FConcertClientPackageManager::IsReloadingPackage(FName PackageName) const
{
	if (bHotReloading)
	{
		return Algo::Find(PackagesPendingHotReload,PackageName) != nullptr;
	}
	return false;
}

bool FConcertClientPackageManager::CanHotReloadOrPurge() const
{
	const bool bIsSuspended = LiveSession->GetSession().GetSendReceiveState() != EConcertSendReceiveState::Default;
	return ConcertSyncClientUtil::CanPerformBlockingAction() && !bIsSuspended;
}

void FConcertClientPackageManager::HotReloadPendingPackages()
{
	SCOPED_CONCERT_TRACE(FConcertClientPackageManager_HotReloadPendingPackages);
	if (CanHotReloadOrPurge() && PackagesPendingHotReload.Num() > 0)
	{
		TGuardValue<bool> HotReloadGuard(bHotReloading, true);
		if (!LiveSession->GetSessionDatabase().HasWritePackageTasksCompleted())
		{
			LiveSession->GetSessionDatabase().FlushAsynchronousTasks();
		}
		ConcertSyncClientUtil::HotReloadPackages(PackagesPendingHotReload);
		PackagesPendingHotReload.Reset();
	}
}

void FConcertClientPackageManager::PurgePendingPackages()
{
	SCOPED_CONCERT_TRACE(FConcertClientPackageManager_PurgePendingPackages);
	if (CanHotReloadOrPurge() && PackagesPendingPurge.Num() > 0)
	{
		ConcertSyncClientUtil::PurgePackages(PackagesPendingPurge);
		PackagesPendingPurge.Reset();
	}
}

bool FConcertClientPackageManager::IsFileSharingEnabled() const
{
	return FileSharingService && FileSharingService->IsEnabled() && EnumHasAnyFlags(LiveSession->GetSessionFlags(), EConcertSyncSessionFlags::EnableFileSharing);
}

bool FConcertClientPackageManager::CanExchangePackageDataAsByteArray(int64 PackageDataSize) const
{
	if (IsFileSharingEnabled())
	{
		return FConcertPackage::ShouldEmbedPackageDataAsByteArray(PackageDataSize); // Test the package data size against a preferred limit.
	}

	return FConcertPackage::CanEmbedPackageDataAsByteArray(PackageDataSize); // The the package data size against the maximum permitted.
}

#undef LOCTEXT_NAMESPACE
