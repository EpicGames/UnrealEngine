// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IConcertSessionHandler.h"
#include "IConcertClientPackageBridge.h"

#include "ConcertClientPersistData.h"
#include "IConcertClientModule.h"
#include "Misc/Attribute.h"
#include "Sandbox/ConcertPackageSandbox.h"
#include "Sandbox/IConcertSandbox.h"


class FConcertSyncClient;
class FConcertSyncClientLiveSession;
class FConcertSandboxPlatformFile;
class ISourceControlProvider;
class IConcertFileSharingService;
class UPackage;
namespace UE::ConcertSyncClient { class FConcertSandboxLock; }

struct FAssetData;
struct FConcertPackage;
struct FConcertPackageInfo;
struct FConcertPackageRejectedEvent;
struct FConcertPackageDataStream;

enum class EConcertPackageUpdateType : uint8;

/** Invoked when a package is too large to be sent to the server if the transport protocol doesn't support large file. */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnConcertClientPackageTooLargeError, const FConcertPackageInfo& /*PackageInfo*/, int64 /*PkgSize*/, int64 /*MaxSize*/);

class FConcertClientPackageManager
{
public:
	/**
	 * Construct the package manager for the specified session.
	 * @param InLiveSession The session for which the package manager will be used.
	 * @param InPackageBridge The package bridge (used by the package manager to register itself for package events)
	 * @param InFileSharingService Optional service (can be null) used by the client to exchange very large packages with the server.
	 * @param InClientRole Role of the client. Used by sandbox system to generate lock reason text.
	 */
	FConcertClientPackageManager(
		const TSharedRef<FConcertSyncClientLiveSession>& InLiveSession, 
		IConcertClientPackageBridge* InPackageBridge, 
		TSharedPtr<IConcertFileSharingService> InFileSharingService,
		const FString& InClientRole
		);
	~FConcertClientPackageManager();
	
	/**
	 * Verify if a package dirty event should be ignored, mainly for the purpose of locking
	 * @return true if dirty even should be ignored for InPackage
	 */
	bool ShouldIgnorePackageDirtyEvent(class UPackage* InPackage) const;
	
	/**
	 * Discard dirty packages changes by queuing them for hot reload
	 */
	void QueueDirtyPackagesForReload();

	/**
	 * Synchronize any pending updates to in-memory packages (hot-reloads or purges) to keep them up-to-date with the on-disk state.
	 */
	void SynchronizeInMemoryPackages();

	/**
	 * Called to handle a local package having its changes discarded.
	 */
	void HandlePackageDiscarded(UPackage* InPackage);

	/**
	 * Called to handle a remote package being received.
	 */
	void HandleRemotePackage(const FGuid& InSourceEndpointId, const int64 InPackageEventId, const bool bApply);

	/**
	 * Called to apply the head revision data for all packages.
	 */
	void ApplyAllHeadPackageData();

	/**
	 * Run Package filters
	 * @param InPackage The package to run the filters on
	 * @return true if the package passes the filters (Is not filtered out)
	 */
	bool PassesPackageFilters(UPackage* InPackage) const;

	/**
	 * Tell if package changes happened during this session.
	 * @return True if the session contains package changes.
	 */
	bool HasSessionChanges() const;

	/** 
	 * Returns the full path to the package if it is valid.
	 */
	TOptional<FString> GetValidPackageSessionPath(FName PackageName) const;

	/**
	 * Persist the session changes from the package name list and prepare it for source control submission.
	 */
	FPersistResult PersistSessionChanges(FPersistParameters InParam);

	/**
	 * Called when a package is too big to be handled by the system.
	 */
	FOnConcertClientPackageTooLargeError& OnConcertClientPackageTooLargeError() { return OnPackageTooLargeErrorDelegate; }

	/**
	 * Returns true if the named package is participating in a package reload.
	 */
	bool IsReloadingPackage(FName PackageName) const;
	
	/** Enqueues a package for hot reload. */
	void EnqueueHotReload(FName InPackageName) { PackagesPendingHotReload.Add(InPackageName); }
	/** Enqueues a package for purge. */
	void EnqueuePurge(FName InPackageName) { PackagesPendingPurge.Add(InPackageName); }

private:
	/**
	 * Is file sharing enabled.
	 */
	bool IsFileSharingEnabled() const;
		
	/**
	 * Returns a full path for given package name if it was deleted and exists on the non-sandbox area.
	 */
	TOptional<FString> GetDeletedPackagePath(FName PackageName) const;

	/**
	 * Apply the package filters on the package info
	 * @return true if the package info passes the filter.
	 */
	bool ApplyPackageFilters(const FConcertPackageInfo& InPackageInfo) const;

	/**
	 * Apply the data in the given package to disk and update the in-memory state.
	 */
	void ApplyPackageUpdate(const FConcertPackageInfo& InPackageInfo, FConcertPackageDataStream& InPackageDataStream);

	/**
	 * Handle a rejected package event, those are sent by the server when a package update is refused.
	 */
	void HandlePackageRejectedEvent(const FConcertSessionContext& InEventContext, const FConcertPackageRejectedEvent& InEvent);

	/**
	 * Called when the dirty state of a package changed.
	 * Used to track currently dirty packages for hot-reload when discarding the manager.
	 */
	void HandlePackageDirtyStateChanged(UPackage* InPackage);

	/**
	 * Called to handle a local package event.
	 */
	void HandleLocalPackageEvent(const FConcertPackageInfo& PackageInfo, const FString& PackagePathname);

#if WITH_EDITOR
	/**
	 * Handle begin event when a duplicate actor is added to the level.
	 */
	void HandleDuplicateActorsBegin();

	/**
	 * Handle end event when a duplicate actor is added to the level.
	 */
	void HandleDuplicateActorsEnd();
#endif

	/**
	 * Utility to save new package data to disk, and also queue if for hot-reload.
	 */
	void SavePackageFile(const FConcertPackageInfo& PackageInfo, FConcertPackageDataStream& InPackageDataStream);

	/**
	 * Utility to remove existing package data from disk, and also queue if for purging.
	 */
	void DeletePackageFile(const FConcertPackageInfo& PackageInfo);

	/**
	 * Can we currently perform content hot-reloads or purges?
	 * True if we are neither suspended nor unable to perform a blocking action, false otherwise.
	 */
	bool CanHotReloadOrPurge() const;

	/**
	 * Hot-reload any pending in-memory packages to keep them up-to-date with the on-disk state.
	 */
	void HotReloadPendingPackages();

	/**
	 * Purge any pending in-memory packages to keep them up-to-date with the on-disk state.
	 */
	void PurgePendingPackages();

	/**
	 * Whether the package data is small enough to be exchanged using a single TArray<> data structure.
	 */
	bool CanExchangePackageDataAsByteArray(int64 PackageDataSize) const;

	/**
	 * World partition nodes need to perform a map reload on initial save but only after all actors have been saved.
	 */
	void AddPendingReloadForNewExternalMaps(const FConcertPackageInfo& PackageInfo, const FString& PackagePathname);

#if WITH_EDITOR
	/** Sandbox for storing package changes to disk within a Concert session. */
	TUniquePtr<UE::ConcertSyncClient::FConcertPackageSandbox> SandboxManager; // TODO UE-354294: Will need to ensure the sandbox also works on cooked clients
#endif

	/**
	 * Session instance this package manager was created for.
	 */
	TSharedPtr<FConcertSyncClientLiveSession> LiveSession;

	/**
	 * Package bridge used by this manager.
	 */
	IConcertClientPackageBridge* PackageBridge;

	/**
	 * Value of the CurrentWorld()->GetCurrentLevel()->bPromptWhenAddingToLevelBeforeCheckout on the level. We suppress this
	 * for sandbox sessions.  This value is only relevant between begin / end delegate on duplicate actors event.
	 */
	bool bPromptWhenAddingToLevelBeforeCheckout = false;

	/**
	 * Indicates if we are currently hot reloading.
	 */
	bool bHotReloading = false;

	/**
	 * Flag to indicate package dirty event should be ignored.
	 */
	bool bIgnorePackageDirtyEvent;

	/**
	 * Set of package names that are currently dirty.
	 * Only used to properly track packages that need hot-reloading when discarding the manager but
	 * currently escape the sandbox and live transaction tracking.
	 */
	TSet<FName> DirtyPackages;

	/**
	 * Array of package names that are pending a content hot-reload.
	 */
	TArray<FName> PackagesPendingHotReload;

	/**
	 * Array of package names that are pending an in-memory purge.
	 */
	TArray<FName> PackagesPendingPurge;

	/**
	 * Called when a package is too large to be handled by the system.
	 */
	FOnConcertClientPackageTooLargeError OnPackageTooLargeErrorDelegate;

	/**
	 * Optional side channel to exchange large blobs (package data) with the server in a scalable way (ex. the request/response transport layer is not designed and doesn't support exchanging 3GB packages).
	 */
	TSharedPtr<IConcertFileSharingService> FileSharingService;

	/**
	 * Keep the list of package for which the pristine state was sent. Used if the session doesn't support live sync. 
	 */
	TSet<FName> EmittedPristinePackages;
};
