// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertClientPresenceActor.h"
#include "UObject/ConstructorHelpers.h"
#include "EngineGlobals.h"
#include "UObject/Package.h"
#include "UObject/StructOnScope.h"
#include "Engine/Scene.h"
#include "ConcertPresenceEvents.h"
#include "ConcertSyncSettings.h"
#include "Components/StaticMeshComponent.h"
#include "Components/TextRenderComponent.h"
#include "ConcertAssetContainer.h"
#include "ConcertClientPresenceMode.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "ConcertClientPresenceManager.h"
#include "ConcertClientMovement.h"
#include "ConcertLogGlobal.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConcertClientPresenceActor)

#define LOCTEXT_NAMESPACE "ConcertClientPresenceActor"

static TAutoConsoleVariable<int32> CVarEnablePresenceInGame(TEXT("Concert.EnablePresenceInGame"), 0, TEXT("Enable Concert Presence in Game"));


//////////////////////////////////////////////////////////////////////////
// AConcertClientPresenceActor

AConcertClientPresenceActor::AConcertClientPresenceActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Initialize if this actor will be editor only based the `EnablePresenceInGame` console variable.
	UpdateVisibleInGameState();

	// Set root component
	{
		USceneComponent* SceneRootComponent = CreateDefaultSubobject<USceneComponent>(TEXT("Root"));
		AddOwnedComponent(SceneRootComponent);
		SetRootComponent(SceneRootComponent);
		SceneRootComponent->SetMobility(EComponentMobility::Movable);

		PresenceMeshComponent = CreateDefaultSubobject<UStaticMeshComponent>(TEXT("Presence"));
		AddOwnedComponent(PresenceMeshComponent);
		PresenceMeshComponent->SetupAttachment(RootComponent);
		PresenceMeshComponent->SetMobility(EComponentMobility::Movable);

		PresenceTextComponent = CreateDefaultSubobject<UTextRenderComponent>(TEXT("Name"));
		AddOwnedComponent(PresenceTextComponent);
		PresenceTextComponent->SetupAttachment(RootComponent);
		PresenceTextComponent->SetMobility(EComponentMobility::Movable);
		PresenceTextComponent->SetHorizontalAlignment(EHTA_Center);
		PresenceTextComponent->AddRelativeLocation(FVector(0.0f, 0.0f, 30.0f));
	}

	PrimaryActorTick.bCanEverTick = true;
	PrimaryActorTick.bStartWithTickEnabled = true;
	SetActorTickEnabled(true);
	SetActorTickInterval(0.0333f);
}

bool AConcertClientPresenceActor::ShouldTickIfViewportsOnly() const
{
	return true;
}

void AConcertClientPresenceActor::UpdateVisibleInGameState()
{
	EShowPresenceInGame PresenceInGame = GetDefault<UConcertSyncConfig>()->ShowPresenceInGame;
	bIsEditorOnlyActor = CVarEnablePresenceInGame.GetValueOnAnyThread() == 0;

	if (bIsEditorOnlyActor && GIsEditor &&
		(PresenceInGame == EShowPresenceInGame::VisibleInGameOnlyEditor || PresenceInGame == EShowPresenceInGame::VisibleInGameAlways))
	{
		bIsEditorOnlyActor = false;
	}
	else if (!GIsEditor && bIsEditorOnlyActor && PresenceInGame == EShowPresenceInGame::VisibleInGameAlways)
	{
		bIsEditorOnlyActor = false;
	}
}

bool AConcertClientPresenceActor::IsEditorOnly() const
{
	return bIsEditorOnlyActor;
}

void AConcertClientPresenceActor::SetPresenceName(const FString& InName)
{
	if (PresenceTextComponent != nullptr)
	{
		PresenceTextComponent->SetText(FText::FromString(InName));
	}
}

void AConcertClientPresenceActor::SetPresenceColor(const FLinearColor& InColor)
{
	static const FName ColorParamName(TEXT("Color"));

	UMaterialInstanceDynamic* PresenceMaterialInstance = PresenceMID;
	if (PresenceMaterialInstance != nullptr)
	{
		PresenceMaterialInstance->SetVectorParameterValue(ColorParamName, InColor);
	}

	UMaterialInstanceDynamic* PresenceTextMaterialInstance = TextMID;
	if (PresenceTextMaterialInstance != nullptr)
	{
		PresenceTextMaterialInstance->SetVectorParameterValue(ColorParamName, InColor);
	}
}

void AConcertClientPresenceActor::HandleEvent(const FStructOnScope& InEvent)
{
	if (InEvent.GetStruct() == FConcertClientPresenceDataUpdateEvent::StaticStruct())
	{
		if (const FConcertClientPresenceDataUpdateEvent* Event = (const FConcertClientPresenceDataUpdateEvent*)InEvent.GetStructMemory())
		{
			double TimestampSeconds = FPlatformTime::Seconds();
			if (!PresenceMovement.IsSet())
			{
				PresenceMovement = FConcertClientMovement(FConcertClientPresenceManager::GetLocationUpdateFrequency(), TimestampSeconds, Event->Position, Event->Orientation);
			}
			else
			{
				PresenceMovement->UpdateLastKnownLocation(TimestampSeconds, Event->Position, &Event->Orientation);
			}
		}
	}
}

void AConcertClientPresenceActor::Tick(float DeltaSeconds)
{
	if (PresenceMovement.IsSet())
	{
		FQuat Orientation;
		FVector Position;
		PresenceMovement->MoveSmooth(DeltaSeconds, Position, &Orientation);

		FTransform PresenceTransform(Orientation, Position);
		SetActorTransform(PresenceTransform);

		if (PresenceTextComponent != nullptr)
		{
			// Must set the world rotation to 0 so that the camera facing 
			// text computed in the material vertex shader will
			// remain camera-facing.
			PresenceTextComponent->SetWorldRotation(FRotator::ZeroRotator);
		}
	}
}

void AConcertClientPresenceActor::InitPresence(const UConcertAssetContainer& InAssetContainer, FName DeviceType)
{
	PresenceDeviceType = DeviceType;
	UStaticMesh* PresenceMesh = InAssetContainer.GenericDesktopMesh;

	if (PresenceMeshComponent->GetStaticMesh() == nullptr)
	{
		PresenceMeshComponent->SetStaticMesh(PresenceMesh);
	}
	PresenceMeshComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
	PresenceMeshComponent->bSelectable = false;
	PresenceMeshComponent->SetCastShadow(false);

	UMaterialInterface* PresenceMaterial = InAssetContainer.PresenceMaterial;

	PresenceMID = UMaterialInstanceDynamic::Create(PresenceMaterial, GetTransientPackage());


	UMaterialInterface* TextMaterial = InAssetContainer.TextMaterial;

	TextMID = UMaterialInstanceDynamic::Create(TextMaterial, GetTransientPackage());
	PresenceTextComponent->SetMaterial(0, TextMID);
}

#undef LOCTEXT_NAMESPACE


