// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertClientPresenceManager.h"
#include "ConcertPresenceEvents.h"
#include "CoreMinimal.h"
#include "IConcertSession.h"
#include "IConcertSessionHandler.h"
#include "ConcertClientPresenceActor.h"
#include "ConcertWorkspaceMessages.h"
#include "ConcertSyncSettings.h"
#include "IConcertModule.h"
#include "IConcertClient.h"
#include "IConcertSyncClient.h"
#include "Engine/Blueprint.h"
#include "Engine/Engine.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Modules/ModuleManager.h"
#include "ConcertLogGlobal.h"
#include "ConcertClientDesktopPresenceActor.h"
#include "ConcertClientVRPresenceActor.h"
#include "Scratchpad/ConcertScratchpad.h"
#include "GameFramework/PlayerController.h"
#include "Engine/GameEngine.h"
#include "HAL/IConsoleManager.h"

#if WITH_EDITOR
#include "Editor.h"
#include "EditorFontGlyphs.h"
#include "EditorWorldExtension.h"
#include "UnrealEdMisc.h"
#include "UnrealEdGlobals.h"
#include "Editor/UnrealEdEngine.h"
#include "LevelEditor.h"
#include "IAssetViewport.h"
#include "SLevelViewport.h"
#include "ConcertAssetContainer.h"
#include "IVREditorModule.h"
#include "VREditorMode.h"
#include "Framework/Application/SlateApplication.h"
#endif

#define LOCTEXT_NAMESPACE "ConcertPresenceManager"

namespace ConcertClientPresenceManagerUtil
{
	// Update frequency 15 Hz
	const double LocationUpdateFrequencySeconds = 0.0667;
}

#if WITH_EDITOR

namespace ConcertClientPresenceManagerUtil
{

bool ShowPresenceInPIE(const bool InIsPIE)
{
	return !InIsPIE || GetDefault<UConcertSyncConfig>()->bShowPresenceInPIE;
}

}

static TAutoConsoleVariable<int32> CVarDisplayPresence(TEXT("Concert.DisplayPresence"), 1, TEXT("Enable display of Concert Presence from remote users."));
static TAutoConsoleVariable<int32> CVarEmitPresence(TEXT("Concert.EmitPresence"), 1, TEXT("Enable display update of Concert Presence to remote users."));

const TCHAR* FConcertClientPresenceManager::AssetContainerPath = TEXT("/ConcertSyncClient/ConcertAssets");

FConcertClientPresenceStateEntry::FConcertClientPresenceStateEntry(TSharedRef<FConcertClientPresenceEventBase> InPresenceEvent)
	: PresenceEvent(MoveTemp(InPresenceEvent))
	, bSyncPending(true)
{
}

FConcertClientPresenceState::FConcertClientPresenceState()
	: bIsConnected(true)
	, EditorPlayMode(EEditorPlayMode::None)
	, bVisible(true)
	, VRDevice(NAME_None)
{
}

FConcertClientPresencePersistentState::FConcertClientPresencePersistentState()
	: bVisible(true)
	, bPropagateToAll(false)
{
}

FConcertClientPresenceManager::FConcertClientPresenceManager(TSharedRef<IConcertClientSession> InSession)
	: Session(InSession)
	, CurrentAvatarMode(nullptr)
	, AssetContainer(nullptr)
	, bIsPresenceEnabled(true)
{
	// Setup the asset container.
	AssetContainer = LoadObject<UConcertAssetContainer>(nullptr, FConcertClientPresenceManager::AssetContainerPath);
	checkf(AssetContainer, TEXT("Failed to load UConcertAssetContainer (%s). See log for reason."), FConcertClientPresenceManager::AssetContainerPath);

	// Create the default presence mode factory
	PresenceModeFactory = MakeShared<FConcertClientDefaultPresenceModeFactory>(this);

	PreviousEndFrameTime = FPlatformTime::Seconds();
	SecondsSinceLastLocationUpdate = ConcertClientPresenceManagerUtil::LocationUpdateFrequencySeconds;

	Register();
}

FConcertClientPresenceManager::~FConcertClientPresenceManager()
{
	Unregister();
	ClearAllPresenceState();
}

void FConcertClientPresenceManager::AddReferencedObjects(FReferenceCollector& Collector)
{
	// Add reference to the asset container
	Collector.AddReferencedObject(AssetContainer);

	// Add reference from the factory, if any 
	if (PresenceModeFactory)
	{
		PresenceModeFactory->AddReferencedObjects(Collector);
	}

	// Add reference from the mode, if any
	if (CurrentAvatarMode)
	{
		CurrentAvatarMode->AddReferencedObjects(Collector);
	}

	// Ensure that Avatar classes used to represent remote clients are referenced.
	for (auto& Pair : OthersAvatarClasses)
	{
		Collector.AddReferencedObject(Pair.Value);
	}
}

const UConcertAssetContainer& FConcertClientPresenceManager::GetAssetContainer() const
{
	return *AssetContainer;
}

bool FConcertClientPresenceManager::IsPresenceVisible(const FConcertClientPresenceState& InPresenceState) const
{
	return InPresenceState.bVisible && IsPIEPresenceEnabled(InPresenceState) && !IsInGame();
}

bool FConcertClientPresenceManager::IsPresenceVisible(const FGuid& InEndpointId) const
{
	const FConcertClientPresenceState* PresenceState = PresenceStateMap.Find(InEndpointId);
	return PresenceState && IsPresenceVisible(*PresenceState);
}

bool FConcertClientPresenceManager::IsPIEPresenceEnabled(const FConcertClientPresenceState& InPresenceState) const
{
	return ConcertClientPresenceManagerUtil::ShowPresenceInPIE(InPresenceState.EditorPlayMode == EEditorPlayMode::PIE);
}

template<class PresenceActorClass, typename PresenceUpdateEventType>
void FConcertClientPresenceManager::UpdatePresence(AConcertClientPresenceActor* InPresenceActor, const PresenceUpdateEventType& InEvent)
{
	if (InPresenceActor)
	{
		if (PresenceActorClass* PresenceActor = Cast<PresenceActorClass>(InPresenceActor))
		{
			PresenceActor->HandleEvent(InEvent);
		}
	}
}

template<typename PresenceUpdateEventType>
void FConcertClientPresenceManager::HandleConcertClientPresenceUpdateEvent(const FConcertSessionContext& InSessionContext, const PresenceUpdateEventType& InEvent)
{
	if (!ShouldProcessPresenceEvent(InSessionContext, PresenceUpdateEventType::StaticStruct(), InEvent))
	{
		UE_LOGF(LogConcert, VeryVerbose, "Dropping presence update event for '%ls' (index %d) as it arrived out-of-order", *InSessionContext.SourceEndpointId.ToString(), InEvent.TransactionUpdateIndex);
		return;
	}

	TSharedRef<PresenceUpdateEventType> EventRef = MakeShared<PresenceUpdateEventType>(InEvent);
	FConcertClientPresenceStateEntry StateEntry(MoveTemp(EventRef));
	FConcertClientPresenceState& PresenceState = EnsurePresenceState(InSessionContext.SourceEndpointId);
	PresenceState.EventStateMap.Emplace(PresenceUpdateEventType::StaticStruct(), MoveTemp(StateEntry));
}

void FConcertClientPresenceManager::Register()
{
	Session->RegisterCustomEventHandler<FConcertClientPresenceVisibilityUpdateEvent>(this, &FConcertClientPresenceManager::HandleConcertClientPresenceVisibilityUpdateEvent);
	Session->RegisterCustomEventHandler<FConcertClientPresenceInVREvent>(this, &FConcertClientPresenceManager::HandleConcertClientPresenceInVREvent);
	Session->RegisterCustomEventHandler<FConcertClientPresenceDataUpdateEvent>(this, &FConcertClientPresenceManager::HandleConcertClientPresenceUpdateEvent<FConcertClientPresenceDataUpdateEvent>);
	Session->RegisterCustomEventHandler<FConcertClientDesktopPresenceUpdateEvent>(this, &FConcertClientPresenceManager::HandleConcertClientPresenceUpdateEvent<FConcertClientDesktopPresenceUpdateEvent>);
	Session->RegisterCustomEventHandler<FConcertClientVRPresenceUpdateEvent>(this, &FConcertClientPresenceManager::HandleConcertClientPresenceUpdateEvent<FConcertClientVRPresenceUpdateEvent>);
	Session->RegisterCustomEventHandler<FConcertPlaySessionEvent>(this, &FConcertClientPresenceManager::HandleConcertPlaySessionEvent);

	// Add handler for session client changing
	Session->OnSessionClientChanged().AddRaw(this, &FConcertClientPresenceManager::OnSessionClientChanged);

	// Register OnEndFrame events
	FCoreDelegates::OnEndFrame.AddRaw(this, &FConcertClientPresenceManager::OnEndFrame);
}

void FConcertClientPresenceManager::Unregister()
{
	Session->OnSessionClientChanged().RemoveAll(this);

	FCoreDelegates::OnEndFrame.RemoveAll(this);

	Session->UnregisterCustomEventHandler<FConcertClientPresenceVisibilityUpdateEvent>(this);
	Session->UnregisterCustomEventHandler<FConcertClientPresenceInVREvent>(this);
	Session->UnregisterCustomEventHandler<FConcertClientPresenceDataUpdateEvent>(this);
	Session->UnregisterCustomEventHandler<FConcertClientDesktopPresenceUpdateEvent>(this);
	Session->UnregisterCustomEventHandler<FConcertClientVRPresenceUpdateEvent>(this);
	Session->UnregisterCustomEventHandler<FConcertPlaySessionEvent>(this);
}

UWorld* FConcertClientPresenceManager::GetWorld(bool bIgnorePIE) const
{
	// Get the current world.
	if (GIsEditor)
	{
		FWorldContext* PIEWorldContext = GEditor->GetPIEWorldContext();
		if (!bIgnorePIE && PIEWorldContext)
		{
			return PIEWorldContext->World();
		}
		return GEditor->GetEditorWorldContext().World();
	}
	else if (UGameEngine* GameEngine = Cast<UGameEngine>(GEngine))
	{
		return GameEngine->GetGameWorld();
	}
	return nullptr;
}

FLevelEditorViewportClient* FConcertClientPresenceManager::GetPerspectiveViewport() const
{
	return (GCurrentLevelEditingViewportClient && GCurrentLevelEditingViewportClient->IsPerspective())
		? GCurrentLevelEditingViewportClient
		: nullptr;
}

void FConcertClientPresenceManager::SetPresenceModeFactory(TSharedRef<IConcertClientPresenceModeFactory> InFactory)
{
	PresenceModeFactory = InFactory;
	CurrentAvatarMode.Reset();
}

void FConcertClientPresenceManager::OnEndFrame()
{
	const double CurrentTime = FPlatformTime::Seconds();

	double DeltaTime = CurrentTime - PreviousEndFrameTime;
	SecondsSinceLastLocationUpdate += DeltaTime;

	// Game client do not generate a presence mode and state.
	if (SecondsSinceLastLocationUpdate >= ConcertClientPresenceManagerUtil::LocationUpdateFrequencySeconds && !IsInGame() && CVarEmitPresence.GetValueOnAnyThread() > 0)
	{
		// if the factory indicate the mode should be reset for recreation.
		if (PresenceModeFactory->ShouldResetPresenceMode())
		{
			CurrentAvatarMode.Reset();
		}

		// Create the presence mode if needed and send vr device events
		if (!CurrentAvatarMode)
		{
			CurrentAvatarMode = PresenceModeFactory->CreatePresenceMode();
			SendPresenceInVREvent();
		}

		// Send our current presence data to remote clients
		if (CurrentAvatarMode)
		{
			CurrentAvatarMode->SendEvents(*Session);
		}

		SecondsSinceLastLocationUpdate = 0.0;
	}
	
	PreviousEndFrameTime = CurrentTime;

	// Synchronize our local state for each remote client
	SynchronizePresenceState();
}

const TSharedPtr<FConcertClientPresenceDataUpdateEvent> FConcertClientPresenceManager::GetCachedPresenceState(const FConcertClientPresenceState& InPresenceState) const
{
	const FConcertClientPresenceStateEntry* StateItem = InPresenceState.EventStateMap.Find(FConcertClientPresenceDataUpdateEvent::StaticStruct());
	if (StateItem)
	{
		return StaticCastSharedRef<FConcertClientPresenceDataUpdateEvent>(StateItem->PresenceEvent);
	}

	return nullptr;
}

const TSharedPtr<FConcertClientPresenceDataUpdateEvent> FConcertClientPresenceManager::GetCachedPresenceState(const FGuid& InEndpointId) const
{
	const FConcertClientPresenceState* PresenceState = PresenceStateMap.Find(InEndpointId);
	return PresenceState ? GetCachedPresenceState(*PresenceState) : nullptr;
}

void FConcertClientPresenceManager::SynchronizePresenceState()
{
	const FName ActiveWorldPathName = *GetWorld()->GetPathName();

	// Process all pending state updates
	for (auto It = PresenceStateMap.CreateIterator(); It; ++It)
	{
		const FGuid& RemoteEndpointId = It.Key();
		FConcertClientPresenceState& PresenceState = It.Value();

		// Find presence world
		FName EventWorldPathName;
		const TSharedPtr<FConcertClientPresenceDataUpdateEvent> PresenceUpdateEvent = GetCachedPresenceState(PresenceState);
		EventWorldPathName = PresenceUpdateEvent.IsValid() ? PresenceUpdateEvent->WorldPath : TEXT("");
		// NOTE: since presence state doesn't report PIE world path, bInCurrentWorld will never be true when in PIE, this will need to change if we want presence to display in PIE
		const bool bInCurrentWorld = !ActiveWorldPathName.IsNone() && ActiveWorldPathName == EventWorldPathName;
		
		const bool bShowPresence = bIsPresenceEnabled && bInCurrentWorld && PresenceState.bIsConnected && IsPresenceVisible(PresenceState) && (CVarDisplayPresence.GetValueOnAnyThread() > 0);
		if (bShowPresence)
		{
			FConcertSessionClientInfo ClientSessionInfo;
			Session->FindSessionClient(RemoteEndpointId, ClientSessionInfo);

			if (!PresenceState.PresenceActor.IsValid())
			{
				PresenceState.PresenceActor = CreatePresenceActor(ClientSessionInfo.ClientInfo, PresenceState.VRDevice);
			}
			else if ((PresenceState.VRDevice.IsNone()  && PresenceState.PresenceActor->GetClass()->GetPathName() != ClientSessionInfo.ClientInfo.DesktopAvatarActorClass) // Not in VR and Desktop Avatar changed?
			        || (!PresenceState.VRDevice.IsNone() && PresenceState.PresenceActor->GetClass()->GetPathName() != ClientSessionInfo.ClientInfo.VRAvatarActorClass))    // In VR and VR Avatar changed?					
			{
				ClearPresenceActor(RemoteEndpointId);
				PresenceState.PresenceActor = CreatePresenceActor(ClientSessionInfo.ClientInfo, PresenceState.VRDevice);
			}

			if (PresenceState.PresenceActor.IsValid())
			{
				AConcertClientPresenceActor* PresenceActor = PresenceState.PresenceActor.Get();

				for (auto StateIt = PresenceState.EventStateMap.CreateIterator(); StateIt; ++StateIt)
				{
					const UScriptStruct* EventKey = StateIt.Key();
					FConcertClientPresenceStateEntry& EventItem = StateIt.Value();

					if (EventItem.bSyncPending)
					{
						FStructOnScope Event(EventKey, (uint8*)&EventItem.PresenceEvent.Get());
						PresenceActor->HandleEvent(Event);
						EventItem.bSyncPending = false;
					}
				}

				if (ClientSessionInfo.ClientInfo.DisplayName != PresenceState.DisplayName)
				{
					PresenceState.DisplayName = ClientSessionInfo.ClientInfo.DisplayName;
					PresenceActor->SetPresenceName(PresenceState.DisplayName);
				}

				if (ClientSessionInfo.ClientInfo.AvatarColor != PresenceState.AvatarColor)
				{
					PresenceState.AvatarColor = ClientSessionInfo.ClientInfo.AvatarColor;
					PresenceActor->SetPresenceColor(PresenceState.AvatarColor);
				}
			}
		}
		else
		{
			ClearPresenceActor(RemoteEndpointId);
		}

		if (!PresenceState.bIsConnected)
		{
			It.RemoveCurrent();
		}
	}
}

bool FConcertClientPresenceManager::ShouldProcessPresenceEvent(const FConcertSessionContext& InSessionContext, const UStruct* InEventType, const FConcertClientPresenceEventBase& InEvent) const
{
	check(InEventType);

	const FName EventId = *FString::Printf(TEXT("PresenceManager.%s.EndpointId:%s"), *InEventType->GetFName().ToString(), *InSessionContext.SourceEndpointId.ToString());

	FConcertScratchpadPtr SenderScratchpad = Session->GetClientScratchpad(InSessionContext.SourceEndpointId);
	if (SenderScratchpad.IsValid())
	{
		// If the event isn't required, then we can drop it if its update index is older than the last update we processed
		if (uint32* EventUpdateIndexPtr = Session->GetScratchpad()->GetValue<uint32>(EventId))
		{
			uint32& EventUpdateIndex = *EventUpdateIndexPtr;
			const bool bShouldProcess = InEvent.TransactionUpdateIndex >= EventUpdateIndex + 1; // Note: We +1 before doing the check to handle overflow
			EventUpdateIndex = InEvent.TransactionUpdateIndex;
			return bShouldProcess;
		}

		// First update for this transaction, just process it
		SenderScratchpad->SetValue<uint32>(EventId, InEvent.TransactionUpdateIndex);
		return true;
	}

	return true;
}

AConcertClientPresenceActor* FConcertClientPresenceManager::CreatePresenceActor(const FConcertClientInfo& InClientInfo, FName VRDevice)
{
	AConcertClientPresenceActor* PresenceActor = SpawnPresenceActor(InClientInfo, VRDevice);

	if (PresenceActor)
	{
		PresenceActor->SetPresenceName(InClientInfo.DisplayName);
		PresenceActor->SetPresenceColor(InClientInfo.AvatarColor);
	}

	return PresenceActor;
}

AConcertClientPresenceActor* FConcertClientPresenceManager::SpawnPresenceActor(const FConcertClientInfo& InClientInfo, FName VRDevice)
{
	check(AssetContainer);

	UWorld* World = GetWorld();
	if (!World)
	{
		UE_LOGF(LogConcert, Warning, "No world active. Presence will not be displayed");
		return nullptr;
	}

	// @todo this is potentially slow and hitchy as clients connect. It might be better to preload all the presence actor types
	UClass* PresenceActorClass = nullptr;
	if (!VRDevice.IsNone())
	{
		auto& AvatarClass = OthersAvatarClasses.FindOrAdd(InClientInfo.VRAvatarActorClass, nullptr);
		if (!AvatarClass && !InClientInfo.VRAvatarActorClass.IsEmpty())
		{
			AvatarClass = LoadObject<UClass>(nullptr, *InClientInfo.VRAvatarActorClass);
		}
		PresenceActorClass = AvatarClass;
	}
	else
	{
		auto& AvatarClass = OthersAvatarClasses.FindOrAdd(InClientInfo.DesktopAvatarActorClass, nullptr);
		if (!AvatarClass && !InClientInfo.DesktopAvatarActorClass.IsEmpty())
		{
			AvatarClass = LoadObject<UClass>(nullptr, *InClientInfo.DesktopAvatarActorClass);
		}
		PresenceActorClass = AvatarClass;
	}

	if (!PresenceActorClass)
	{
		UE_LOGF(LogConcert, Warning, "Failed to load presence actor class '%ls'. Presence will not be displayed", !VRDevice.IsNone() ? *InClientInfo.VRAvatarActorClass : *InClientInfo.DesktopAvatarActorClass);
		return nullptr;
	}

	AConcertClientPresenceActor* PresenceActor = nullptr;
	{
		const bool bWasWorldPackageDirty = World->GetOutermost()->IsDirty();

		FActorSpawnParameters ActorSpawnParameters;
		ActorSpawnParameters.Name = MakeUniqueObjectName(World, PresenceActorClass, PresenceActorClass->GetFName()); // @todo how should spawned actors be named?
		ActorSpawnParameters.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
		ActorSpawnParameters.ObjectFlags = EObjectFlags::RF_DuplicateTransient;
		ActorSpawnParameters.bDeferConstruction = true;

		PresenceActor = World->SpawnActor<AConcertClientPresenceActor>(PresenceActorClass, ActorSpawnParameters);

		// Don't dirty the level file after spawning a transient actor
		if (!bWasWorldPackageDirty)
		{
			World->GetOutermost()->SetDirtyFlag(false);
		}
	}

	if (!PresenceActor)
	{
		UE_LOGF(LogConcert, Warning, "Failed to spawn presence actor of class '%ls'. Presence will not be displayed", !VRDevice.IsNone() ? *InClientInfo.VRAvatarActorClass : *InClientInfo.DesktopAvatarActorClass);
		return nullptr;
	}

	// Setup the asset container.
	PresenceActor->InitPresence(*AssetContainer, VRDevice);
	{
		FEditorScriptExecutionGuard UCSGuard;
		PresenceActor->FinishSpawning(FTransform(), true);
	}

	return PresenceActor;
}

void FConcertClientPresenceManager::ClearPresenceActor(const FGuid& InEndpointId)
{
	FConcertClientPresenceState* PresenceState = PresenceStateMap.Find(InEndpointId);
	if (PresenceState)
	{
		DestroyPresenceActor(PresenceState->PresenceActor);
		PresenceState->PresenceActor.Reset();
	}
}

void FConcertClientPresenceManager::DestroyPresenceActor(TWeakObjectPtr<AConcertClientPresenceActor> InPresenceActor)
{
	if (AConcertClientPresenceActor* PresenceActor = InPresenceActor.Get())
	{
		UWorld* World = PresenceActor->GetWorld();
		const bool bWasWorldPackageDirty = World->GetOutermost()->IsDirty();

		const bool bNetForce = false;
		const bool bShouldModifyLevel = false;	// Don't modify level for transient actor destruction
		World->DestroyActor(PresenceActor, bNetForce, bShouldModifyLevel);

		// Don't dirty the level file after destroying a transient actor
		if (!bWasWorldPackageDirty)
		{
			World->GetOutermost()->SetDirtyFlag(false);
		}
	}
}

void FConcertClientPresenceManager::ClearAllPresenceState()
{
	for (auto& Elem : PresenceStateMap)
	{
		DestroyPresenceActor(Elem.Value.PresenceActor);
	}
	PresenceStateMap.Empty();
}

void FConcertClientPresenceManager::HandleConcertClientPresenceVisibilityUpdateEvent(const FConcertSessionContext& InSessionContext, const FConcertClientPresenceVisibilityUpdateEvent& InEvent)
{
	SetPresenceVisibility(InEvent.ModifiedEndpointId, InEvent.bVisibility);
}

void FConcertClientPresenceManager::HandleConcertPlaySessionEvent(const FConcertSessionContext& InSessionContext, const FConcertPlaySessionEvent& InEvent)
{
	bool bPlaying = (InEvent.EventType == EConcertPlaySessionEventType::BeginPlay || InEvent.EventType == EConcertPlaySessionEventType::SwitchPlay);

	// This event is sent by the server so the InSession.SourceEndpointId 
	// will be the server's guid not the client's.
	FConcertClientPresenceState& PresenceState = EnsurePresenceState(InEvent.PlayEndpointId);

	if (bPlaying)
	{
		PresenceState.EditorPlayMode = InEvent.bIsSimulating ? EEditorPlayMode::SIE : EEditorPlayMode::PIE;
	}
	else
	{
		PresenceState.EditorPlayMode = EEditorPlayMode::None;
	}
}

void FConcertClientPresenceManager::OnSessionClientChanged(IConcertClientSession& InSession, EConcertClientStatus InClientStatus, const FConcertSessionClientInfo& InClientInfo)
{
	if (InClientStatus == EConcertClientStatus::Connected || InClientStatus == EConcertClientStatus::Updated)
	{
		// Sync persistent presence when a client connects or is updated
		if (FConcertClientPresencePersistentState* PresencePersistentState = PresencePersistentStateMap.Find(InClientInfo.ClientInfo.DisplayName))
		{
			SetPresenceVisibility(InClientInfo.ClientEndpointId, PresencePersistentState->bVisible, PresencePersistentState->bPropagateToAll);
		}

		if (!IsInGame())
		{
			// Send avatar-related info for this client when a remote client connects or is updated
			SendPresenceInVREvent(&InClientInfo.ClientEndpointId);
		}
	}
	else if (InClientStatus == EConcertClientStatus::Disconnected)
	{
		// Disconnect presence when a client disconnects
		if (FConcertClientPresenceState* PresenceState = PresenceStateMap.Find(InClientInfo.ClientEndpointId))
		{
			PresenceState->bIsConnected = false;
		}
	}
}

void FConcertClientPresenceManager::SendPresenceInVREvent(const FGuid* InEndpointId)
{
	FConcertClientPresenceInVREvent Event;
	Event.VRDevice = CurrentAvatarMode ? CurrentAvatarMode->GetVRDeviceType() : FName();

	if (InEndpointId)
	{
		Session->SendCustomEvent(Event, *InEndpointId, EConcertMessageFlags::ReliableOrdered);
	}
	else
	{
		Session->SendCustomEvent(Event, Session->GetSessionClientEndpointIds(), EConcertMessageFlags::ReliableOrdered);
	}
}

void FConcertClientPresenceManager::HandleConcertClientPresenceInVREvent(const FConcertSessionContext& InSessionContext, const FConcertClientPresenceInVREvent& InEvent)
{
	UpdatePresenceAvatar(InSessionContext.SourceEndpointId, InEvent.VRDevice);
}

void FConcertClientPresenceManager::UpdatePresenceAvatar(const FGuid& InEndpointId, FName VRDevice)
{
	FConcertClientPresenceState& PresenceState = EnsurePresenceState(InEndpointId);
	PresenceState.VRDevice = VRDevice;

	if (PresenceState.PresenceActor.IsValid())
	{
		// Presence actor will be recreated on next call to OnEndFrame
		ClearPresenceActor(InEndpointId);
	}
}

void FConcertClientPresenceManager::SetPresenceVisibility(const FGuid& InEndpointId, bool bVisibility, bool bPropagateToAll)
{
	FConcertClientPresenceState& PresenceState = EnsurePresenceState(InEndpointId);
	PresenceState.bVisible = bVisibility;

	if (bPropagateToAll)
	{
		FConcertClientPresenceVisibilityUpdateEvent VisibilityUpdateEvent;
		VisibilityUpdateEvent.ModifiedEndpointId = InEndpointId;
		VisibilityUpdateEvent.bVisibility = bVisibility;

		Session->SendCustomEvent(VisibilityUpdateEvent, Session->GetSessionClientEndpointIds(), EConcertMessageFlags::ReliableOrdered);
	}
}

bool FConcertClientPresenceManager::IsInGame() const
{
	return !GIsEditor;
}

bool FConcertClientPresenceManager::IsInPIE() const
{
	return GEditor && GEditor->PlayWorld && !GEditor->bIsSimulatingInEditor;
}

bool FConcertClientPresenceManager::IsInSIE() const
{
	return GEditor && GEditor->PlayWorld && GEditor->bIsSimulatingInEditor;
}

void FConcertClientPresenceManager::SetPresenceEnabled(const bool bIsEnabled)
{
	bIsPresenceEnabled = bIsEnabled;
}

void FConcertClientPresenceManager::SetPresenceVisibility(const FString& InDisplayName, bool bVisibility, bool bPropagateToAll)
{
	FConcertClientPresencePersistentState& PresencePersistentState = PresencePersistentStateMap.FindOrAdd(InDisplayName);
	PresencePersistentState.bVisible = bVisibility;
	PresencePersistentState.bPropagateToAll = bPropagateToAll;

	TArray<FGuid, TInlineAllocator<2>> MatchingEndpointIds;
	for (const auto& PresenceStatePair : PresenceStateMap)
	{
		if (PresenceStatePair.Value.DisplayName == InDisplayName)
		{
			MatchingEndpointIds.Add(PresenceStatePair.Key);
		}
	}

	for (const FGuid& MatchingEndpointId : MatchingEndpointIds)
	{
		SetPresenceVisibility(MatchingEndpointId, bVisibility, bPropagateToAll);
	}

	// We also need to propagate a fake visibility change if the display name matches our local 
	// presence data, as that isn't handled by the loop above since we have no local presence
	if (bPropagateToAll && Session->GetLocalClientInfo().DisplayName == InDisplayName)
	{
		FConcertClientPresenceVisibilityUpdateEvent VisibilityUpdateEvent;
		VisibilityUpdateEvent.ModifiedEndpointId = Session->GetSessionClientEndpointId();
		VisibilityUpdateEvent.bVisibility = bVisibility;

		Session->SendCustomEvent(VisibilityUpdateEvent, Session->GetSessionClientEndpointIds(), EConcertMessageFlags::ReliableOrdered);
	}
}

FTransform FConcertClientPresenceManager::GetPresenceTransform(const FGuid& EndpointId) const
{
	if (CurrentAvatarMode && EndpointId == Session->GetSessionClientEndpointId())
	{
		return CurrentAvatarMode->GetTransform();
	}
	if (const TSharedPtr<FConcertClientPresenceDataUpdateEvent> OtherClientState = GetCachedPresenceState(EndpointId))
	{
		return FTransform(OtherClientState->Orientation, OtherClientState->Position);
	}
	return FTransform::Identity;
}

void FConcertClientPresenceManager::TogglePresenceVisibility(const FGuid& InEndpointId, bool bPropagateToAll)
{
	if (const FConcertClientPresenceState* PresenceState = PresenceStateMap.Find(InEndpointId))
	{
		SetPresenceVisibility(InEndpointId, !PresenceState->bVisible, bPropagateToAll);
	}
}

FConcertClientPresenceState& FConcertClientPresenceManager::EnsurePresenceState(const FGuid& InEndpointId)
{
	FConcertClientPresenceState* PresenceState = PresenceStateMap.Find(InEndpointId);
	if (!PresenceState)
	{
		PresenceState = &PresenceStateMap.Add(InEndpointId);
		{
			FConcertSessionClientInfo ClientSessionInfo;
			if (Session->FindSessionClient(InEndpointId, ClientSessionInfo))
			{
				PresenceState->DisplayName = ClientSessionInfo.ClientInfo.DisplayName;
				PresenceState->AvatarColor = ClientSessionInfo.ClientInfo.AvatarColor;
			}
		}
		PresencePersistentStateMap.FindOrAdd(PresenceState->DisplayName);
	}
	return *PresenceState;
}

void FConcertClientPresenceManager::GetPresenceClientActions(const FConcertSessionClientInfo& InClientInfo, TArray<FConcertActionDefinition>& OutActionDefs)
{
	// Only add buttons for the clients in our session
	if (InClientInfo.ClientEndpointId != Session->GetSessionClientEndpointId())
	{
		FConcertSessionClientInfo Unused;
		if (!Session->FindSessionClient(InClientInfo.ClientEndpointId, Unused))
		{
			return;
		}
	}

	FConcertActionDefinition& JumpToPresenceDef = OutActionDefs.AddDefaulted_GetRef();
	JumpToPresenceDef.IsEnabled = MakeAttributeSP(this, &FConcertClientPresenceManager::IsJumpToPresenceEnabled, InClientInfo.ClientEndpointId);
	JumpToPresenceDef.Text = FEditorFontGlyphs::Map_Marker;
	JumpToPresenceDef.ToolTipText = LOCTEXT("JumpToPresenceToolTip", "Jump to the presence location of this client");
	JumpToPresenceDef.OnExecute.BindSP(this, &FConcertClientPresenceManager::OnJumpToPresence, InClientInfo.ClientEndpointId, FTransform::Identity);
	JumpToPresenceDef.IconStyle = TEXT("Concert.JumpToLocation");

	FConcertActionDefinition& ShowHidePresenceDef = OutActionDefs.AddDefaulted_GetRef();
	ShowHidePresenceDef.IsEnabled = MakeAttributeSP(this, &FConcertClientPresenceManager::IsShowHidePresenceEnabled, InClientInfo.ClientEndpointId);
	ShowHidePresenceDef.Text = MakeAttributeSP(this, &FConcertClientPresenceManager::GetShowHidePresenceText, InClientInfo.ClientEndpointId);
	ShowHidePresenceDef.ToolTipText = MakeAttributeSP(this, &FConcertClientPresenceManager::GetShowHidePresenceToolTip, InClientInfo.ClientEndpointId);
	ShowHidePresenceDef.OnExecute.BindSP(this, &FConcertClientPresenceManager::OnShowHidePresence, InClientInfo.ClientEndpointId);
	ShowHidePresenceDef.IconStyle = MakeAttributeSP(this, &FConcertClientPresenceManager::GetShowHidePresenceIconStyle, InClientInfo.ClientEndpointId);
}

bool FConcertClientPresenceManager::IsJumpToPresenceEnabled(FGuid InEndpointId) const
{
	// Disable this button for ourselves since we don't have presence
	if (InEndpointId == Session->GetSessionClientEndpointId())
	{
		return false;
	}

	// Only enable the button if we have a valid perspective viewport to move and we're not in VR
	if (!GetPerspectiveViewport() || IVREditorModule::Get().GetVRModeBase())
	{
		return false;
	}

	// Can only jump to clients that exist, have cached state and both clients are in the same level.
	if (const FConcertClientPresenceState* PresenceState = PresenceStateMap.Find(InEndpointId))
	{
		const TSharedPtr<FConcertClientPresenceDataUpdateEvent> CachedPresenceState = GetCachedPresenceState(*PresenceState);
		if (CachedPresenceState.IsValid())
		{
			// The clients should be in the same world to enable teleporting. Note that both paths below are the from
			// the editor world context. (The presence manager don't use PIE/SIE context world path).
			return *GetWorld(/*bIgnorePIE=*/true)->GetPathName() == CachedPresenceState->WorldPath;
		}
	}

	return false; // We don't know "InEndPointId".
}

double FConcertClientPresenceManager::GetLocationUpdateFrequency()
{
	return ConcertClientPresenceManagerUtil::LocationUpdateFrequencySeconds;
}

void FConcertClientPresenceManager::InitiateJumpToPresence(const FGuid& InEndpointId, const FTransform& InTransformOffset)
{
	OnJumpToPresence(InEndpointId, InTransformOffset);
}

void FConcertClientPresenceManager::OnJumpToPresence(FGuid InEndpointId, FTransform InTransformOffset)
{
	const TSharedPtr<FConcertClientPresenceDataUpdateEvent> OtherClientState = GetCachedPresenceState(InEndpointId);
	if (OtherClientState.IsValid())
	{
		FQuat JumpRotation = InTransformOffset.GetRotation() * OtherClientState->Orientation;
		FRotator OtherClientRotation(JumpRotation.Rotator());

		FVector JumpPosition = OtherClientState->Position + InTransformOffset.GetLocation();

		// Disregard pitch and roll when teleporting to a VR presence or we are in VR mode.
		if (OtherClientState->PresenceType == EPresenceModeType::VRMode || IVREditorModule::Get().GetVRModeBase())
		{
			OtherClientRotation.Pitch = 0.0f;
			OtherClientRotation.Roll = 0.0f;
		}

		if (IsInPIE())
		{
			check(GEditor->PlayWorld);

			// In 'play in editor', we need to change the 'player' location/orientation.
			if (APlayerController* PC = GEditor->PlayWorld->GetFirstPlayerController())
			{
				PC->ClientSetLocation(JumpPosition, OtherClientRotation);
			}
		}
		else
		{
			if (FLevelEditorViewportClient* PerspectiveViewport = GetPerspectiveViewport())
			{
				if (UVREditorModeBase* VRMode = IVREditorModule::Get().GetVRModeBase())
				{
					const FTransform HeadTransform = VRMode->GetHeadTransform();
					const FTransform RoomTransform = VRMode->GetRoomTransform();
					JumpPosition -= HeadTransform.GetLocation() - RoomTransform.GetLocation();
					OtherClientRotation.Yaw -= (HeadTransform.Rotator().Yaw - RoomTransform.Rotator().Yaw);
				}

				PerspectiveViewport->SetViewLocation(JumpPosition);
				PerspectiveViewport->SetViewRotation(OtherClientRotation);
			}
			else
			{
				UE_LOGF(LogConcert, Log, "Unable to find a perspective viewport to jump presence.");
			}
		}
	}
}

bool FConcertClientPresenceManager::IsShowHidePresenceEnabled(FGuid InEndpointId) const
{
	// Disable this button for ourselves since we don't have presence
	if (InEndpointId == Session->GetSessionClientEndpointId())
	{
		return false;
	}

	const FConcertClientPresenceState* PresenceState = PresenceStateMap.Find(InEndpointId);
	return PresenceState && IsPIEPresenceEnabled(*PresenceState);
}

FText FConcertClientPresenceManager::GetShowHidePresenceText(FGuid InEndpointId) const
{
	return IsPresenceVisible(InEndpointId)
		? FEditorFontGlyphs::Eye
		: FEditorFontGlyphs::Eye_Slash;
}

FName FConcertClientPresenceManager::GetShowHidePresenceIconStyle(FGuid InEndpointId) const
{
	return IsPresenceVisible(InEndpointId)
		? TEXT("Concert.ShowPresence")  // Eye open icon.
		: TEXT("Concert.HidePresence"); // Eye closed icon.
}

FText FConcertClientPresenceManager::GetShowHidePresenceToolTip(FGuid InEndpointId) const
{
	return IsPresenceVisible(InEndpointId)
		? LOCTEXT("HidePresenceToolTip", "Hide the presence for this client\nHold Ctrl to propagate this visibility change to all connected clients.")
		: LOCTEXT("ShowPresenceToolTip", "Show the presence for this client\nHold Ctrl to propagate this visibility change to all connected clients.");
}

void FConcertClientPresenceManager::OnShowHidePresence(FGuid InEndpointId)
{
	const bool bPropagateToAll = FSlateApplication::Get().GetModifierKeys().IsControlDown();
	TogglePresenceVisibility(InEndpointId, bPropagateToAll);
}

FString FConcertClientPresenceManager::GetPresenceWorldPath(const FGuid& InEndpointId, EEditorPlayMode& OutEditorPlayMode) const
{
	FString WorldPath;
	OutEditorPlayMode = EEditorPlayMode::None;

	// Is it the local client endpoint?
	if (InEndpointId == Session->GetSessionClientEndpointId())
	{
		if (IsInPIE())
		{
			OutEditorPlayMode = EEditorPlayMode::PIE;
		}
		else if (IsInSIE())
		{
			OutEditorPlayMode = EEditorPlayMode::SIE;
		}

		// Always return the Non-PIE/SIE world path. When the editor is in PIE or SIE, it prefixes the world name with something like 'UEDPIE_10_'
		// and this leak into the world path. For presence management purpose, we want don't want to display the PIE/SIE context world path to
		// the user and we want to be able to jump to the other presence when two users are visualizing the same level (comparing their world path)
		// whatever the play mode is (PIE/SIE/None).
		if (UWorld* CurrentWorld = GetWorld(/*bIgnorePIE=*/true))
		{
			WorldPath = CurrentWorld->GetPathName();
		}
	}
	else
	{
		// Is it the endpoint a known remote client?
		if (const FConcertClientPresenceState* PresenceState = PresenceStateMap.Find(InEndpointId))
		{
			const TSharedPtr<FConcertClientPresenceDataUpdateEvent> CachedPresenceState = GetCachedPresenceState(*PresenceState);
			if (CachedPresenceState.IsValid())
			{
				OutEditorPlayMode = PresenceState->EditorPlayMode;
				WorldPath = CachedPresenceState->WorldPath.ToString(); // The cached world path is the non-PIE world path (i.e. the level editor context world path).
			}
		}
	}
	return WorldPath;
}

FConcertClientDefaultPresenceModeFactory::FConcertClientDefaultPresenceModeFactory(FConcertClientPresenceManager* InManager)
	: Manager(InManager)
	, VRDeviceType(NAME_None)
	, bShouldResetPresenceMode(true)
{
	// Add handler for VR mode
	IVREditorModule::Get().OnVREditingModeEnter().AddRaw(this, &FConcertClientDefaultPresenceModeFactory::OnVREditingModeEnter);
	IVREditorModule::Get().OnVREditingModeExit().AddRaw(this, &FConcertClientDefaultPresenceModeFactory::OnVREditingModeExit);

	// Set the initial vr device if any
	QueryVRDeviceType();
}

FConcertClientDefaultPresenceModeFactory::~FConcertClientDefaultPresenceModeFactory()
{
	IVREditorModule::Get().OnVREditingModeEnter().RemoveAll(this);
	IVREditorModule::Get().OnVREditingModeExit().RemoveAll(this);
}

bool FConcertClientDefaultPresenceModeFactory::ShouldResetPresenceMode() const
{
	return bShouldResetPresenceMode;
}

TUniquePtr<IConcertClientBasePresenceMode> FConcertClientDefaultPresenceModeFactory::CreatePresenceMode()
{
	bShouldResetPresenceMode = false;
	if (!VRDeviceType.IsNone())
	{
		return MakeUnique<FConcertClientVRPresenceMode>(Manager, VRDeviceType);
	}
	else
	{
		return MakeUnique<FConcertClientDesktopPresenceMode>(Manager);
	}
}

void FConcertClientDefaultPresenceModeFactory::AddReferencedObjects(FReferenceCollector& Collector)
{
	// No reference to add
}

void FConcertClientDefaultPresenceModeFactory::OnVREditingModeEnter()
{
	QueryVRDeviceType();
	bShouldResetPresenceMode = true;
}

void FConcertClientDefaultPresenceModeFactory::OnVREditingModeExit()
{
	VRDeviceType = NAME_None;
	bShouldResetPresenceMode = true;
}

void FConcertClientDefaultPresenceModeFactory::QueryVRDeviceType()
{
	if (UVREditorModeBase* VRModeBase = IVREditorModule::Get().GetVRModeBase())
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (UVREditorMode* VRMode = Cast<UVREditorMode>(VRModeBase))
		{
			VRDeviceType = VRMode->GetHMDDeviceType();
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		else
		{
			static const FName Generic("Generic");
			VRDeviceType = Generic;
		}
	}
	else
	{
		VRDeviceType = NAME_None;
	}
}

#else

double FConcertClientPresenceManager::GetLocationUpdateFrequency()
{
	return ConcertClientPresenceManagerUtil::LocationUpdateFrequencySeconds;
}

#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
