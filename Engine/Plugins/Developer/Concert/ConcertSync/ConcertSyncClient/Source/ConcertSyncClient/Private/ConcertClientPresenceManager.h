// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConcertActionDefinition.h"
#include "ConcertPresenceEvents.h"
#include "ConcertMessages.h"
#include "ConcertWorkspaceMessages.h"
#include "ConcertClientPresenceActor.h"
#include "ConcertClientPresenceMode.h"
#include "IConcertClientPresenceManager.h"
#include "UObject/Class.h"
#include "UObject/GCObject.h"

#if WITH_EDITOR

#include "ViewportWorldInteraction.h"

class IConcertClientSession;
enum class EEditorPlayMode : uint8;

/** Remote client event items */
struct FConcertClientPresenceStateEntry
{
	FConcertClientPresenceStateEntry(TSharedRef<FConcertClientPresenceEventBase> InPresenceEvent);

	/** Presence state */
	TSharedRef<FConcertClientPresenceEventBase> PresenceEvent;

	/** Whether state needs to be synchronized to actor */
	bool bSyncPending;
};

/** State for remote clients associated with client id */
struct FConcertClientPresenceState
{
	FConcertClientPresenceState();

	/** State map */
	TMap<UScriptStruct*, FConcertClientPresenceStateEntry> EventStateMap;

	/** Display name */
	FString DisplayName;

	/** Avatar color. */
	FLinearColor AvatarColor;

	/** Whether client is connected */
	bool bIsConnected;

	/** The editor play mode (PIE/SIE/None). */
	EEditorPlayMode EditorPlayMode;

	/** Whether client is visible */
	bool bVisible;

	/** Whether client is using a VRDevice */
	FName VRDevice;

	/** Presence actor */
	TWeakObjectPtr<AConcertClientPresenceActor> PresenceActor;
};

/** State that persists beyond remote client sessions, associated with display name */
struct FConcertClientPresencePersistentState
{
	FConcertClientPresencePersistentState();

	/** Whether client is visible */
	bool bVisible;

	/** Whether the visibility of this client should be propagated to others? */
	bool bPropagateToAll;
};

class FConcertClientPresenceManager : public TSharedFromThis<FConcertClientPresenceManager>, public FGCObject, public IConcertClientPresenceManager
{
public:
	FConcertClientPresenceManager(TSharedRef<IConcertClientSession> InSession);
	~FConcertClientPresenceManager();

	//~ FGCObject interfaces
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FConcertClientPresenceManager");
	}

	/** Gets the container for all the assets of Concert clients. */
	const class UConcertAssetContainer& GetAssetContainer() const;

	/** Returns true if current session is in Game (i.e. !GIsEditor) */
	bool IsInGame() const;

	/** Returns true if current session is in PIE */
	bool IsInPIE() const;

	/** Returns true if current session is in SIE */
	bool IsInSIE() const;

	/** 
	 * Get the current world
	 * @param bIgnorePIE if bIgnorePIE is true will still return the editor world even if an PIE world exists.
	 * @return the current world
	 */
	UWorld* GetWorld(bool bIgnorePIE = false) const;

	/** Get the active perspective viewport */
	FLevelEditorViewportClient * GetPerspectiveViewport() const;

	/** Set the presence mode factory. */
	virtual void SetPresenceModeFactory(TSharedRef<IConcertClientPresenceModeFactory> InFactory) override;

	/** Set whether presence is currently enabled and should be shown (unless hidden by other settings) */
	virtual void SetPresenceEnabled(const bool bIsEnabled = true) override;

	/** Set presence visibility by name */
	virtual void SetPresenceVisibility(const FString& InDisplayName, bool bVisibility, bool bPropagateToAll = false) override;

	/** Set Presence visibility by client id. */
	virtual void SetPresenceVisibility(const FGuid& EndpointId, bool bVisibility, bool bPropagateToAll = false) override;

	/** Get the presence transform */
	virtual FTransform GetPresenceTransform(const FGuid& EndpointId) const override;

	/** Jump (teleport) to another presence */
	virtual void InitiateJumpToPresence(const FGuid& InEndpointId, const FTransform& InTransformOffset = FTransform::Identity) override;

	/** Get location update frequency */
	static double GetLocationUpdateFrequency();

	/**
	 * Returns the path to the UWorld object opened in the editor of the specified client endpoint.
	 * The information may be unavailable if the client was disconnected, the information hasn't replicated yet
	 * or the code was not compiled as part of the UE Editor. This function always returns the editor context world
	 * path, never the PIE/SIE context world path (which contains a UEDPIE_%d_ decoration embedded) even if the user
	 * is in PIE or SIE. For example, the PIE/SIE context world path would look like '/Game/UEDPIE_10_FooMap.FooMap'
	 * while the editor context world returned by this function looks like '/Game/FooMap.FooMap'.
	 * @param[in] EndpointId The end point of a clients connected to the session (local or remote).
	 * @param[out] OutEditorPlayMode Indicates if the client corresponding to the end point is in PIE, SIE or simply editing.
	 * @return The path to the world being opened in the specified end point editor or an empty string if the information is not available.
	 */
	virtual FString GetPresenceWorldPath(const FGuid& InEndpointId, EEditorPlayMode& OutEditorPlayMode) const override;

	/**
	 * Returns presence action specific that can be performed with the specified client, like jumping to this client presence.
	 * @param InClientInfo The client for which the actions must be defined.
	 * @param OutActions The available action for this client.
	 */
	void GetPresenceClientActions(const FConcertSessionClientInfo& InClientInfo, TArray<FConcertActionDefinition>& OutActions);

private:

	/** Register event and delegate handlers */
	void Register();

	/** Unregister event and delegate handlers */
	void Unregister();

	/** Is presence visible for the given state? */
	bool IsPresenceVisible(const FConcertClientPresenceState& InPresenceState) const;

	/** Return if PIE presence is potentially visible for a state. */
	bool IsPIEPresenceEnabled(const FConcertClientPresenceState& InPresenceState) const;

	/** Is presence visible for the given endpoint id? */
	bool IsPresenceVisible(const FGuid& InEndpointId) const;

	/** Handles end-of-frame updates */
	void OnEndFrame();

	/** Synchronize presence state */
	void SynchronizePresenceState();

	/** Get cached presence state for given endpoint id */
	const TSharedPtr<FConcertClientPresenceDataUpdateEvent> GetCachedPresenceState(const FConcertClientPresenceState& InPresenceState) const;

	/** Get cached presence state for given endpoint id */
	const TSharedPtr<FConcertClientPresenceDataUpdateEvent> GetCachedPresenceState(const FGuid& InEndpointId) const;

	/** Update existing presence with event data */
	template<class PresenceActorClass, typename PresenceUpdateEventType>
	void UpdatePresence(AConcertClientPresenceActor* InPresenceActor, const PresenceUpdateEventType& InEvent);

	/** Handle a presence data update from another client session */
	template<typename PresenceUpdateEventType>
	void HandleConcertClientPresenceUpdateEvent(const FConcertSessionContext& InSessionContext, const PresenceUpdateEventType& InEvent);

	/** Handle a presence visibility update from another client session */
	void HandleConcertClientPresenceVisibilityUpdateEvent(const FConcertSessionContext& InSessionContext, const FConcertClientPresenceVisibilityUpdateEvent& InEvent);

	/** Returns true if presence event was not received out-of-order */
	bool ShouldProcessPresenceEvent(const FConcertSessionContext& InSessionContext, const UStruct* InEventType, const FConcertClientPresenceEventBase& InEvent) const;

	/** Create a new presence actor */
	AConcertClientPresenceActor* CreatePresenceActor(const FConcertClientInfo& InClientInfo, FName VRDevice);

	/** Spawn a presence actor */
	AConcertClientPresenceActor* SpawnPresenceActor(const FConcertClientInfo& InClientInfo, FName VRDevice);

	/** Clear presence */
	void ClearPresenceActor(const FGuid& InEndpointId);

	/** Destroy a presence */
	void DestroyPresenceActor(TWeakObjectPtr<AConcertClientPresenceActor> PresenceActor);

	/** Clear all presence */
	void ClearAllPresenceState();

	/** Handle a play session update from another client session */
	void HandleConcertPlaySessionEvent(const FConcertSessionContext& InSessionContext, const FConcertPlaySessionEvent &InEvent);

	/** Handle a client session disconnect */
	void OnSessionClientChanged(IConcertClientSession& InSession, EConcertClientStatus InClientStatus, const FConcertSessionClientInfo& InClientInfo);

	/** Notifies other clients to update avatar for this client */
	void SendPresenceInVREvent(const FGuid* InEndpointId = nullptr);

	/** Handle a presence in VR update from another client session */
	void HandleConcertClientPresenceInVREvent(const FConcertSessionContext& InSessionContext, const FConcertClientPresenceInVREvent& InEvent);

	/** Updates presence avatar for remote client by invalidating current presence actor */
	void UpdatePresenceAvatar(const FGuid& InEndpointId, FName VRDevice);

	/** Toggle presence visibility */
	void TogglePresenceVisibility(const FGuid& InEndpointId, bool bPropagateToAll = false);

	/** Ensure presence state info for client session */
	FConcertClientPresenceState& EnsurePresenceState(const FGuid& InEndpointId);

	/** Is the jump-to button enabled for the given endpoint? */
	bool IsJumpToPresenceEnabled(FGuid InEndpointId) const;

	/** Handle the show/hide button being clicked for the given endpoint */
	void OnJumpToPresence(FGuid InEndpointId, FTransform InTransformOffset);

	/** Is the show/hide button enabled for the given endpoint? */
	bool IsShowHidePresenceEnabled(FGuid InEndpointId) const;

	/** Get the correct show/hide text for the given endpoint */
	FText GetShowHidePresenceText(FGuid InEndpointId) const;
	
	/** Get the correct show/hide icon style for the given endpoint */
	FName GetShowHidePresenceIconStyle(FGuid InEndpointId) const;

	/** Get the correct show/hide tooltip for the given endpoint */
	FText GetShowHidePresenceToolTip(FGuid InEndpointId) const;

	/** Handle the show/hide button being clicked for the given endpoint */
	void OnShowHidePresence(FGuid InEndpointId);

	/** Session Pointer */
	TSharedRef<IConcertClientSession> Session;

	/** Factory in charge of creating presence mode and avatar. */
	TSharedPtr<IConcertClientPresenceModeFactory> PresenceModeFactory;

	/** Presence avatar mode for this client */
	TUniquePtr<IConcertClientBasePresenceMode> CurrentAvatarMode;

	/** The asset container path */
	static const TCHAR* AssetContainerPath;

	/** Container of assets */
	TObjectPtr<class UConcertAssetContainer> AssetContainer;

	/** True if presence is currently enabled and should be shown (unless hidden by other settings) */
	bool bIsPresenceEnabled;

	/** The list of loaded classes representing remote clients avatar. */
	TMap<FString, TObjectPtr<UClass>> OthersAvatarClasses;

	/** Presence state associated with remote client id */
	TMap<FGuid, FConcertClientPresenceState> PresenceStateMap;

	/** Presence state associated with client display name */
	TMap<FString, FConcertClientPresencePersistentState> PresencePersistentStateMap;

	/** Time of previous call to OnEndFrame */
	double PreviousEndFrameTime;

	/** Time since last location update for this client */
	double SecondsSinceLastLocationUpdate;
};

/**
 * Implementation for the default PresenceMode factory
 */
class FConcertClientDefaultPresenceModeFactory : public IConcertClientPresenceModeFactory
{
public:
	FConcertClientDefaultPresenceModeFactory(FConcertClientPresenceManager* InManager);
	virtual ~FConcertClientDefaultPresenceModeFactory();

	virtual bool ShouldResetPresenceMode() const override;
	virtual TUniquePtr<IConcertClientBasePresenceMode> CreatePresenceMode() override;
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
private:
	/** Handle entering VR */
	void OnVREditingModeEnter();

	/** Handle exiting VR */
	void OnVREditingModeExit();

	/** Determine new value for VRDeviceType based on current mode */
	void QueryVRDeviceType();

	/** Holds the manager. */
	FConcertClientPresenceManager* Manager;

	/** Holds the vr device type, NAME_None if not in VR. */
	FName VRDeviceType;

	/** Flag to indicate the presence mode should be recreated from the factory. */
	bool bShouldResetPresenceMode;
};

#else

class FConcertClientPresenceManager
{
public:
	static double GetLocationUpdateFrequency();
};

#endif
