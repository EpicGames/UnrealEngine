// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertClientPresenceMode.h"
#include "ConcertClientPresenceManager.h"
#include "CoreMinimal.h"
#include "GameFramework/Pawn.h"
#include "IConcertSession.h"
#include "IConcertSessionHandler.h"
#include "ConcertClientPresenceActor.h"
#include "ConcertPresenceEvents.h"
#include "ConcertSyncArchives.h"
#include "ConcertLogGlobal.h"
#include "Scratchpad/ConcertScratchpad.h"
#include "Engine/Blueprint.h"
#include "Engine/Engine.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Modules/ModuleManager.h"
#include "ConcertClientDesktopPresenceActor.h"
#include "ConcertClientVRPresenceActor.h"
#include "EngineUtils.h"
#include "Features/IModularFeatures.h"
#include "IMotionController.h"

#if WITH_EDITOR
#include "Editor.h"
#include "EditorWorldExtension.h"
#include "IAssetViewport.h"
#include "IVREditorModule.h"
#include "LevelEditor.h"
#include "SLevelViewport.h"
#include "Teleporter/VREditorTeleporter.h"
#include "VREditorInteractor.h"
#include "VREditorMode.h"
#include "ViewportWorldInteraction.h"
#endif

#if WITH_EDITOR

void FConcertClientBasePresenceMode::SetUpdateIndex(IConcertClientSession& Session, const FName& InEventName, FConcertClientPresenceEventBase& OutEvent)
{
	const FName EventId = *FString::Printf(TEXT("PresenceManager.%s.EndpointId:%s"), *InEventName.ToString(), *Session.GetSessionClientEndpointId().ToString());

	OutEvent.TransactionUpdateIndex = 0;

	if (uint32* PresenceUpdateIndexPtr = Session.GetScratchpad()->GetValue<uint32>(EventId))
	{
		uint32& PresenceUpdateIndex = *PresenceUpdateIndexPtr;
		OutEvent.TransactionUpdateIndex = PresenceUpdateIndex++;
	}
	else
	{
		Session.GetScratchpad()->SetValue<uint32>(EventId, OutEvent.TransactionUpdateIndex);
	}
}

void FConcertClientBasePresenceMode::SendEvents(IConcertClientSession& Session)
{
	check(ParentManager);

	UWorld* World = ParentManager->GetWorld();
	if (!World)
	{
		return;
	}

	const FTransform PresenceHeadTransform = GetTransform();

	EEditorPlayMode EditorPlayModePlaceholder;
	FConcertClientPresenceDataUpdateEvent PresenceDataUpdatedEvent;
	PresenceDataUpdatedEvent.WorldPath = *ParentManager->GetPresenceWorldPath(Session.GetSessionClientEndpointId(), EditorPlayModePlaceholder); // The Non-PIE world path, i.e. the "UEDPIE_%d_" decoration stripped away.
	PresenceDataUpdatedEvent.Position = PresenceHeadTransform.GetLocation();
	PresenceDataUpdatedEvent.Orientation = PresenceHeadTransform.GetRotation();
	if (!ParentManager->IsInPIE() && IVREditorModule::Get().GetVRModeBase())
	{
		PresenceDataUpdatedEvent.PresenceType = EPresenceModeType::VRMode;
	}

	SetUpdateIndex(Session, FConcertClientPresenceDataUpdateEvent::StaticStruct()->GetFName(), PresenceDataUpdatedEvent);

	Session.SendCustomEvent(PresenceDataUpdatedEvent, Session.GetSessionClientEndpointIds(), EConcertMessageFlags::None);
}

FTransform FConcertClientBasePresenceMode::GetTransform()
{
	check(ParentManager);

	FTransform NewHeadTransform = FTransform::Identity;

	if (ParentManager->IsInPIE())
	{
		check(GEditor->PlayWorld);

		// In PIE, take the transform of the active Player Controller
		if (APlayerController* PC = GEditor->PlayWorld->GetFirstPlayerController())
		{
			FVector PCLocation = FVector::ZeroVector;
			FRotator PCRotation = FRotator::ZeroRotator;
			PC->GetPlayerViewPoint(PCLocation, PCRotation);
			NewHeadTransform = FTransform(PCRotation, PCLocation);
		}
	}
	else if (UVREditorModeBase* VRMode = IVREditorModule::Get().GetVRModeBase())
	{
		NewHeadTransform = VRMode->GetHeadTransform();
	}
	else if (FLevelEditorViewportClient* PerspectiveViewport = ParentManager->GetPerspectiveViewport())
	{
		// Take the position from the active viewport
		NewHeadTransform = FTransform(PerspectiveViewport->GetViewRotation(), PerspectiveViewport->GetViewLocation());
	}
	else
	{ 
		NewHeadTransform = LastHeadTransform;
	}

	LastHeadTransform = NewHeadTransform;
	return LastHeadTransform;
}

void FConcertClientDesktopPresenceMode::SendEvents(IConcertClientSession& Session)
{
	FConcertClientBasePresenceMode::SendEvents(Session);

	// Send desktop events if not in PIE and not in VR editor
	if (!ParentManager->IsInPIE() && !IVREditorModule::Get().GetVRModeBase())
	{
		FLevelEditorViewportClient* ActiveViewportClient = ParentManager->GetPerspectiveViewport();
		if (ActiveViewportClient && ActiveViewportClient->Viewport && ActiveViewportClient->Viewport->GetSizeXY().GetMin() > 0)
		{
			FIntPoint CurrentCursorLocation;
			ActiveViewportClient->Viewport->GetMousePos(CurrentCursorLocation);

			if (CurrentCursorLocation != CachedDesktopCursorLocation)
			{
				CachedDesktopCursorLocation = CurrentCursorLocation;

				FConcertClientDesktopPresenceUpdateEvent Event;
				Event.bMovingCamera = false;
				FViewportCursorLocation CursorWorldLocation = ActiveViewportClient->GetCursorWorldLocationFromMousePos();
				FVector LineCheckStart = CursorWorldLocation.GetOrigin();
				FVector LineCheckEnd = CursorWorldLocation.GetOrigin() + CursorWorldLocation.GetDirection() * HALF_WORLD_MAX;

				Event.TraceStart = LineCheckStart;
				Event.TraceEnd = LineCheckEnd;

				// If not tracking figure out what is being hovered over in 3d space
				if (ActiveViewportClient->IsMovingCamera())
				{
					Event.TraceEnd = GEditor->ClickLocation;
					Event.bMovingCamera = true;
				}
				else if (!ActiveViewportClient->IsTracking())
				{
					FVector HitProxyTrace = LineCheckEnd;
					FVector LineTrace = LineCheckEnd;
					HHitProxy* Proxy = ActiveViewportClient->Viewport->GetHitProxy(CursorWorldLocation.GetCursorPos().X, CursorWorldLocation.GetCursorPos().Y);
					HActor* ActorProxy = HitProxyCast<HActor>(Proxy);
					if (ActorProxy && ActorProxy->Actor)
					{
						FVector Normal = (LineCheckEnd - LineCheckStart).GetSafeNormal();
						// stop at the intersection of the proxy along the line check vector;
						FVector ActorLocation = ActorProxy->Actor->GetActorLocation();
						FVector IntersectPoint = FMath::LinePlaneIntersection(LineCheckStart, LineCheckEnd, ActorLocation, Normal);
						HitProxyTrace = IntersectPoint;
					}

					{
						FHitResult Result;
						FCollisionQueryParams TraceParams(SCENE_QUERY_STAT(DestopPresenceCursorTrace), true);
						bool bSuccess = ActiveViewportClient->GetWorld()->LineTraceSingleByChannel(Result, LineCheckStart, LineCheckEnd, ECC_Visibility, TraceParams);
						if (bSuccess)
						{
							LineTrace = Result.ImpactPoint;
						}


						// chose closest point to origin.  Always favor the trace if the proxy and the trace reached the same actor since the trace will be more accurate
						if ((ActorProxy && Result.GetHitObjectHandle() == ActorProxy->Actor) || FVector::DistSquared(LineCheckStart, LineTrace) < FVector::DistSquared(LineCheckStart, HitProxyTrace))
						{
							Event.TraceEnd = LineTrace;
						}
						else
						{
							// @todo - HitProxyTrace is usually not near LineTrace which causes the laser to jump when the camera stops moving.
							// Event.TraceEnd = HitProxyTrace;
							Event.TraceEnd = LineTrace;
						}
					}
				}
				else
				{
					// Use the world position of the tracker if tracking
					Event.TraceEnd = ActiveViewportClient->GetModeTools()->PivotLocation;
				}

				SetUpdateIndex(Session, FConcertClientDesktopPresenceUpdateEvent::StaticStruct()->GetFName(), Event);

				Session.SendCustomEvent(Event, Session.GetSessionClientEndpointIds(), EConcertMessageFlags::None);
			}
		}
	}
}

void FConcertClientVRPresenceMode::SendEvents(IConcertClientSession& Session)
{
	check(ParentManager);

	FConcertClientBasePresenceMode::SendEvents(Session);

	UWorld* World = ParentManager->GetWorld();
	if (!World)
	{
		return;
	}

	// Send motion controller data if in PIE and VR editor
	if (ParentManager->IsInPIE() || IVREditorModule::Get().GetVRModeBase())
	{
		const FTransform PresenceRoomTransform = GetRoomTransform();

		// Get the player index to query
		int32 PlayerIndex = 0;
		if (ParentManager->IsInPIE())
		{
			check(GEditor->PlayWorld);
			if (ULocalPlayer* LocalPlayer = GEditor->PlayWorld->GetFirstLocalPlayerFromController())
			{
				PlayerIndex = LocalPlayer->GetControllerId();
			}
		}

		// Get the common motion controller data
		TArray<IMotionController*> MotionControllers = IModularFeatures::Get().GetModularFeatureImplementations<IMotionController>(IMotionController::GetModularFeatureName());
		auto GetControllerTransform = [PlayerIndex, World, PresenceRoomTransform, &MotionControllers](const FName MotionSource) -> FTransform
		{
			FTransform MotionControllerToWorld = FTransform::Identity;
			for (IMotionController* MotionController : MotionControllers)
			{
				FRotator MotionControllerOrientation;
				FVector MotionControllerPosition;
				if (MotionController && MotionController->GetControllerOrientationAndPosition(PlayerIndex, MotionSource, MotionControllerOrientation, MotionControllerPosition, World->GetWorldSettings()->WorldToMeters))
				{
					MotionControllerToWorld = FTransform(MotionControllerOrientation, MotionControllerPosition) * PresenceRoomTransform;
					break;
				}
			}
			return MotionControllerToWorld;
		};

		FTransform LeftMotionControllerTransform = GetControllerTransform(IMotionController::LeftHandSourceId);
		FTransform RightMotionControllerTransform = GetControllerTransform(IMotionController::RightHandSourceId);
		
		FConcertClientVRPresenceUpdateEvent Event;
		Event.LeftMotionControllerOrientation = LeftMotionControllerTransform.GetRotation();
		Event.LeftMotionControllerPosition = LeftMotionControllerTransform.GetLocation();
		Event.RightMotionControllerOrientation = RightMotionControllerTransform.GetRotation();
		Event.RightMotionControllerPosition = RightMotionControllerTransform.GetLocation();

		// Grab the laser position from VR editor too
		if (UVREditorModeBase* VRMode = IVREditorModule::Get().GetVRModeBase())
		{
			for (int32 HandIndex = 0; HandIndex < 2; ++HandIndex)
			{
				const EControllerHand Hand = (HandIndex == 0)
					? EControllerHand::Left
					: EControllerHand::Right;

				FVector LaserStart, LaserEnd;
				if (VRMode->GetLaserForHand(Hand, LaserStart, LaserEnd))
				{
					Event.Lasers[HandIndex] = FConcertLaserData(LaserStart, LaserEnd);
				}
			}
		}

		SetUpdateIndex(Session, FConcertClientDesktopPresenceUpdateEvent::StaticStruct()->GetFName(), Event);

		Session.SendCustomEvent(Event, Session.GetSessionClientEndpointIds(), EConcertMessageFlags::None);
	}
}

FTransform FConcertClientVRPresenceMode::GetRoomTransform()
{
	check(ParentManager);

	FTransform NewRoomTransform = FTransform::Identity;

	if (ParentManager->IsInPIE())
	{
		check(GEditor->PlayWorld);

		// In PIE, take the transform of the active Pawn
		if (APlayerController* PC = GEditor->PlayWorld->GetFirstPlayerController())
		{
			NewRoomTransform = FTransform(PC->GetPawn()->GetActorRotation(), PC->GetPawn()->GetActorLocation());
		}
	}
	else if (UVREditorModeBase* VRMode = IVREditorModule::Get().GetVRModeBase())
	{
		NewRoomTransform = VRMode->GetRoomTransform();
	}
	else if (FLevelEditorViewportClient* PerspectiveViewport = ParentManager->GetPerspectiveViewport())
	{
		// Take the position from the active viewport
		NewRoomTransform = FTransform(PerspectiveViewport->GetViewRotation(), PerspectiveViewport->GetViewLocation());
	}
	else
	{
		NewRoomTransform = LastRoomTransform;
	}

	LastRoomTransform = NewRoomTransform;
	return LastRoomTransform;
}

#endif // WITH_EDITOR
