// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IConcertClientPresenceMode.h"
#include "IConcertSessionHandler.h"
#include "ConcertPresenceEvents.h"
#include "ConcertMessages.h"
#include "ConcertClientPresenceActor.h"
#include "UObject/Class.h"
#include "UObject/StrongObjectPtr.h"

#if WITH_EDITOR

class IConcertClientSession;

/**
 * Implementation of the Base Presence Mode
 */
class FConcertClientBasePresenceMode : public IConcertClientBasePresenceMode
{
public:
	explicit FConcertClientBasePresenceMode(class FConcertClientPresenceManager* InManager, FName InVRDeviceType)
		: LastHeadTransform(FTransform::Identity)
		, ParentManager(InManager)
		, VRDeviceType(InVRDeviceType)
	{}
	virtual ~FConcertClientBasePresenceMode() = default;

	// IConcertClientBasePresenceMode implementation
	virtual void SendEvents(IConcertClientSession& Session) override;
	virtual FTransform GetTransform() override;	
	virtual FName GetVRDeviceType() const override
	{
		return VRDeviceType;
	}
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override {}

protected:
	/** Set event update index on an event, used for out-of-order event handling */
	void SetUpdateIndex(IConcertClientSession& Session, const FName& InEventName, FConcertClientPresenceEventBase& OutEvent);

	class FConcertClientPresenceManager* GetManager() const
	{
		return ParentManager;
	}

	/** Last head transform returned. */
	FTransform LastHeadTransform;

	/** Parent manager */
	class FConcertClientPresenceManager* ParentManager;

	/** Holds the vr device type name, if any. */
	FName VRDeviceType;
};

class FConcertClientDesktopPresenceMode : public FConcertClientBasePresenceMode
{
public:
	explicit FConcertClientDesktopPresenceMode(class FConcertClientPresenceManager* InManager)
		: FConcertClientBasePresenceMode(InManager, NAME_None) {}
	virtual ~FConcertClientDesktopPresenceMode() = default;

	/** Send events for this presence mode */
	virtual void SendEvents(IConcertClientSession& Session) override;

protected:
	/** Cached desktop cursor location to avoid resending changes when mouse did not move */
	FIntPoint CachedDesktopCursorLocation;
};

class FConcertClientVRPresenceMode : public FConcertClientBasePresenceMode
{
public:
	explicit FConcertClientVRPresenceMode(class FConcertClientPresenceManager* InManager, FName InVRDeviceType)
		: FConcertClientBasePresenceMode(InManager, MoveTemp(InVRDeviceType))
		, LastRoomTransform(FTransform::Identity) {}
	virtual ~FConcertClientVRPresenceMode() = default;

	/** Send events for this presence mode */
	virtual void SendEvents(IConcertClientSession& Session) override;

protected:
	/** Get the current room transformation */
	FTransform GetRoomTransform();

	/** Last room transform returned. */
	FTransform LastRoomTransform;
};



#endif
