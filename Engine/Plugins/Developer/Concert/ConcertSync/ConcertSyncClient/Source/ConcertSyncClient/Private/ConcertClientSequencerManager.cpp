// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertClientSequencerManager.h"
#include "ConcertSequencerMessages.h"
#include "ConcertTransportMessages.h"
#include "Delegates/IDelegateInstance.h"
#include "HAL/IConsoleManager.h"
#include "LevelSequence.h"
#include "Misc/AssertionMacros.h"
#include "Misc/QualifiedFrameTime.h"
#include "Modules/ModuleManager.h"
#include "Logging/LogMacros.h"
#include "Logging/StructuredLog.h"

#include "IConcertClient.h"
#include "IConcertSyncClient.h"
#include "IConcertSyncClientModule.h"

#include "IConcertSession.h"
#include "ConcertSettings.h"
#include "ConcertClientSettings.h"
#include "ConcertSyncArchives.h"
#include "ConcertClientWorkspace.h"
#include "ConcertTransactionEvents.h"

#include "Engine/GameEngine.h"
#include "MovieSceneSequence.h"
#include "MovieSceneTimeHelpers.h"
#include "LevelSequencePlayer.h"
#include "LevelSequenceActor.h"

#include "UObject/PackageReload.h"
#include "UObject/UObjectGlobals.h"

#if WITH_EDITOR
	#include "AnimatedRange.h"
	#include "ISequencerModule.h"
	#include "ISequencer.h"
	#include "SequencerSettings.h"
	#include "Subsystems/AssetEditorSubsystem.h"
	#include "Editor.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(LogConcertSequencerSync, Warning, All)

#if WITH_EDITOR

// Allow transactions on all edits to level sequences.
static TAutoConsoleVariable<bool> CVarAllowAllTransactionsOnLevelSequences(TEXT("Concert.AllowAllTransactionsOnLevelSequences"), true, TEXT("Allow all transactions to pass the transaction filter when those transactions have an outermost object that is a level sequence."));

// Enable Sequence Playback Syncing
static TAutoConsoleVariable<int32> CVarEnablePlaybackSync(TEXT("Concert.EnableSequencerPlaybackSync"), 1, TEXT("Enable Concert Sequencer Playback Syncing of opened Sequencer."));

// Enable Sequence Playing on game client
static TAutoConsoleVariable<int32> CVarEnableSequencePlayer(TEXT("Concert.EnableSequencePlayer"), 1, TEXT("Enable Concert Sequence Players on `-game` client."));

static TAutoConsoleVariable<int32> CVarEnableLoopingOnPlayer(TEXT("Concert.EnableLoopingOnPlayer"), 1, TEXT("Enable Looping Sequence Players when sequencer looping is enabled."));

// Enable opening Sequencer on remote machine whenever a Sequencer is opened, if both instances have this option on.
static TAutoConsoleVariable<int32> CVarEnableRemoteSequencerOpen(TEXT("Concert.EnableOpenRemoteSequencer"), 1, TEXT("Enable Concert remote Sequencer opening."));

// Enable closing Sequencer for this user when a remote user closes the sequence.
static TAutoConsoleVariable<int32> CVarEnableRemoteSequencerClose(TEXT("Concert.EnableCloseRemoteSequencer"), 0, TEXT("Enable Concert remote Sequencer closing."));

// Enable synchronizing the timeline of unrelated Sequencers on remote machine whenever a Sequencer state event is received, if both instances have this option on.
static TAutoConsoleVariable<int32> CVarEnableUnrelatedTimelineSync(TEXT("Concert.EnableUnrelatedTimelineSync"), 0, TEXT("Enable syncing unrelated sequencer timeline."));

// Enable the sending of undefined message types to the connected endpoints.
// This message is only used to indicate timeline state when not in playback
// mode. This is an experimental cvar.
static TAutoConsoleVariable<int32> CVarEnableUndefinedSequencerMessages(TEXT("Concert.EnableUndefinedSequencerMessages"), 1, TEXT("Enable undefined sequencer messages to be sent."));

// Enable the sending of messages as unreliable.  Messages of this type may get skipped if the network is congested.
// Sequencer state events can be skipped as the sequencer will send regular ticks that would issue new state events.
static TAutoConsoleVariable<int32> CVarSendStateEventsAsUnreliable(TEXT("Concert.SendSequencerStateEventsAsUnreliable"), 0, TEXT("Enable sending sequencer events in unreliable mode. Note this "));

// Enable always closing player on remote machine whenever a sequencer is closed on an editor.
static TAutoConsoleVariable<int32> CVarAlwaysCloseGamePlayerOnCloseEvent(
	TEXT("Concert.AlwaysCloseGamePlayerOnCloseEvent"), 1, TEXT("Force this player to close even if other editors have it open. This CVar only works on `-game` instances."));

static TAutoConsoleVariable<float> CVarSequencerStatePacingDuration(
	TEXT("Concert.SequencerStatePacingDuration"), 0.20f, TEXT("Duration we use (in seconds) to pace sequencer state events to clients."));

static TAutoConsoleVariable<bool> CVarSequencerStatePacingEnabled(
	TEXT("Concert.SequencerStatePacingEnabled"), 1, TEXT("Use sequencer state pacing to prevent too many messages sent to the server."));


class FConcertClientSequencerStateEventPacer
{
public:
	bool ShouldIncludeMessageForSend(FConcertClientSequencerManager::EPlaybackMode Mode)
	{
		if (CVarEnableUndefinedSequencerMessages.GetValueOnAnyThread() == 0 && Mode == FConcertClientSequencerManager::EPlaybackMode::Undefined)
		{
			return false;
		}
		return true;
	}

	void Reset()
	{
		bIsFirstEvent = true;
		CurrentStateEvent.Reset();
	}

	void AddStateEvent(TSharedPtr<IConcertClientSession>& Session, FConcertSequencerStateEvent InEvent, FConcertClientSequencerManager::EPlaybackMode Mode)
	{
		if (!ShouldIncludeMessageForSend(Mode))
		{
			return;
		}

		bool bShouldForceSend = bIsFirstEvent || !IsPacingEnabled();
		if (!bShouldForceSend && InEvent.State.bLoopMode && CurrentStateEvent)
		{
			// If we have loop enabled and we have started back over again then we should force a send.
			bShouldForceSend = InEvent.State.Time.AsSeconds() < CurrentStateEvent->State.Time.AsSeconds();
		}
		CheckForContinuationOfPlay(InEvent);
		CurrentStateEvent = MoveTemp(InEvent);
		Tick(Session, bShouldForceSend);
	}

	void CheckForContinuationOfPlay(const FConcertSequencerStateEvent& InEvent)
	{
		if (CurrentStateEvent)
		{
			const FConcertSequencerState& Current = CurrentStateEvent->State;
			const FConcertSequencerState& Updated = InEvent.State;
			bIsAContinuationOfPlay = Current.PlayerStatus == EConcertMovieScenePlayerStatus::Playing
				&& Updated.PlayerStatus == EConcertMovieScenePlayerStatus::Playing
				&& Updated.PlaybackRange == Current.PlaybackRange
				&& Updated.PlaybackSpeed == Current.PlaybackSpeed
				&& Updated.SequenceObjectPath == Current.SequenceObjectPath;
		}
	}

	void Tick(TSharedPtr<IConcertClientSession>& Session, bool bForceSend = false)
	{
		if (CurrentStateEvent)
		{
			double CurrentTime = FPlatformTime::Seconds();
			const double DeltaTime = CurrentTime - LastSendTime;
			if (bForceSend || DeltaTime > CVarSequencerStatePacingDuration.GetValueOnGameThread())
			{
				bool bShouldSendReliably = bForceSend && IsPacingEnabled();
				EConcertMessageFlags Flags = bShouldSendReliably || CVarSendStateEventsAsUnreliable.GetValueOnAnyThread() == 0 ? EConcertMessageFlags::ReliableOrdered : EConcertMessageFlags::None;
				if (bForceSend || !bIsAContinuationOfPlay)
				{
					Session->SendCustomEvent(*CurrentStateEvent, Session->GetSessionServerEndpointId(), Flags);
					LastSendTime = CurrentTime;
					CurrentStateEvent.Reset();
					bIsFirstEvent = false;
					bIsAContinuationOfPlay = false;
				}
			}
		}
	}

	bool IsPacingEnabled() const
	{
		return !bIsPacingSuspended && CVarSequencerStatePacingEnabled.GetValueOnAnyThread();
	}

	static void SetPacingSuspended(bool bInPacingSuspended)
	{
		bIsPacingSuspended = bInPacingSuspended;
	}

private:
	TOptional<FConcertSequencerStateEvent> CurrentStateEvent;
	double LastSendTime = 0;
	bool bIsFirstEvent = true;
	bool bIsAContinuationOfPlay = false;
	static bool bIsPacingSuspended;
};

bool FConcertClientSequencerStateEventPacer::bIsPacingSuspended = false;

class FConcertClientSequencePreloader : public TSharedFromThis<FConcertClientSequencePreloader>
{
public:
	void OnPreloadEvent(const FConcertSessionContext& InEventContext, const FConcertSequencerPreloadRequest& InEvent);

	void OnRegister(TSharedRef<IConcertClientSession> InSession)
	{
		WeakSession = InSession;

		InSession->RegisterCustomEventHandler<FConcertSequencerPreloadRequest>(this, &FConcertClientSequencePreloader::OnPreloadEvent);
	}

	void OnUnregister(TSharedRef<IConcertClientSession> InSession)
	{
		PreloadPendingSequences.Empty();
		PreloadedSequences.Empty();

		// Unregister our events and explicitly reset the session ptr
		if (TSharedPtr<IConcertClientSession> Session = WeakSession.Pin())
		{
			check(Session == InSession);
			Session->UnregisterCustomEventHandler<FConcertSequencerPreloadRequest>(this);
		}

		WeakSession.Reset();
	}

	void AddReferencedObjects(FReferenceCollector& Collector)
	{
		Collector.AddReferencedObjects(PreloadedSequences);
	}

protected:
	TWeakPtr<IConcertClientSession> WeakSession;

	/** Map of sequence assets for which preload has been requested but not completed. */
	TMap<FTopLevelAssetPath, TSoftObjectPtr<ULevelSequence>> PreloadPendingSequences;

	/** Map of preloaded sequence assets. */
	TMap<FTopLevelAssetPath, TObjectPtr<ULevelSequence>> PreloadedSequences;
};


FConcertClientSequencerManager::FConcertClientSequencerManager(IConcertSyncClient* InOwnerSyncClient)
	: OwnerSyncClient(InOwnerSyncClient)
	, Preloader(MakeShared<FConcertClientSequencePreloader>())
{
	check(OwnerSyncClient);

	bRespondingToTransportEvent = false;

	if (GIsEditor)
	{
		ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>("Sequencer");
		OnSequencerCreatedHandle = SequencerModule.RegisterOnSequencerCreated(
			FOnSequencerCreated::FDelegate::CreateRaw(this, &FConcertClientSequencerManager::OnSequencerCreated)
		);
	}

	FCoreUObjectDelegates::OnPackageReloaded.AddRaw(this, &FConcertClientSequencerManager::HandleAssetReload);
}

namespace UE::Private::ConcertClientSequencerManager
{
const FString PendingTakePath = TEXT("/Temp/TakeRecorder/PendingTake.PendingTake:PendingTake");

bool IsPendingTakePath(const FString& InSequencePath)
{
	return InSequencePath == PendingTakePath;
}
}

FConcertClientSequencerManager::~FConcertClientSequencerManager()
{
	if (GIsEditor)
	{
		ISequencerModule* SequencerModulePtr = FModuleManager::Get().GetModulePtr<ISequencerModule>("Sequencer");
		if (SequencerModulePtr)
		{
			SequencerModulePtr->UnregisterOnSequencerCreated(OnSequencerCreatedHandle);
		}
	}

	SetActiveWorkspace(nullptr);

	for (FOpenSequencerData& OpenSequencer : OpenSequencers)
	{
		TSharedPtr<ISequencer> Sequencer = OpenSequencer.WeakSequencer.Pin();
		if (Sequencer.IsValid())
		{
			Sequencer->OnGlobalTimeChanged().Remove(OpenSequencer.OnGlobalTimeChangedHandle);
			Sequencer->OnCloseEvent().Remove(OpenSequencer.OnCloseEventHandle);
		}
	}
	FCoreUObjectDelegates::OnPackageReloaded.RemoveAll(this);
}

void FConcertClientSequencerManager::OnSequencerCreated(TSharedRef<ISequencer> InSequencer)
{
	// Find a Sequencer state for a newly opened sequencer if we have one.
	UMovieSceneSequence* Sequence = InSequencer->GetRootMovieSceneSequence();
	check(Sequence != nullptr);

	const FString SequenceObjectPath = Sequence->GetPathName();

	UE_LOGF(LogConcertSequencerSync, Verbose, "OnSequencerCreated: %ls", *SequenceObjectPath);

	if (!SequencerStates.Contains(*SequenceObjectPath))
	{
		FConcertSequencerState NewState;
		NewState.Time = InSequencer->GetGlobalTime();
		SequencerStates.Add(*SequenceObjectPath, NewState);
	}

	FConcertSequencerState& SequencerState = SequencerStates.FindOrAdd(*SequenceObjectPath);

	// Setup the Sequencer
	FOpenSequencerData OpenSequencer;
	OpenSequencer.WeakSequencer = TWeakPtr<ISequencer>(InSequencer);
	OpenSequencer.PlaybackMode = EPlaybackMode::Undefined;
	OpenSequencer.OnGlobalTimeChangedHandle = InSequencer->OnGlobalTimeChanged().AddRaw(this, &FConcertClientSequencerManager::OnSequencerTimeChanged, OpenSequencer.WeakSequencer);
	OpenSequencer.OnCloseEventHandle = InSequencer->OnCloseEvent().AddRaw(this, &FConcertClientSequencerManager::OnSequencerClosed);
	OpenSequencer.StateEventPacer = MakePimpl<FConcertClientSequencerStateEventPacer>();
	int32 OpenIndex = OpenSequencers.Add(MoveTemp(OpenSequencer));

	// Setup stored state.  Since this is an open event by the sequencer we might be the controller and should not set player
	// state from previous stored state.  Hence playback mode is set to undefined.  We should wait for a global time event from
	// other clients before we set our playback state.
	//
	// Setting the global time will notify the server of this newly opened state.
	InSequencer->SetGlobalTime(SequencerState.Time.ConvertTo(InSequencer->GetRootTickResolution()));
	// Since setting the global time will potentially have set our playback mode put us back to undefined
	OpenSequencers[OpenIndex].PlaybackMode = EPlaybackMode::Undefined;

	// if we allow for Sequencer remote opening send an event, if we aren't currently responding to one
	if (!bRespondingToTransportEvent && IsSequencerRemoteOpenEnabled())
	{
		if (TSharedPtr<IConcertClientSession> Session = WeakSession.Pin())
		{
			if (CanSendSequencerEvent(SequenceObjectPath))
			{
				FConcertSequencerOpenEvent OpenEvent;
				OpenEvent.SequenceObjectPath = SequenceObjectPath;

				if (UE::Private::ConcertClientSequencerManager::IsPendingTakePath(SequenceObjectPath))
				{
					// The pending take may have a level sequence loaded into it.  So we have to capture it with the object writer
					// and transmit in our open event message.
					//
					ULevelSequence* LevelSequence = Cast<ULevelSequence>(Sequence);

					FConcertSyncRemapObjectPath RemapperDelegate;
					RemapperDelegate.BindLambda([](FString& InPath)
					{
						if (UE::Private::ConcertClientSequencerManager::IsPendingTakePath(InPath))
						{
							InPath = TEXT("/Engine/Transient.__PendingLevelSequence__");
						}
					});
					FConcertSyncObjectWriter SyncObjectWriter(nullptr, LevelSequence, OpenEvent.TakeData.Bytes, true, false, RemapperDelegate);
					SyncObjectWriter.SetSerializeNestedObjects(true);
					SyncObjectWriter.SerializeObject(LevelSequence);
				}
				UE_LOGF(LogConcertSequencerSync, Verbose, "    Sending OpenEvent: %ls", *OpenEvent.SequenceObjectPath);
				Session->SendCustomEvent(OpenEvent, Session->GetSessionServerEndpointId(), EConcertMessageFlags::ReliableOrdered);
			}
		}
	}
}

TArray<FConcertClientSequencerManager::FOpenSequencerData*, TInlineAllocator<1>>
FConcertClientSequencerManager::GatherRootSequencersByState(const FString& InSequenceObjectPath)
{
	TArray<FOpenSequencerData*, TInlineAllocator<1>> OutSequencers;
	for (FOpenSequencerData& Entry : OpenSequencers)
	{
		TSharedPtr<ISequencer> Sequencer = Entry.WeakSequencer.Pin();
		UMovieSceneSequence*   Sequence = Sequencer.IsValid() ? Sequencer->GetRootMovieSceneSequence() : nullptr;

		if (Sequence && (Sequence->GetPathName() == InSequenceObjectPath || IsUnrelatedSequencerTimelineSyncEnabled()))
		{
			OutSequencers.Add(&Entry);
		}
	}
	return OutSequencers;
}

void FConcertClientSequencerManager::SetActiveWorkspace(TSharedPtr<FConcertClientWorkspace> InWorkspace)
{
	if (TSharedPtr<FConcertClientWorkspace> SharedWorkspace = Workspace.Pin())
	{
		SharedWorkspace->OnWorkspaceEndFrameCompleted().RemoveAll(this);
	}

	Workspace = InWorkspace;

	if (InWorkspace.IsValid())
	{
		InWorkspace->OnWorkspaceEndFrameCompleted().AddRaw(this, &FConcertClientSequencerManager::OnWorkspaceEndFrameCompleted);
	}
}

float FConcertClientSequencerManager::GetLatencyCompensationMs() const
{
	IConcertClientRef ConcertClient = OwnerSyncClient->GetConcertClient();
	return ConcertClient->IsConfigured()
		? ConcertClient->GetConfiguration()->ClientSettings.LatencyCompensationMs
		: 0.0f;
}

ETransactionFilterResult FConcertClientSequencerManager::ShouldObjectBeTransacted(const FConcertTransactionFilterArgs& FilterArgs)
{
	const bool bShouldCheckOutermost = CVarAllowAllTransactionsOnLevelSequences.GetValueOnAnyThread();
	if (bShouldCheckOutermost)
	{
		UObject* Outermost = IsValid(FilterArgs.ObjectToFilter) ? FilterArgs.ObjectToFilter->GetOutermostObject() : nullptr;
		if (WeakSession.IsValid() && Outermost && Outermost->IsA<ULevelSequence>() && !Outermost->HasAnyFlags(RF_Transient))
		{
			return ETransactionFilterResult::IncludeObject;
		}
	}

	return ETransactionFilterResult::UseDefault;
}

void FConcertClientSequencerManager::Register(TSharedRef<IConcertClientSession> InSession)
{
	// Hold onto the session so we can trigger events
	WeakSession = InSession;

	// Register our events
	InSession->RegisterCustomEventHandler<FConcertSequencerStateEvent>(this, &FConcertClientSequencerManager::OnTransportEvent);
	InSession->RegisterCustomEventHandler<FConcertSequencerCloseEvent>(this, &FConcertClientSequencerManager::OnCloseEvent);
	InSession->RegisterCustomEventHandler<FConcertSequencerOpenEvent>(this, &FConcertClientSequencerManager::OnOpenEvent);
	InSession->RegisterCustomEventHandler<FConcertSequencerStateSyncEvent>(this, &FConcertClientSequencerManager::OnStateSyncEvent);
	InSession->RegisterCustomEventHandler<FConcertSequencerTimeAdjustmentEvent>(this, &FConcertClientSequencerManager::OnTimeAdjustmentEvent);

	if (GIsEditor)
	{
		if (IConcertClientTransactionBridge* TransactionBridge = OwnerSyncClient->GetTransactionBridge())
		{
			TransactionBridge->RegisterTransactionFilter(TEXT("ConcertSequencerLS"), FOnFilterTransactionDelegate::CreateRaw(this, &FConcertClientSequencerManager::ShouldObjectBeTransacted));				
		}
	}

	Preloader->OnRegister(InSession);
}

void FConcertClientSequencerManager::Unregister(TSharedRef<IConcertClientSession> InSession)
{
	// Unregister our events and explicitly reset the session ptr
	if (TSharedPtr<IConcertClientSession> Session = WeakSession.Pin())
	{
		check(Session == InSession);

		Session->UnregisterCustomEventHandler<FConcertSequencerStateEvent>(this);
		Session->UnregisterCustomEventHandler<FConcertSequencerCloseEvent>(this);
		Session->UnregisterCustomEventHandler<FConcertSequencerOpenEvent>(this);
		Session->UnregisterCustomEventHandler<FConcertSequencerStateSyncEvent>(this);
		Session->UnregisterCustomEventHandler<FConcertSequencerTimeAdjustmentEvent>(this);
		Session->UnregisterCustomEventHandler<FConcertSequencerPreloadRequest>(this);

		if (GEditor)
		{
			if (IConcertClientTransactionBridge* TransactionBridge = OwnerSyncClient->GetTransactionBridge())
			{
				TransactionBridge->UnregisterTransactionFilter(TEXT("ConcertSequencerLS"));
			}
			
			TArray<UMovieSceneSequence*> Sequences;
			for (FOpenSequencerData& OpenSequencer : OpenSequencers)
			{
				TSharedPtr<ISequencer> Sequencer = OpenSequencer.WeakSequencer.Pin();
				if (Sequencer.IsValid() && Sequencer->GetRootMovieSceneSequence())
				{
					Sequences.Add(Sequencer->GetRootMovieSceneSequence());
				}
			}

			for (UMovieSceneSequence* MovieSequence : Sequences)
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllEditorsForAsset(MovieSequence);
			}
		}
	}

	for (TTuple<FName, TObjectPtr<ALevelSequenceActor>>& Item : SequencePlayers)
	{
		ALevelSequenceActor* LevelSequenceActor = Item.Value;
		if (LevelSequenceActor)
		{
			DestroyPlayer(LevelSequenceActor);
		}				
	}

	SequencePlayers.Reset();
	WeakSession.Reset();

	Preloader->OnUnregister(InSession);
}

void SetConsoleVariableRespectingPriority(IConsoleVariable* AsVariable, bool bValue)
{
	EConsoleVariableFlags Flags = (EConsoleVariableFlags)( AsVariable->GetFlags() & ECVF_SetByMask );
	AsVariable->Set( bValue ? 1 : 0, Flags );
}

void SetConsoleVariableRespectingPriority(IConsoleVariable* AsVariable, float InValue)
{
	EConsoleVariableFlags Flags = (EConsoleVariableFlags)( AsVariable->GetFlags() & ECVF_SetByMask );
	AsVariable->Set( InValue, Flags );
}

bool FConcertClientSequencerManager::IsSequencerPlaybackSyncEnabled() const
{
	return CVarEnablePlaybackSync.GetValueOnAnyThread() > 0;
}

void FConcertClientSequencerManager::SetSequencerPlaybackSync(bool bEnable)
{
	SetConsoleVariableRespectingPriority(CVarEnablePlaybackSync->AsVariable(), bEnable);
}

bool FConcertClientSequencerManager::IsUnrelatedSequencerTimelineSyncEnabled() const
{
	return CVarEnableUnrelatedTimelineSync.GetValueOnAnyThread() > 0;
}

void FConcertClientSequencerManager::SetUnrelatedSequencerTimelineSync(bool bEnable)
{
	SetConsoleVariableRespectingPriority(CVarEnableUnrelatedTimelineSync->AsVariable(), bEnable);
}

bool FConcertClientSequencerManager::IsSequencerRemoteOpenEnabled() const
{
	return CVarEnableRemoteSequencerOpen.GetValueOnAnyThread() > 0;
}

bool FConcertClientSequencerManager::IsSequencerRemoteCloseEnabled() const
{
	return CVarEnableRemoteSequencerClose.GetValueOnAnyThread() > 0;
}

bool FConcertClientSequencerManager::IsSequencerPacingEnabled() const
{
	return CVarSequencerStatePacingEnabled.GetValueOnAnyThread();
}

float FConcertClientSequencerManager::SequencerPacingDuration() const
{
	return CVarSequencerStatePacingDuration.GetValueOnAnyThread();
}

void FConcertClientSequencerManager::SetSequencerRemoteOpen(bool bEnable)
{
	SetConsoleVariableRespectingPriority(CVarEnableRemoteSequencerOpen->AsVariable(), bEnable);
}

void FConcertClientSequencerManager::SetSequencerRemoteClose(bool bEnable)
{
	SetConsoleVariableRespectingPriority(CVarEnableRemoteSequencerClose->AsVariable(), bEnable);
}

void FConcertClientSequencerManager::SetSequencerPacingEnabled(bool bEnable)
{
	SetConsoleVariableRespectingPriority(CVarSequencerStatePacingEnabled->AsVariable(), bEnable);
}

void FConcertClientSequencerManager::SetSequencerPacingDuration(float Duration)
{
	SetConsoleVariableRespectingPriority(CVarSequencerStatePacingDuration->AsVariable(), Duration);
}

void FConcertClientSequencerManager::SuspendSequencerPacing()
{
	FConcertClientSequencerStateEventPacer::SetPacingSuspended(true);
}

void FConcertClientSequencerManager::ResumeSequencerPacing()
{
	FConcertClientSequencerStateEventPacer::SetPacingSuspended(false);
}

bool FConcertClientSequencerManager::ShouldAlwaysCloseGameSequencerPlayer() const
{
	return CVarAlwaysCloseGamePlayerOnCloseEvent.GetValueOnAnyThread() > 0;
}

namespace UE::Private::ConcertClientSequencerManager
{

void ApplyPlayRangeToPlayer(ULevelSequencePlayer* Player, const FFrameNumberRange& PlayRange)
{
	check(Player);

	const bool bLowerBoundClosed = PlayRange.GetLowerBound().IsClosed();
	const bool bUpperBoundClosed = PlayRange.GetUpperBound().IsClosed();

	if (!ensureMsgf(bLowerBoundClosed, TEXT("PlayRange lower bound is open, which is not supported.")))
	{
		return;
	}

	if (!Player->GetSequence() || !Player->GetSequence()->GetMovieScene())
	{
		UE_LOGF(LogConcertSequencerSync, Warning, "ApplyPlayRangeToPlayer (%ls): Missing sequence or scene",
			*Player->GetSequenceName());
		return;
	}

	// Convert passed-in range from tick resolution to display rate.
	UMovieScene* MovieScene = Player->GetSequence()->GetMovieScene();
	const FFrameRate TickRate = MovieScene->GetTickResolution();
	const FFrameRate PlayerRate = Player->GetFrameRate();

	const FFrameNumber NewStartFrame = ConvertFrameTime(
		UE::MovieScene::DiscreteInclusiveLower(PlayRange), TickRate, PlayerRate).FloorToFrame();

	int32 NewDuration;
	if (bUpperBoundClosed)
	{
		const FFrameNumber NewEndFrame = ConvertFrameTime(
			UE::MovieScene::DiscreteExclusiveUpper(PlayRange), TickRate, PlayerRate).FloorToFrame();

		NewDuration = (NewEndFrame - NewStartFrame).Value;
	}
	else
	{
		NewDuration = TNumericLimits<int32>::Max() - 1;
	}

	const int32 CurrentDuration = Player->GetFrameDuration();
	const FFrameNumber CurrentStartFrame = Player->GetStartTime().Time.GetFrame();
	if (CurrentDuration != NewDuration || CurrentStartFrame != NewStartFrame)
	{
		UE_LOGF(LogConcertSequencerSync, Verbose, "SetFrameRange (%ls): Start %d, duration %d (was %d, %d)",
			*Player->GetSequenceName(), NewStartFrame.Value, NewDuration, CurrentStartFrame.Value, CurrentDuration);

		Player->SetFrameRange(NewStartFrame.Value, NewDuration);
	}
}

bool IsLoopingEnabled(TSharedPtr<ISequencer>& InSequencer)
{
	check(InSequencer);

	if (USequencerSettings* Settings = InSequencer->GetSequencerSettings())
	{
		ESequencerLoopMode LoopMode = Settings->GetLoopMode();
		if (LoopMode == SLM_Loop || LoopMode == SLM_LoopSelectionRange)
		{
			return true;
		}
	}

	return false;
}

FMovieSceneSequencePlaybackSettings GetPlaybackSettings(bool bInLoopMode)
{
	FMovieSceneSequencePlaybackSettings PlaybackSettings;

	// Sequencer behaves differently to Player. Sequencer pauses at the last frame and Player Stops and goes to the
	// first frame unless we set this flag.
	//
	PlaybackSettings.bPauseAtEnd = true;

	if (bInLoopMode && CVarEnableLoopingOnPlayer.GetValueOnAnyThread() > 0)
	{
		// Loop indefinitely
		PlaybackSettings.LoopCount.Value = -1;
	}
	return PlaybackSettings;
}


}

void FConcertClientSequencerManager::OnSequencerClosed(TSharedRef<ISequencer> InSequencer)
{
	const UMovieSceneSequence* Sequence = InSequencer->GetRootMovieSceneSequence();
	const FString SequenceObjectPath = (Sequence != nullptr) ? Sequence->GetPathName() : FString();

	UE_LOGF(LogConcertSequencerSync, Verbose, "OnSequencerClosed: %ls",
		(!SequenceObjectPath.IsEmpty()) ? *SequenceObjectPath : TEXT("<NO SEQUENCE>"));

	// Find the associated open sequencer index
	int Index = 0;
	for (; Index < OpenSequencers.Num(); ++Index)
	{
		if (OpenSequencers[Index].WeakSequencer.HasSameObject(&InSequencer.Get()))
		{
			break;
		}
	}
	// We didn't find the sequencer
	if (Index >= OpenSequencers.Num())
	{
		return;
	}

	FOpenSequencerData& ClosingSequencer = OpenSequencers[Index];

	if (Sequence)
	{
		// Send close event to server and put back playback mode to undefined
		if (TSharedPtr<IConcertClientSession> Session = WeakSession.Pin())
		{
			// Find the associated sequence path name
			if (CanSendSequencerEvent(SequenceObjectPath))
			{
				UE_LOGF(LogConcertSequencerSync, Verbose, "    Sending CloseEvent: %ls", *SequenceObjectPath);

				FConcertSequencerCloseEvent CloseEvent;
				CloseEvent.bControllerClose = ClosingSequencer.PlaybackMode == EPlaybackMode::Controller; // this sequencer had control over the sequence playback
				CloseEvent.SequenceObjectPath = SequenceObjectPath;
				ClosingSequencer.StateEventPacer->Tick(Session, true /*bForceSend*/);
				Session->SendCustomEvent(CloseEvent, Session->GetSessionServerEndpointId(), EConcertMessageFlags::ReliableOrdered);
			}
		}
		else
		{
			UE_LOGF(LogConcertSequencerSync, VeryVerbose, "    Removing Sequencer state for sequence: %ls", *SequenceObjectPath);
			SequencerStates.Remove(*SequenceObjectPath);
		}
	}

	// Removed the closed Sequencer
	OpenSequencers.RemoveAtSwap(Index);
}

void FConcertClientSequencerManager::OnStateSyncEvent(const FConcertSessionContext& InEventContext, const FConcertSequencerStateSyncEvent& InEvent)
{
	UE_LOGF(LogConcertSequencerSync, Verbose, "Event Received - StateSync: Syncing %d Sequencer states", InEvent.SequencerStates.Num());

	for (const auto& State : InEvent.SequencerStates)
	{
		FConcertSequencerState& SequencerState = SequencerStates.FindOrAdd(*State.SequenceObjectPath);
		SequencerState = State;

		const TArray<FOpenSequencerData*, TInlineAllocator<1>> OpenSequencersForObject = GatherRootSequencersByState(SequencerState.SequenceObjectPath);

		// Keep track of whether a Sequencer is open for this particular
		// sequence, since the array of open Sequencers may contain unrelated
		// Sequencers.
		bool bFoundSequencerForObject = false;

		UE_LOGF(LogConcertSequencerSync, VeryVerbose, "    Syncing %d Sequencers for sequence %ls", OpenSequencersForObject.Num(), *SequencerState.SequenceObjectPath);
		for (FOpenSequencerData* OpenSequencer : OpenSequencersForObject)
		{
			TSharedPtr<ISequencer> Sequencer = OpenSequencer->WeakSequencer.Pin();
			if (Sequencer.IsValid() && IsSequencerPlaybackSyncEnabled())
			{
				Sequencer->SetGlobalTime(SequencerState.Time.ConvertTo(Sequencer->GetRootTickResolution()));
				Sequencer->SetPlaybackStatus((EMovieScenePlayerStatus::Type)SequencerState.PlayerStatus);
				Sequencer->SetPlaybackSpeed(SequencerState.PlaybackSpeed);

				const UMovieSceneSequence* Sequence = Sequencer->GetRootMovieSceneSequence();
				if (Sequence && (Sequence->GetPathName() == SequencerState.SequenceObjectPath))
				{
					bFoundSequencerForObject = true;
				}
			}
		}

		if (!bFoundSequencerForObject)
		{
			UE_LOGF(LogConcertSequencerSync, VeryVerbose, "    No existing Sequencer with sequence %ls open. Will open and sync a new one at end of frame.", *SequencerState.SequenceObjectPath);
			FConcertSequencerOpenEvent OpenEvent{SequencerState.SequenceObjectPath, SequencerState.TakeData};
			PendingSequenceOpenEvents.Add(MoveTemp(OpenEvent));
			PendingSequencerEvents.Add(SequencerState);
		}
	}
}

void FConcertClientSequencerManager::OnSequencerTimeChanged(TWeakPtr<ISequencer> InSequencer)
{
	if (bRespondingToTransportEvent)
	{
		return;
	}

	TGuardValue<bool> ReentrancyGuard(bRespondingToTransportEvent, true);

	TSharedPtr<ISequencer> Sequencer = InSequencer.Pin();
	UMovieSceneSequence*   Sequence  = Sequencer.IsValid() ? Sequencer->GetRootMovieSceneSequence() : nullptr;

	TSharedPtr<IConcertClientSession> Session = WeakSession.Pin();
	if (Session.IsValid() && Sequence && IsSequencerPlaybackSyncEnabled())
	{
		if (!CanSendSequencerEvent(Sequence->GetPathName()))
		{
			return;
		}

		// Find the entry that has been updated so we can check/assign its playback mode, or add it in case a Sequencer root sequence was just reassigned
		FConcertSequencerState& SequencerState = SequencerStates.FindOrAdd(*Sequence->GetPathName());

		FOpenSequencerData* OpenSequencer = Algo::FindBy(OpenSequencers, InSequencer, &FOpenSequencerData::WeakSequencer);
		check(OpenSequencer);
		// We only send transport events if we're driving playback (Controller), or nothing is currently playing back to our knowledge (Undefined)
		// @todo: Do we need to handle race conditions and/or contention between sequencers either initiating playback or scrubbing?
		if (OpenSequencer->PlaybackMode == EPlaybackMode::Controller || OpenSequencer->PlaybackMode == EPlaybackMode::Undefined)
		{
			FConcertSequencerStateEvent SequencerStateEvent;
			SequencerStateEvent.State.SequenceObjectPath	= Sequence->GetPathName();
			SequencerStateEvent.State.Time					= Sequencer->GetGlobalTime();
			SequencerStateEvent.State.PlayerStatus			= (EConcertMovieScenePlayerStatus)Sequencer->GetPlaybackStatus();
			SequencerStateEvent.State.PlaybackSpeed			= Sequencer->GetPlaybackSpeed();

			SequencerStateEvent.State.bLoopMode = UE::Private::ConcertClientSequencerManager::IsLoopingEnabled(Sequencer);

			UMovieScene* MovieScene = Sequence->GetMovieScene();
			check(MovieScene);
			SequencerStateEvent.State.PlaybackRange			= MovieScene->GetPlaybackRange();
			SequencerState = SequencerStateEvent.State;

			// Send to client and server
			UE_LOGF(LogConcertSequencerSync, Verbose, "Sending StateEvent: %ls, at frame: %d",
				*SequencerStateEvent.State.SequenceObjectPath, SequencerStateEvent.State.Time.Time.FrameNumber.Value);

			OpenSequencer->StateEventPacer->AddStateEvent(Session, SequencerStateEvent, OpenSequencer->PlaybackMode);

			// If we're playing then ensure we are set to controller (driving the playback on all clients)
			if (SequencerStateEvent.State.PlayerStatus == EConcertMovieScenePlayerStatus::Playing)
			{
				OpenSequencer->PlaybackMode = EPlaybackMode::Controller;
			}
			else
			{
				OpenSequencer->StateEventPacer->Reset();
				OpenSequencer->PlaybackMode = EPlaybackMode::Undefined;
			}
		}
	}
}

void FConcertClientSequencerManager::OnCloseEvent(const FConcertSessionContext&, const FConcertSequencerCloseEvent& InEvent)
{
	UE_LOGF(LogConcertSequencerSync, Verbose, "Event Received - Close: %ls. Deferring until end of frame.", *InEvent.SequenceObjectPath);
	PendingSequenceCloseEvents.Add(InEvent);
}

void FConcertClientSequencerManager::OnOpenEvent(const FConcertSessionContext&, const FConcertSequencerOpenEvent& InEvent)
{
	UE_LOGF(LogConcertSequencerSync, Verbose, "Event Received - Open: %ls. Deferring until end of frame.", *InEvent.SequenceObjectPath);
	PendingSequenceOpenEvents.Add(InEvent);
}

void FConcertClientSequencerManager::HandleAssetReload(const EPackageReloadPhase InPackageReloadPhase, FPackageReloadedEvent* InPackageReloadedEvent)
{
	if (InPackageReloadPhase != EPackageReloadPhase::PostPackageFixup)
	{
		return;
	}

	for (TTuple<FName, TObjectPtr<ALevelSequenceActor>>& Item : SequencePlayers)
	{
		ALevelSequenceActor* LevelSequenceActor = Item.Value;
		// If we have a null LevelSequenceActor it means that it has already been destroyed by a close event.  We will not
		// recreate the asset until all editors have closed it.
		if (LevelSequenceActor)
		{
			const UPackage* PackageReloaded = InPackageReloadedEvent->GetNewPackage();
			ULevelSequence* LevelSequence = LevelSequenceActor->GetSequence();
			if (LevelSequence && LevelSequence->GetPackage() == PackageReloaded)
			{
				FString AssetPathName = LevelSequence->GetPathName();
				UE_LOGF(LogConcertSequencerSync, Display, "Rebuild required for LevelSequence %ls", *Item.Key.ToString());
				TTuple<FName,FString> Pending(Item.Key,AssetPathName);
				PendingDestroy.Emplace(MoveTemp(Pending));
			}
		}
	}
}

void FConcertClientSequencerManager::ApplyCloseEventToPlayers(const FConcertSequencerCloseEvent& CloseEvent)
{
	UE_LOGF(LogConcertSequencerSync, Verbose, "    Closing sequence players: %ls, is from controller: %d", *CloseEvent.SequenceObjectPath, CloseEvent.bControllerClose);

	TObjectPtr<ALevelSequenceActor>* Player = SequencePlayers.Find(*CloseEvent.SequenceObjectPath);
	if (!Player)
	{
		UE_LOGF(LogConcertSequencerSync, Verbose, "        No open players to close for sequence %ls", *CloseEvent.SequenceObjectPath);
		return;
	}

	ALevelSequenceActor* LevelSequenceActor = *Player;

	if (CloseEvent.bControllerClose && LevelSequenceActor && LevelSequenceActor->GetSequencePlayer())
	{
		LevelSequenceActor->GetSequencePlayer()->Stop();
	}

	if (!CanClose(CloseEvent))
	{
		return;
	}

	DestroyPlayer(LevelSequenceActor);

	// Always remove the player on close event. This will allow it to be re-opened on an OpenEvent.
	UE_LOGF(LogConcertSequencerSync, VeryVerbose, "    Removing sequence player for sequence: %ls", *CloseEvent.SequenceObjectPath);
	SequencePlayers.Remove(*CloseEvent.SequenceObjectPath);
}

void FConcertClientSequencerManager::ApplyCloseEvent(const FConcertSequencerCloseEvent& CloseEvent)
{
	UE_LOGF(LogConcertSequencerSync, Verbose, "Handling Event - Close: %ls", *CloseEvent.SequenceObjectPath);

	FConcertSequencerState* SequencerState = SequencerStates.Find(*CloseEvent.SequenceObjectPath);
	if (SequencerState)
	{
		const TArray<FOpenSequencerData*, TInlineAllocator<1>> OpenSequencersForObject = GatherRootSequencersByState(SequencerState->SequenceObjectPath);

		// if the event was that a sequencer that was in controller playback mode was closed, stop playback
		if (CloseEvent.bControllerClose)
		{
			SequencerState->PlayerStatus = EConcertMovieScenePlayerStatus::Stopped;
			for (FOpenSequencerData* OpenSequencer : OpenSequencersForObject)
			{
				OpenSequencer->PlaybackMode = EPlaybackMode::Undefined;
				TSharedPtr<ISequencer> Sequencer = OpenSequencer->WeakSequencer.Pin();
				Sequencer->SetPlaybackStatus(EMovieScenePlayerStatus::Stopped);
			}
		}

		if (GEditor && IsSequencerRemoteCloseEnabled())
		{
			for (FOpenSequencerData* OpenSequencer : OpenSequencersForObject)
			{
				TSharedPtr<ISequencer> Sequencer = OpenSequencer->WeakSequencer.Pin();
				UMovieSceneSequence* Sequence = Sequencer.IsValid() ? Sequencer->GetRootMovieSceneSequence() : nullptr;

				// Verify that this Sequencer is open for the particular
				// sequence being closed since the array of open Sequencers
				// may contain unrelated Sequencers.
				if (Sequence && (Sequence->GetPathName() == CloseEvent.SequenceObjectPath))
				{
					GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllEditorsForAsset(Sequence);
				}
			}
		}

		// Discard the state if it's no longer opened by anyone.
		//
		if (CloseEvent.EditorsWithSequencerOpened == 0)
		{
			UE_LOGF(LogConcertSequencerSync, VeryVerbose, "    Removing Sequencer state for sequence: %ls", *CloseEvent.SequenceObjectPath);
			SequencerStates.Remove(*CloseEvent.SequenceObjectPath);
		}
	}

	ApplyCloseEventToPlayers(CloseEvent);
}

void FConcertClientSequencerManager::ApplyOpenEvent(const FConcertSequencerOpenEvent& InOpenEvent)
{
	TGuardValue<bool> ReentrancyGuard(bRespondingToTransportEvent, true);
	FString SequenceObjectPath = InOpenEvent.SequenceObjectPath;
	UE_LOGF(LogConcertSequencerSync, Verbose, "Handling Event - Open: %ls", *SequenceObjectPath);

	bool bDidOpen = false;
#if WITH_EDITOR
	if (IsSequencerRemoteOpenEnabled() && GIsEditor)
	{
		if (!UE::Private::ConcertClientSequencerManager::IsPendingTakePath(SequenceObjectPath))
		{
			// Don't open the asset until we have loaded in the pending take data.
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(SequenceObjectPath);
		}

		bDidOpen = true;
	}

#endif
	if (!GIsEditor && CVarEnableSequencePlayer.GetValueOnAnyThread() > 0)
	{
		CreateNewSequencePlayerIfNotExists(*SequenceObjectPath);
		bDidOpen = true;
	}

	if (GIsEditor && bDidOpen && UE::Private::ConcertClientSequencerManager::IsPendingTakePath(SequenceObjectPath))
	{
		if (ULevelSequence* PendingLevelSequence = FindObject<ULevelSequence>(nullptr, *UE::Private::ConcertClientSequencerManager::PendingTakePath))
		{
			// Apply pending take data to our version of the pending take.
			//
			FConcertSyncEncounteredMissingObject MissingObjectDelegate;
			MissingObjectDelegate.BindLambda([](const FStringView MissingObject)
				{
					UE_LOGF(LogConcertSequencerSync, Display, "Missing Object %ls when loading PendingTake", *FString(MissingObject));
				});

			if (InOpenEvent.TakeData.Bytes.Num() > 0)
			{
				FConcertSyncWorldRemapper Remapper(
					TEXT("/Engine/Transient.__PendingLevelSequence__"), PendingLevelSequence->GetPathName());
				FConcertSyncObjectReader Reader(nullptr, MoveTemp(Remapper), nullptr, PendingLevelSequence, InOpenEvent.TakeData.Bytes,
												MissingObjectDelegate);
				Reader.SetSerializeNestedObjects(true);
				Reader.SerializeObject(PendingLevelSequence);
			}
			else
			{
				UE_LOGF(LogConcertSequencerSync, Display, "Missing take data on pending take.");
			}

#if WITH_EDITOR
			{
				// We must specify the pending level sequence and not the pending take.  If we specify the pending take after we serialized into the existing
				// asset the open asset will cause take recorder tab to assume that we want to edit the pending take where we really just want to view
				// the level sequence.
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(PendingLevelSequence);
			}
#endif
			PendingLevelSequence->GetPackage()->SetDirtyFlag(false);
		}
	}
}

void FConcertClientSequencerManager::CreateNewSequencePlayerIfNotExists(const FString& SequenceObjectPath)
{
	UE_LOGF(LogConcertSequencerSync, Verbose, "CreateNewSequencePlayerIfNotExists: %ls", *SequenceObjectPath);

	if (SequencePlayers.Contains(*SequenceObjectPath))
	{
		UE_LOGF(LogConcertSequencerSync, Verbose, "    Sequence player already exists for sequence %ls", *SequenceObjectPath);
		return;
	}

	UWorld* CurrentWorld = nullptr;
	if (UGameEngine* GameEngine = Cast<UGameEngine>(GEngine))
	{
		CurrentWorld = GameEngine->GetGameWorld();
	}
	check(CurrentWorld);

	// Get the actual sequence
	ULevelSequence* Sequence = LoadObject<ULevelSequence>(nullptr, *SequenceObjectPath);
	if (!Sequence)
	{
		UE_LOGF(LogConcertSequencerSync, Warning, "    Failed to load level sequence while creating new sequence player: %ls", *SequenceObjectPath);
		return;
	}

	FMovieSceneSequencePlaybackSettings PlaybackSettings = UE::Private::ConcertClientSequencerManager::GetPlaybackSettings(false);

	// This call will initialize LevelSequenceActor as an output parameter.
	ALevelSequenceActor* LevelSequenceActor = nullptr;
	ULevelSequencePlayer* Player = ULevelSequencePlayer::CreateLevelSequencePlayer(CurrentWorld->PersistentLevel, Sequence, PlaybackSettings, LevelSequenceActor);
	check(Player);

	SequencePlayers.Add(*SequenceObjectPath, {LevelSequenceActor});
}

bool FConcertClientSequencerManager::CanClose(const FConcertSequencerCloseEvent& InEvent) const
{
	const bool bShouldClose = ShouldAlwaysCloseGameSequencerPlayer();

	return InEvent.EditorsWithSequencerOpened == 0 || bShouldClose;
}

void FConcertClientSequencerManager::DestroyPlayer(ALevelSequenceActor* LevelSequenceActor)
{
	if (LevelSequenceActor && LevelSequenceActor->GetSequencePlayer())
	{
		UE_LOGF(LogConcertSequencerSync, VeryVerbose, "Destroying LevelSequenceActor: %ls", *LevelSequenceActor->GetPathName());
		LevelSequenceActor->GetSequencePlayer()->Stop();

		LevelSequenceActor->SetSequence(nullptr);
		LevelSequenceActor->Destroy(false, false);
	}
}

void FConcertClientSequencerManager::OnTransportEvent(const FConcertSessionContext&, const FConcertSequencerStateEvent& InEvent)
{
	PendingSequencerEvents.Add(InEvent.State);
}

void FConcertClientSequencerManager::OnTimeAdjustmentEvent(const FConcertSessionContext&, const FConcertSequencerTimeAdjustmentEvent& InEvent)
{
	UE_LOGF(LogConcertSequencerSync, Verbose, "Event Received - TimeAdjustment: %ls. Deferring until end of frame.", *InEvent.SequenceObjectPath);
	PendingTimeAdjustmentEvents.Add(InEvent);
}

void FConcertClientSequencerManager::ApplyTimeAdjustmentEvent(const FConcertSequencerTimeAdjustmentEvent& TimeAdjustmentEvent)
{
	if (bRespondingToTransportEvent)
	{
		return;
	}

	TGuardValue<bool> ReentrancyGuard(bRespondingToTransportEvent, true);

	UE_LOGF(LogConcertSequencerSync, Verbose, "Handling Event - TimeAdjustment: %ls", *TimeAdjustmentEvent.SequenceObjectPath);

	if (GIsEditor)
	{
		ApplyTimeAdjustmentToSequencers(TimeAdjustmentEvent);
	}
	else if (CVarEnableSequencePlayer.GetValueOnAnyThread() > 0)
	{
		ApplyTimeAdjustmentToPlayers(TimeAdjustmentEvent);
	}
}

bool ApplyStartFrameToMovieScene(FFrameNumber StartFrame, UMovieScene* MovieScene)
{
	check(MovieScene);

	FFrameNumber DeltaFrame = StartFrame - MovieScene->GetPlaybackRange().GetLowerBoundValue();
	if (DeltaFrame == 0)
	{
		return false;
	}

	for (UMovieSceneSection* Section : MovieScene->GetAllSections())
	{
		Section->ComputeEffectiveRange();
		Section->MoveSection(DeltaFrame);
	}
	MovieScene->SetPlaybackRange(TRange<FFrameNumber>(StartFrame, TNumericLimits<int32>::Max() - 1), false);

	return true;
}

void FConcertClientSequencerManager::ApplyTimeAdjustmentToSequencers(const FConcertSequencerTimeAdjustmentEvent& TimeAdjustmentEvent)
{
	UE_LOGF(LogConcertSequencerSync, Verbose, "    TimeAdjustment: Updating Sequencers for sequence %ls", *TimeAdjustmentEvent.SequenceObjectPath);

	for (FOpenSequencerData* OpenSequencer : GatherRootSequencersByState(TimeAdjustmentEvent.SequenceObjectPath))
	{
		ISequencer* Sequencer = OpenSequencer->WeakSequencer.Pin().Get();
		// If the entry is driving playback (PlaybackMode == Controller) then we never respond time adjustment events.
		if (!Sequencer || OpenSequencer->PlaybackMode == EPlaybackMode::Controller)
		{
			continue;
		}

		// Adjust the range of the sequencer based on the time provided.
		//
		UMovieScene* MovieScene = Sequencer->GetRootMovieSceneSequence()->GetMovieScene();
		if (MovieScene)
		{
			if (ApplyStartFrameToMovieScene(TimeAdjustmentEvent.PlaybackStartFrame, MovieScene))
			{
				FFrameRate FrameRate = MovieScene->GetTickResolution();
				double CurrentTimeSeconds = FrameRate.AsSeconds(FFrameTime(TimeAdjustmentEvent.PlaybackStartFrame));
				TRange<double> NewRange(CurrentTimeSeconds, CurrentTimeSeconds+10.0);
				Sequencer->SetViewRange(NewRange, EViewRangeInterpolation::Immediate);
				Sequencer->SetClampRange(Sequencer->GetViewRange());
			}
		}

	}
}

void FConcertClientSequencerManager::ApplyTimeAdjustmentToPlayers(const FConcertSequencerTimeAdjustmentEvent& TimeAdjustmentEvent)
{
	UE_LOGF(LogConcertSequencerSync, Verbose, "    TimeAdjustment: Updating sequence players for sequence %ls", *TimeAdjustmentEvent.SequenceObjectPath);

	TObjectPtr<ALevelSequenceActor>* SeqPlayer = SequencePlayers.Find(*TimeAdjustmentEvent.SequenceObjectPath);
	if (!SeqPlayer)
	{
		UE_LOGF(LogConcertSequencerSync, Verbose, "        No sequence player for sequence %ls", *TimeAdjustmentEvent.SequenceObjectPath);
		return;
	}

	ALevelSequenceActor* LevelSequenceActor = *SeqPlayer;
	if (LevelSequenceActor && LevelSequenceActor->GetSequencePlayer())
	{
		UMovieScene* MovieScene = LevelSequenceActor->LevelSequenceAsset->GetMovieScene();
		if (MovieScene)
		{
			ApplyStartFrameToMovieScene(TimeAdjustmentEvent.PlaybackStartFrame, MovieScene);
		}
	}
}

void FConcertClientSequencePreloader::OnPreloadEvent(const FConcertSessionContext& InEventContext, const FConcertSequencerPreloadRequest& InEvent)
{
	for (const FTopLevelAssetPath& SequenceObjectPath : InEvent.SequenceObjectPaths)
	{
		const bool bShouldPreload = InEvent.bShouldBePreloaded;
		UE_LOGFMT(LogConcertSequencerSync, Verbose, "OnPreloadEvent: {Path} should be {Action} the preload set",
			SequenceObjectPath.ToString(), bShouldPreload ? TEXT("added to") : TEXT("removed from"));
		if (bShouldPreload)
		{
			if (TObjectPtr<ULevelSequence>* ExistingSequence = PreloadedSequences.Find(SequenceObjectPath);
				ExistingSequence && *ExistingSequence)
			{
				UE_LOGFMT(LogConcertSequencerSync, Warning, "OnPreloadEvent: {Path} already in preload set", SequenceObjectPath.ToString());
				continue;
			}
			else if (PreloadPendingSequences.Contains(SequenceObjectPath))
			{
				UE_LOGFMT(LogConcertSequencerSync, Warning, "OnPreloadEvent: {Path} already has outstanding async load", SequenceObjectPath.ToString());
				continue;
			}

			const FSoftObjectPath SoftSequenceObjectPath(SequenceObjectPath);
			TSoftObjectPtr<ULevelSequence> SoftSequenceObject(SoftSequenceObjectPath);

			if (ULevelSequence* AlreadyLoadedSequence = SoftSequenceObject.Get())
			{
				UE_LOGFMT(LogConcertSequencerSync, Verbose, "OnPreloadEvent: {Path} was already loaded", SequenceObjectPath.ToString());
				PreloadedSequences.Add(SequenceObjectPath, AlreadyLoadedSequence);

				if (TSharedPtr<IConcertClientSession> Session = WeakSession.Pin(); ensure(Session))
				{
					// Inform the server.
					FConcertSequencerPreloadAssetStatusMap Response;
					Response.Sequences.Add(SequenceObjectPath, EConcertSequencerPreloadStatus::Succeeded);
					Session->SendCustomEvent(Response, Session->GetSessionServerEndpointId(), EConcertMessageFlags::ReliableOrdered);
				}
			}
			else
			{
				UE_LOGFMT(LogConcertSequencerSync, Verbose, "OnPreloadEvent: Initiating async package load for {Path}", SequenceObjectPath.ToString());
				PreloadPendingSequences.Add(SequenceObjectPath, SoftSequenceObject);
				LoadPackageAsync(SoftSequenceObject.GetLongPackageName(), FLoadPackageAsyncDelegate::CreateLambda(
					[WeakThis = AsWeak(),
					WeakRequestSession = WeakSession,
					SequenceObjectPath,
					SoftSequenceObject]
					(const FName& PackageName, UPackage* LoadedPackage, EAsyncLoadingResult::Type Result)
					{
						TSharedPtr<FConcertClientSequencePreloader> This = WeakThis.Pin();
						if (!This)
						{
							UE_LOGFMT(LogConcertSequencerSync, Warning, "Discarding async load result for stale preloader");
							return;
						}

						TSharedPtr<IConcertClientSession> Session = This->WeakSession.Pin();
						if (!Session || Session != WeakRequestSession)
						{
							UE_LOGFMT(LogConcertSequencerSync, Warning, "Discarding async load result issued by mismatched session");
							return;
						}

						if (!This->PreloadPendingSequences.Remove(SequenceObjectPath))
						{
							UE_LOGFMT(LogConcertSequencerSync, Warning, "Discarding async load result for sequence no longer pending");
							return;
						}

						if (Result != EAsyncLoadingResult::Succeeded)
						{
							UE_LOGFMT(LogConcertSequencerSync, Error, "EAsyncLoadingResult != Succeeded for {Package} ({Result})", PackageName, Result);
						}

						ULevelSequence* LoadedSequence = SoftSequenceObject.Get();
						if (!LoadedSequence)
						{
							UE_LOGFMT(LogConcertSequencerSync, Error, "Failed to resolve {Path} after async package load", SequenceObjectPath.ToString());
						}

						// Inform the server of the async load result, and GC ref the loaded sequence if successful.
						FConcertSequencerPreloadAssetStatusMap Response;

						if (Result == EAsyncLoadingResult::Succeeded && LoadedSequence)
						{
							UE_LOGFMT(LogConcertSequencerSync, Verbose, "OnPreloadEvent: Async load completed successfully for {Path}", SequenceObjectPath.ToString());
							This->PreloadedSequences.Add(SequenceObjectPath, LoadedSequence);

							Response.Sequences.Add(SequenceObjectPath, EConcertSequencerPreloadStatus::Succeeded);
						}
						else
						{
							Response.Sequences.Add(SequenceObjectPath, EConcertSequencerPreloadStatus::Failed);
						}

						Session->SendCustomEvent(Response, Session->GetSessionServerEndpointId(), EConcertMessageFlags::ReliableOrdered);
					}
				));
			}
		}
		else
		{
			if (!(PreloadPendingSequences.Remove(SequenceObjectPath)
				|| PreloadedSequences.Remove(SequenceObjectPath)))
			{
				UE_LOGFMT(LogConcertSequencerSync, Warning, "OnPreloadEvent: Tried to remove {Path} not in preload set", *SequenceObjectPath.ToString());
			}
		}
	}
}

void FConcertClientSequencerManager::ApplyTransportEvent(const FConcertSequencerState& EventState)
{
	if (bRespondingToTransportEvent)
	{
		return;
	}

	TGuardValue<bool> ReentrancyGuard(bRespondingToTransportEvent, true);

	UE_LOGF(LogConcertSequencerSync, Verbose, "Handling Event - Transport: %ls", *EventState.SequenceObjectPath);

	// Update the sequencer pointing to the same sequence
	FConcertSequencerState& SequencerState = SequencerStates.FindOrAdd(*EventState.SequenceObjectPath);

	// Record the Sequencer State
	SequencerState = EventState;

	if (GIsEditor)
	{
		ApplyEventToSequencers(SequencerState);
	}
	else if (CVarEnableSequencePlayer.GetValueOnAnyThread() > 0)
	{
		ApplyEventToPlayers(SequencerState);
	}
}

void FConcertClientSequencerManager::ApplyEventToSequencers(const FConcertSequencerState& EventState)
{
	UE_LOGF(LogConcertSequencerSync, Verbose, "    Transport: Update Sequencer for sequence %ls, at frame: %d", *EventState.SequenceObjectPath, EventState.Time.Time.FrameNumber.Value);
	FConcertSequencerState& SequencerState = SequencerStates.FindOrAdd(*EventState.SequenceObjectPath);

	// Record the Sequencer State
	SequencerState = EventState;

	float LatencyCompensationMs = GetLatencyCompensationMs();

	// Update all opened sequencer with this root sequence
	for (FOpenSequencerData* OpenSequencer : GatherRootSequencersByState(EventState.SequenceObjectPath))
	{
		ISequencer* Sequencer = OpenSequencer->WeakSequencer.Pin().Get();
		// If the entry is driving playback (PlaybackMode == Controller) then we never respond to external transport events
		if (!Sequencer || OpenSequencer->PlaybackMode == EPlaybackMode::Controller)
		{
			continue;
		}

		FFrameRate SequenceRate = Sequencer->GetRootTickResolution();
		FFrameTime IncomingTime = EventState.Time.ConvertTo(SequenceRate);

		UMovieScene* MovieScene = Sequencer->GetRootMovieSceneSequence()->GetMovieScene();
		check(MovieScene);
		MovieScene->SetPlaybackRange(EventState.PlaybackRange, false);

		// If the event is coming from a sequencer that is playing back, we are an agent to its updates until it stops
		// We also apply any latency compensation when playing back
		if (EventState.PlayerStatus == EConcertMovieScenePlayerStatus::Playing)
		{
			OpenSequencer->PlaybackMode = EPlaybackMode::Agent;

			FFrameTime CurrentTime = Sequencer->GetGlobalTime().Time;

			// We should be playing back, but are not currently - we compensate the event time for network latency and commence playback
			if (Sequencer->GetPlaybackStatus() != EMovieScenePlayerStatus::Playing)
			{
				// @todo: latency compensation could be more accurate (and automatic) if we're genlocked, and events are timecoded.
				// @todo: latency compensation does not take into account slomo tracks on the sequence - should it? (that would be intricate to support)
				FFrameTime CompensatedTime = IncomingTime + (LatencyCompensationMs / 1000.0) * SequenceRate;

				// Log time metrics
				UE_LOGF(LogConcertSequencerSync, Display, "Starting multi-user playback for sequence '%ls':\n"
					"    Current Time           = %d+%fs (%f seconds)\n"
					"    Incoming Time          = %d+%fs (%f seconds)\n"
					"    Compensated Time       = %d+%fs (%f seconds)",
					*EventState.SequenceObjectPath,
					CurrentTime.FrameNumber.Value, CurrentTime.GetSubFrame(), CurrentTime / SequenceRate,
					IncomingTime.FrameNumber.Value, IncomingTime.GetSubFrame(), IncomingTime / SequenceRate,
					CompensatedTime.FrameNumber.Value, CompensatedTime.GetSubFrame(), CompensatedTime / SequenceRate
				);

				Sequencer->SetGlobalTime(CompensatedTime);
				Sequencer->SetPlaybackStatus(EMovieScenePlayerStatus::Playing);
				Sequencer->SetPlaybackSpeed(EventState.PlaybackSpeed);
			}
			else
			{
				// We're already playing so just report the time metrics, but adjust playback speed
				FFrameTime Error = FMath::Abs(IncomingTime - CurrentTime);
				Sequencer->SetPlaybackSpeed(EventState.PlaybackSpeed);

				UE_LOGF(LogConcertSequencerSync, Display, "Incoming update to sequence '%ls':\n"
					"    Current Time       = %d+%fs (%f seconds)\n"
					"    Incoming Time      = %d+%fs (%f seconds)\n"
					"    Error              = %d+%fs (%f seconds)",
					*EventState.SequenceObjectPath,
					CurrentTime.FrameNumber.Value, CurrentTime.GetSubFrame(), CurrentTime / SequenceRate,
					IncomingTime.FrameNumber.Value, IncomingTime.GetSubFrame(), IncomingTime / SequenceRate,
					Error.FrameNumber.Value, Error.GetSubFrame(), Error / SequenceRate
				);
			}
		}
		else
		{
			OpenSequencer->PlaybackMode = EPlaybackMode::Undefined;

			// If the incoming event is not playing back, set the player status to that of the event, and set the time
			if (Sequencer->GetPlaybackStatus() != (EMovieScenePlayerStatus::Type)EventState.PlayerStatus)
			{
				Sequencer->SetPlaybackStatus((EMovieScenePlayerStatus::Type)EventState.PlayerStatus);
			}

			// Set time after the status so that audio correctly stops playing after the sequence stops
			Sequencer->SetGlobalTime(IncomingTime);
			Sequencer->SetPlaybackSpeed(EventState.PlaybackSpeed);
		}
	}
}

void FConcertClientSequencerManager::ApplyEventToPlayers(const FConcertSequencerState& EventState)
{
	UE_LOGF(LogConcertSequencerSync, Verbose, "    Transport: Update sequence player for sequence %ls, at frame: %d", *EventState.SequenceObjectPath, EventState.Time.Time.FrameNumber.Value);

	TObjectPtr<ALevelSequenceActor>* SeqPlayer = SequencePlayers.Find(*EventState.SequenceObjectPath);
	if (!SeqPlayer)
	{
		UE_LOGF(LogConcertSequencerSync, Verbose, "        No sequence player for sequence %ls", *EventState.SequenceObjectPath);
		return;
	}

	ALevelSequenceActor* LevelSequenceActor = *SeqPlayer;
	if (LevelSequenceActor && LevelSequenceActor->GetSequencePlayer())
	{
		ULevelSequencePlayer* Player = LevelSequenceActor->GetSequencePlayer();
		float LatencyCompensationMs = GetLatencyCompensationMs();

		FFrameRate SequenceRate = Player->GetFrameRate();
		FFrameTime IncomingTime = EventState.Time.ConvertTo(SequenceRate);

		UE::Private::ConcertClientSequencerManager::ApplyPlayRangeToPlayer(Player, EventState.PlaybackRange);

		// If the event is coming from a sequencer that is playing back, we are an agent to its updates until it stops
		// We also apply any latency compensation when playing back
		if (EventState.PlayerStatus == EConcertMovieScenePlayerStatus::Playing)
		{
			FFrameTime CurrentTime = Player->GetCurrentTime().Time;

			FMovieSceneSequencePlaybackSettings PlaybackSettings = UE::Private::ConcertClientSequencerManager::GetPlaybackSettings(EventState.bLoopMode);
			Player->SetPlaybackSettings(PlaybackSettings);
			// We should be playing back, but are not currently - we compensate the event time for network latency and commence playback
			if (!Player->IsPlaying())
			{
				// @todo: latency compensation could be more accurate (and automatic) if we're genlocked, and events are timecoded.
				// @todo: latency compensation does not take into account slomo tracks on the sequence - should it? (that would be intricate to support)
				FFrameTime CompensatedTime = IncomingTime + (LatencyCompensationMs / 1000.0) * SequenceRate;

				// Log time metrics
				UE_LOGF(LogConcertSequencerSync, Display, "Starting multi-user playback for sequence '%ls':\n"
					"    Current Time           = %d+%fs (%f seconds)\n"
					"    Incoming Time          = %d+%fs (%f seconds)\n"
					"    Compensated Time       = %d+%fs (%f seconds)",
					*EventState.SequenceObjectPath,
					CurrentTime.FrameNumber.Value, CurrentTime.GetSubFrame(), CurrentTime / SequenceRate,
					IncomingTime.FrameNumber.Value, IncomingTime.GetSubFrame(), IncomingTime / SequenceRate,
					CompensatedTime.FrameNumber.Value, CompensatedTime.GetSubFrame(), CompensatedTime / SequenceRate
				);

				Player->SetPlaybackPosition(FMovieSceneSequencePlaybackParams(CompensatedTime, EUpdatePositionMethod::Play));
				Player->SetPlayRate(EventState.PlaybackSpeed);
				Player->Play();
			}
			else
			{
				// We're already playing so just report the time metrics, but adjust playback speed
				FFrameTime Error = FMath::Abs(IncomingTime - CurrentTime);
				Player->SetPlayRate(EventState.PlaybackSpeed);

				UE_LOGF(LogConcertSequencerSync, Display, "Incoming update to sequence '%ls':\n"
					"    Current Time       = %d+%fs (%f seconds)\n"
					"    Incoming Time      = %d+%fs (%f seconds)\n"
					"    Error              = %d+%fs (%f seconds)",
					*EventState.SequenceObjectPath,
					CurrentTime.FrameNumber.Value, CurrentTime.GetSubFrame(), CurrentTime / SequenceRate,
					IncomingTime.FrameNumber.Value, IncomingTime.GetSubFrame(), IncomingTime / SequenceRate,
					Error.FrameNumber.Value, Error.GetSubFrame(), Error / SequenceRate
				);
			}
		}
		else
		{
			switch (EventState.PlayerStatus)
			{
			case EConcertMovieScenePlayerStatus::Stepping:
				// fallthrough, handles as scrub
			case EConcertMovieScenePlayerStatus::Scrubbing:
				Player->SetPlaybackPosition(FMovieSceneSequencePlaybackParams(IncomingTime, EUpdatePositionMethod::Scrub));
				break;
			case EConcertMovieScenePlayerStatus::Paused:
				Player->Pause();
				Player->SetPlaybackPosition(FMovieSceneSequencePlaybackParams(IncomingTime, EUpdatePositionMethod::Jump));
				break;
			case EConcertMovieScenePlayerStatus::Stopped:
				// Stopping will reset the position, so we need to stop first and then set the position.
				Player->Pause();
				Player->SetPlaybackPosition(FMovieSceneSequencePlaybackParams(IncomingTime, EUpdatePositionMethod::Jump));
				break;
			case EConcertMovieScenePlayerStatus::Jumping:
				// fallthrough, handles as stop
			default:
				Player->SetPlaybackPosition(FMovieSceneSequencePlaybackParams(IncomingTime, EUpdatePositionMethod::Jump));
			}

			Player->SetPlayRate(EventState.PlaybackSpeed);
		}
	}
}

bool FConcertClientSequencerManager::CanSendSequencerEvent(const FString& ObjectPath) const
{
	if (TSharedPtr<FConcertClientWorkspace> SharedWorkspace = Workspace.Pin())
	{
		FString PackageName = FPackageName::ObjectPathToPackageName(ObjectPath);
		return !SharedWorkspace->IsReloadingPackage(FName(*PackageName));
	}
	return true;
}

void FConcertClientSequencerManager::OnWorkspaceEndFrameCompleted()
{
	for (const FConcertSequencerCloseEvent& CloseEvent: PendingSequenceCloseEvents)
	{
		ApplyCloseEvent(CloseEvent);
	}
	PendingSequenceCloseEvents.Reset();

	if (!PendingDestroy.IsEmpty())
	{
		for (const TTuple<FName, FString>& Player : PendingDestroy)
		{
			if (TObjectPtr<ALevelSequenceActor>* SequencePlayer = SequencePlayers.Find(Player.Get<0>()))
			{
				DestroyPlayer(*SequencePlayer);
				SequencePlayers.Remove(Player.Get<0>());
				PendingCreate.Add(Player.Get<1>());
			}
		}

		if (GEngine)
		{
			GEngine->ForceGarbageCollection(true);
		}

		// Wait for the next frame to continue. Garbage collection will not happen
		// immediately so we need to wait for the next frame before we re-create.
		UE_LOGF(LogConcertSequencerSync, VeryVerbose, "Running garbage collection after destroying %d sequence players. Deferring remaining pending events until next frame", PendingDestroy.Num());

		PendingDestroy.Reset();

		return;
	}

	// Checking for the time adjustment event must happen first because any sequencer state changes may adjust the MovieScene range based on the state message.
	// If this happens then we have no way of knowing if we need to adjust the take sections based on a new start position.   By making this check
	// happen first we avoid this scenario and can properly check time adjustment events and move sections to the new start position.
	for (const FConcertSequencerTimeAdjustmentEvent& Event : PendingTimeAdjustmentEvents)
	{
		ApplyTimeAdjustmentEvent(Event);
	}
	PendingTimeAdjustmentEvents.Reset();

	for (const FString& Player: PendingCreate)
	{
		CreateNewSequencePlayerIfNotExists(Player);
	}
	PendingCreate.Reset();

	for (const FConcertSequencerOpenEvent& SequenceOpenEvent : PendingSequenceOpenEvents)
	{
		ApplyOpenEvent(SequenceOpenEvent);
	}
	PendingSequenceOpenEvents.Reset();

	for (const FConcertSequencerState& State : PendingSequencerEvents)
	{
		ApplyTransportEvent(State);
	}
	PendingSequencerEvents.Reset();

	if (GEditor)
	{
		if (TSharedPtr<IConcertClientSession> Session = WeakSession.Pin())
		{
			for (FOpenSequencerData& Entry : OpenSequencers)
			{
				if (Entry.StateEventPacer)
				{
					Entry.StateEventPacer->Tick(Session);
				}
			}
		}
	}
}

void FConcertClientSequencerManager::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(SequencePlayers);
	Preloader->AddReferencedObjects(Collector);
}

#endif

