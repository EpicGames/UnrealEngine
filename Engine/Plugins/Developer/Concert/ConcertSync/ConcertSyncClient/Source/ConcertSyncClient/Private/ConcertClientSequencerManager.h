// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreDefines.h"
#include "UObject/NameTypes.h"
#include "UObject/GCObject.h"
#include "IConcertClientSequencerManager.h"
#include "ConcertSequencerMessages.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Templates/PimplPtr.h"

struct FConcertSessionContext;
class IConcertSyncClient;
class IConcertClientSession;
class ALevelSequenceActor;

#if WITH_EDITOR

class ISequencer;
class FConcertClientSequencePreloader;
class FConcertClientWorkspace;
struct FSequencerInitParams;
class ULevelSequence;
class FConcertClientSequencerStateEventPacer;
enum class ETransactionFilterResult : uint8;
struct FConcertTransactionFilterArgs;

/**
 * Sequencer manager that is held by the client sync module that keeps track of open sequencer UIs, regardless of whether a session is open or not
 * Events are registered to client sessions that will then operate on any tracked sequencer UIs
 */
class FConcertClientSequencerManager : public FGCObject, public IConcertClientSequencerManager
{
public:
	/**
	 * Constructor - registers OnSequencerCreated handler with the sequencer module
	 */
	explicit FConcertClientSequencerManager(IConcertSyncClient* InOwnerSyncClient);

	/**
	 * Destructor - unregisters OnSequencerCreated handler from the sequencer module
	 */
	virtual ~FConcertClientSequencerManager();

	/**
	 * Register all custom sequencer events for the specified client session
	 *
	 * @param InSession       The client session to register custom events with
	 */
	void Register(TSharedRef<IConcertClientSession> InSession);


	/**
	 * Unregister previously registered custom sequencer events from the specified client session
	 *
	 * @param InSession       The client session to unregister custom events from
	 */
	void Unregister(TSharedRef<IConcertClientSession> InSession);

	/**
	 * @return true if playback syncing across opened sequencer is enabled
	 */
	virtual bool IsSequencerPlaybackSyncEnabled() const override;

	/**
	 * Set the playback syncing option in Multi-User which syncs playback across user for opened sequencer.
	 * 
	 * @param bEnable The value to set for playback syncing of opened sequencer
	 */
	virtual void SetSequencerPlaybackSync(bool bEnable) override;

	/**
	 * @return true if unrelated timeline syncing across opened sequencer is enabled
	 */
	virtual bool IsUnrelatedSequencerTimelineSyncEnabled() const override;

	/**
	 * Set the unrelated timeline syncing option in Multi-User which syncs time from any remote sequence.
	 *
	 * @param bEnable The value to set for unrelated timeline syncing.
	 */
	virtual void SetUnrelatedSequencerTimelineSync(bool bEnable) override;

	/**
	 * @return true if the remote open option is enabled.
	 */
	virtual bool IsSequencerRemoteOpenEnabled() const override;

	/**
	 * @return true if the remote close option is enabled.
	 */
	virtual bool IsSequencerRemoteCloseEnabled() const override;

	/**
	 * @return true if the sequencer pacer is enabled.
	 */
	virtual bool IsSequencerPacingEnabled() const override;

	/**
	 * @return the current duration for the sequencer pacer.
	 */
	virtual float SequencerPacingDuration() const override;

	/**
	 * Checks the CVar to see if we are allowed to forcefully close the player on game instances.
	 *
	 * @return true if we should always close a sequence player on a -game instance.
	 */
	bool ShouldAlwaysCloseGameSequencerPlayer() const;

	/**
	 * Set the remote open option in Multi-User
	 * which opens Sequencer for other users when this option is enabled on both user machines.
	 *
	 * @param bEnable The value to set for the remote open option
	 */
	virtual void SetSequencerRemoteOpen(bool bEnable) override;

	/**
	 * Set the remote close option in Multi-User
	 * which closes Sequencer for this user when the sequence is closed by a remote user.
	 *
	 * @param bEnable The value to set for the remote close option
	 */
	virtual void SetSequencerRemoteClose(bool bEnable) override;

	/**
	 * Specifies the sequencer pacing behavior while in a session.  This will set Concert.SequencerStatePacingEnabled console variable.
	 *
	 * @param bValue The value to be set for the sequencer pacing option
	 */
	virtual void SetSequencerPacingEnabled(bool bEnable) override;

	/**
	 * Sets the sequencer pacing duration if pacing is enabled. This will set Concert.SequencerStatePacingDuration console variable.
	 *
	 * @param Duration The spacing between state event messages.  The default value is 1.0
	 */
	virtual void SetSequencerPacingDuration(float Duration) override;

	/**
	 * Temporarily suspend sequencer pacing on this client.
	 */
	virtual void SuspendSequencerPacing() override;

	/**
	 * Resume sequencer pacing on this client if the console variable is enabled.
	 */
	virtual void ResumeSequencerPacing() override;

	/** Assign the current active workspace to this sequencer. */
	void SetActiveWorkspace(TSharedPtr<FConcertClientWorkspace> Workspace);

private:
	/** Enum signifying how a sequencer UI is currently playing. Necessary to prevent transport event contention. */
	enum class EPlaybackMode
	{
		/** This sequencer's time should be propagated to the collaboration server */
		Controller,
		/** This sequencer's time should be updated in response to an event from the collaboration server */
		Agent,
		/** To our knowledge, no sequencer is playing back, and this sequencer will both send and receive transport events */
		Undefined
	};

	/** Struct containing the Open Sequencer data */
	struct FOpenSequencerData
	{
		/** Enum that signifies whether to send/receive transport events. */
		EPlaybackMode PlaybackMode;

		/** Weak pointer to the sequencer itself, if locally opened. */
		TWeakPtr<ISequencer> WeakSequencer;

		/** Delegate handle to the Global Time Changed event for the sequencer, if locally opened. */
		FDelegateHandle OnGlobalTimeChangedHandle;

		/** Delegate handle to the Close event for the sequencer, if locally opened. */
		FDelegateHandle OnCloseEventHandle;

		/** Pointer event pacer for state events. This class will pace state events so that we do not overwhelm the connection with too many. */
		TPimplPtr<FConcertClientSequencerStateEventPacer> StateEventPacer;
	};

private:
	/**
	 * Returns true if we can send the provided sequencer event onto the server.
	 */
	bool CanSendSequencerEvent(const FString& ObjectPath) const;

	/**
	 * Delegate function called when we have entered into a reload state. We need to know when we are in a asset reload
	 * so that we can properly reload the sequence player.
	 */
	void HandleAssetReload(const EPackageReloadPhase InPackageReloadPhase, FPackageReloadedEvent* InPackageReloadedEvent);

	/**
	 * Called when a sequencer closes.
	 *
	 * @param InSequencer The sequencer that closed.
	 */
	void OnSequencerClosed(TSharedRef<ISequencer> InSequencer);

	/**
	 * Called on receipt of an external transport event from another client
	 *
	 * @param InEventContext              The context for the current client session
	 * @param InEvent                     The sequencer transport event received from the client
	 */
	void OnTransportEvent(const FConcertSessionContext&, const FConcertSequencerStateEvent& InEvent);

	/**
	 * Called on receipt of an external close event from the server
	 *
	 * @param InEventContext              The context for the current client session
	 * @param InEvent                     The sequencer close event received from the server
	 */
	void OnCloseEvent(const FConcertSessionContext&, const FConcertSequencerCloseEvent& InEvent);

	/**
	 * Called on receipt of an external open event from the server
	 *
	 * @param InEventContext              The context for the current client session
	 * @param InEvent                     The sequencer open event received from the server
	 */
	void OnOpenEvent(const FConcertSessionContext&, const FConcertSequencerOpenEvent& InEvent);

	/**
	 * Called on receipt of an external state sync event from the server
	 *
	 * @param InEventContext              The context for the current client session
	 * @param InEvent                     The sequencer state sync event received from the server
	 */
	void OnStateSyncEvent(const FConcertSessionContext& InEventContext, const FConcertSequencerStateSyncEvent& InEvent);

	/**
	 * Called on receipt of an external time adjustment event from the server
	 * @param InEventContext             The context for the current session.
	 * @param InEvent                    The sequencer time adjustment event.
	 */
	void OnTimeAdjustmentEvent(const FConcertSessionContext& InEventContext, const FConcertSequencerTimeAdjustmentEvent& InEvent);

	/**
	 * Called when the global time has been changed for the specified Sequencer
	 *
	 * @param InSequencer                 The sequencer that has just updated its time
	 */
	void OnSequencerTimeChanged(TWeakPtr<ISequencer> InSequencer);

	/**
	 * Handle the creation of a newly opened sequencer instance
	 *
	 * @param InSequencer                 The sequencer that has just been created. Should not hold persistent shared references.
	 */
	void OnSequencerCreated(TSharedRef<ISequencer> InSequencer);

	/**
	 * Handle the end of frame callback to apply pending sequencer events.
	 * 
	 * Note that this is called through a delegate on the workspace instead of
	 * being called directly by a core engine delegate. This is to ensure that
	 * all workspace operations have completed first.
	 */
	void OnWorkspaceEndFrameCompleted();

	/**
	 * Apply a Sequencer open event
	 *
	 * @param SequenceObjectPath	The sequence to open
	 */
	void ApplyOpenEvent(const FConcertSequencerOpenEvent& InOpenEvent);

	/**
	 * Create a new sequence player to be used in -game instances.
	 *
	 * @param SequenceObjectPath	The object path for the Level Sequence object we are creating.
	 */
	void CreateNewSequencePlayerIfNotExists(const FString& SequenceObjectPath);

	/**
	 * Apply a Sequencer event
	 *
	 * @param PendingState	The pending state to apply
	 */
	void ApplyTransportEvent(const FConcertSequencerState& PendingState);

	/**
	 * Apply a Sequencer event to opened Sequencers
	 *
	 * @param PendingState	The pending state to apply
	 */
	void ApplyEventToSequencers(const FConcertSequencerState& PendingState);

	/**
	 * Apply a Sequencer event to SequencePlayers
	 *
	 * @param PendingState	The pending state to apply
	 */
	void ApplyEventToPlayers(const FConcertSequencerState& PendingState);

	/**
	 * Apply a Sequencer close event to open players.
	 * 
	 * This is a helper function called by ApplyCloseEvent().
	 *
	 * @param CloseEvent  The close event state
	 */
	void ApplyCloseEventToPlayers(const FConcertSequencerCloseEvent& CloseEvent);

	/**
	 * Apply a Sequencer close event to open Sequencers and players.
	 *
	 * @param CloseEvent  The close event state
	 */
	void ApplyCloseEvent(const FConcertSequencerCloseEvent& CloseEvent);

	/**
	 * Apply a sequencer time adjustment event.
	 * 
	 * @param TimeAdjustmentEvent  The time adjustment event state
	 */
	void ApplyTimeAdjustmentEvent(const FConcertSequencerTimeAdjustmentEvent& TimeAdjustmentEvent);

	/**
	 * Apply a sequencer time adjustment to players.
	 * 
	 * @param TimeAdjustmentEvent  The time adjustment event state
	 */
	void ApplyTimeAdjustmentToPlayers(const FConcertSequencerTimeAdjustmentEvent& TimeAdjustmentEvent);

	/**
	 * Apply a sequencer time adjustment to sequencers.
	 * 
	 * @param TimeAdjustmentEvent  The time adjustment event state
	 */
	void ApplyTimeAdjustmentToSequencers(const FConcertSequencerTimeAdjustmentEvent& TimeAdjustmentEvent);

	/**
	 * Gather all the currently open sequencer UIs that have the specified path as their root sequence
	 *
	 * @param InSequenceObjectPath        The full path to the root asset to gather sequences for
	 * @return An array containing all the entries that apply to the supplied sequence path
	 */
	TArray<FOpenSequencerData*, TInlineAllocator<1>> GatherRootSequencersByState(const FString& InSequenceObjectPath);

	/**
	 * Get the amount of latency compensation to apply to time-synchronization sensitive interactions
	 */
	float GetLatencyCompensationMs() const;

	/** FGCObject interface*/
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FConcertClientSequencerManager");
	}

private:
	/**
	 * Delegate to provide a hint to the transaction bridge that the given object should be transacted.  We special case LevelSequences
	 * and allow any transaction that has a level sequence as an outermost object to be included because there are many object types
	 * in level sequence transactions that do not share UMovieSceneSignedObject as a base class, which is used by the default transaction
	 * filters.
	 */
	ETransactionFilterResult ShouldObjectBeTransacted(const FConcertTransactionFilterArgs& FilterArgs);
		
	/** Destroy the given sequence player with corresponding actor. */
	void DestroyPlayer(ALevelSequenceActor* LevelSequenceActor);

	/** Indicates if this sequence player can be closed. */
	bool CanClose(const FConcertSequencerCloseEvent& InEvent) const;

	/** Pointer to the sync client that owns us. */
	IConcertSyncClient* OwnerSyncClient;

	/** List of pending sequencer events to apply at end of frame. */
	TArray<FConcertSequencerState> PendingSequencerEvents;

	/** List of pending sequencer open events to apply at end of frame. */
	TArray<FConcertSequencerOpenEvent> PendingSequenceOpenEvents;

	/** List of pending sequencer open events to apply at end of frame. */
	TArray<FConcertSequencerCloseEvent> PendingSequenceCloseEvents;

	/** List of pending time adjustment events. */
	TArray<FConcertSequencerTimeAdjustmentEvent> PendingTimeAdjustmentEvents;

	/**
	 * List of pending sequence players that are scheduled for destruction. The first member of the tuple is the key /
	 * level sequence player we a removing.  The second member tuple is the soft object path name of the replacement
	 * level sequence when it is recreated.
	 */
	TArray<TTuple<FName,FString>> PendingDestroy;

	/** List of pending sequence players that are scheduled for creation. */
	TArray<FString> PendingCreate;

	/** Map of all currently opened Root Sequence State in a session, locally opened or not. */
	TMap<FName, FConcertSequencerState> SequencerStates;

	/** List of all locally opened sequencer. */
	TArray<FOpenSequencerData> OpenSequencers;

	/** Map of opened sequence players, if not in editor mode. */
	TMap<FName, TObjectPtr<ALevelSequenceActor>> SequencePlayers;

	/** Handle preload requests from the server, asynchronously loading and maintaining GC references to sequence assets. */
	TSharedRef<FConcertClientSequencePreloader> Preloader;

	/** Boolean that is set when we are handling any transport event to prevent re-entrancy */
	bool bRespondingToTransportEvent;

	/** Delegate handle for the global sequencer created event registered with the sequencer module */
	FDelegateHandle OnSequencerCreatedHandle;

	/** Weak pointer to the client session with which to send events. May be null or stale. */
	TWeakPtr<IConcertClientSession> WeakSession;

	/** Weak pointer to the active workspace. */
	TWeakPtr<FConcertClientWorkspace> Workspace;

	friend class FConcertClientSequencerStateEventPacer;
};

#endif // WITH_EDITOR
