// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertClientTransactionBridge.h"
#include "ConcertClientObjectFactory.h"
#include "ConcertLogGlobal.h"
#include "ConcertSyncSettings.h"
#include "ConcertSyncClientUtil.h"
#include "ConcertTransactionEvents.h"
#include "IConcertClientTransactionBridge.h"

#include "GameFramework/Actor.h"
#include "HAL/IConsoleManager.h"
#include "Misc/ITransactionObjectAnnotation.h"
#include "Misc/PackageName.h"
#include "Misc/CoreDelegates.h"
#include "Misc/TransactionCommon.h"
#include "UObject/Package.h"
#include "LevelInstance/LevelInstanceActor.h"
#include "LevelInstance/LevelInstanceComponent.h"
#include "Components/PrimitiveComponent.h"
#include "UObject/OverridableManager.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/WeakObjectPtr.h"


#if WITH_EDITOR
	#include "Editor.h"
	#include "UnrealEdGlobals.h"
	#include "Editor/UnrealEdEngine.h"
	#include "Editor/TransBuffer.h"
	#include "Engine/Selection.h"

	#include "Framework/Commands/UIAction.h"

	#include "Elements/Framework/EngineElementsLibrary.h"
	#include "Elements/Framework/TypedElementSelectionSet.h"
#endif

LLM_DEFINE_TAG(Concert_ConcertClientTransactionBridge);
#define LOCTEXT_NAMESPACE "ConcertClientTransactionBridge"

extern ENGINE_API bool GFlushRenderingCommandsOnPreEditChange;

namespace ConcertClientTransactionBridgeUtil
{

static TAutoConsoleVariable<int32> CVarIgnoreTransactionIncludeFilter(TEXT("Concert.IgnoreTransactionFilters"), 0, TEXT("Ignore Transaction Object Allow List Filtering"));
static TAutoConsoleVariable<int32> CVarAlwaysSuspendBroadcastUndoRedo(TEXT("Concert.AlwaysSuspendBroadcastUndoRedo"), 0,
																	  TEXT("Always suspend the undo/redo broadcast message. Can help with transactions that steal user focus."));

#if WITH_EDITOR

void ReportCurrentTransactionState()
{
	UE_LOGF(LogConcert, Display, "Undo/Redo GIsTransacting is %ls.", GIsTransacting ? TEXT("true") : TEXT("false"));
	UE_LOGF(LogConcert, Display, "GEditor->CanTransact() is %ls.", GEditor && GEditor->CanTransact() ? TEXT("true") : TEXT("false"));
	UE_LOGF(LogConcert, Display, "GEditor->IsTransactionActive() is %ls.", GEditor && GEditor->IsTransactionActive() ? TEXT("true") : TEXT("false"));
}

void ResetTransaction()
{
	if (GEditor)
	{
		GEditor->ResetTransaction(LOCTEXT("ConcertDebugCvarReset", "Reset by console variable command."));
	}
}

FAutoConsoleCommand CVarReportCurrentTransactionState(TEXT("Concert.Debug.ReportTransactionState"),
	TEXT("Report to the log file the current transaction state."),
	FExecuteAction::CreateStatic(&ReportCurrentTransactionState));

FAutoConsoleCommand CVarResetTransaction(TEXT("Concert.Debug.ResetTransaction"),
	TEXT("Reset the current open transaction."),
	FExecuteAction::CreateStatic(&ResetTransaction));

#endif // WITH_EDITOR
	   
UClass* LoadClassHelper(const FSoftClassPath& ClassPath)
{
	static TMap<FSoftClassPath, TWeakObjectPtr<UClass>> ClassCache;
	if (TWeakObjectPtr<UClass>* ExistingClass = ClassCache.Find(ClassPath))
	{
		if (ExistingClass->IsStale())
		{
			*ExistingClass = ClassPath.TryLoadClass<UObject>();
		}
		return ExistingClass->Get();
	}
	return ClassCache.Add(ClassPath, ClassPath.TryLoadClass<UObject>()).Get();
}
	
bool RunTransactionFilters(const TArray<FTransactionClassFilter>& InFilters, UObject* InObject)
{
	bool bMatchFilter = false;
	for (const FTransactionClassFilter& TransactionFilter : InFilters)
	{
		UClass* TransactionOuterClass = LoadClassHelper(TransactionFilter.ObjectOuterClass);
		if (!TransactionOuterClass || InObject->IsInA(TransactionOuterClass))
		{
			for (const FSoftClassPath& ObjectClass : TransactionFilter.ObjectClasses)
			{
				UClass* TransactionClass = LoadClassHelper(ObjectClass);
				if (TransactionClass && InObject->IsA(TransactionClass))
				{
					bMatchFilter = true;
					break;
				}
			}
		}
	}

	return bMatchFilter;
}

#if WITH_EDITOR
template <typename TUObjectType>
FTypedElementHandle AcquireTypedElementHandle(const TUObjectType* InObject)
{
	if constexpr(std::is_same<AActor,TUObjectType>::value)
	{
		return UEngineElementsLibrary::AcquireEditorActorElementHandle(InObject, /*bAllowCreate*/false);
	}
	else if constexpr(std::is_same<UActorComponent,TUObjectType>::value)
	{
		return UEngineElementsLibrary::AcquireEditorComponentElementHandle(InObject, /*bAllowCreate*/false);
	}
	else
	{
		return {};
	}
}

template <typename TUObjectArrayType>
void DeselectElements(UTypedElementSelectionSet* InSelectionSet, const TUObjectArrayType& InUObjectArray, const FTypedElementSelectionOptions& SelectionOptions)
{
	using ElementType = typename TUObjectArrayType::ElementType;
	static_assert(std::is_same<AActor*,ElementType>::value || std::is_same<UActorComponent*,ElementType>::value);
	if (InUObjectArray.Num() == 0)
	{
		return;
	}
	TArray<FTypedElementHandle> HandlesToRemoveFromSelection;
	for (ElementType Object: InUObjectArray)
	{
		if (FTypedElementHandle Handle = AcquireTypedElementHandle(Object))
		{
			HandlesToRemoveFromSelection.Add(Handle);
		}
	}
	InSelectionSet->DeselectElements(HandlesToRemoveFromSelection, SelectionOptions);
}
#endif

void DeselectActorsAndActorComponents(const TArray<AActor*>& DeletedActors, const TArray<UActorComponent*>& DeletedActorComponents)
{
#if WITH_EDITOR
	if (!GEditor)
	{
		return;
	}

	const FTypedElementSelectionOptions SelectionOptions = FTypedElementSelectionOptions()
		.SetAllowHidden(true)
		.SetAllowGroups(false)
		.SetWarnIfLocked(false)
		.SetChildElementInclusionMethod(ETypedElementChildInclusionMethod::Recursive);

	DeselectElements(GEditor->GetSelectedActors()->GetElementSelectionSet(), DeletedActors, SelectionOptions);
	DeselectElements(GEditor->GetSelectedComponents()->GetElementSelectionSet(), DeletedActorComponents, SelectionOptions);
#endif
}

ETransactionFilterResult ApplyCustomFilter(const TMap<FName, FOnFilterTransactionDelegate>& CustomFilters, UObject* InObject, UPackage* InChangedPackage, const FTransactionObjectEvent& InTransactionEvent)
{
	ensure(InObject && InChangedPackage);
	const FConcertTransactionFilterArgs FilterArgs(InObject, InChangedPackage, InTransactionEvent);
	
	for (const auto& Item : CustomFilters)
	{
		if (Item.Value.IsBound())
		{
			ETransactionFilterResult Result = Item.Value.Execute(FilterArgs);
			if (Result != ETransactionFilterResult::UseDefault)
			{
				return Result;
			}
		}
	}
	return ETransactionFilterResult::UseDefault;
}

ETransactionFilterResult ApplyTransactionFilters(const TMap<FName, FOnFilterTransactionDelegate>& CustomFilters, UObject* InObject, UPackage* InChangedPackage, const FTransactionObjectEvent& InTransactionEvent)
{
	// An object is persistent if neither it nor any of its outers are transient
	auto IsObjectPersistent = [](const UObject* Obj)
	{
		for (; Obj; Obj = Obj->GetOuter())
		{
			if (Obj->HasAnyFlags(RF_Transient))
			{
				return false;
			}
		}
		return true;
	};

	auto WasObjectPersistent = [&IsObjectPersistent](const EObjectFlags OriginalFlags, const FName OriginalOuterPathName)
	{
		if (!EnumHasAnyFlags(OriginalFlags, RF_Transient))
		{
			if (UObject* OriginalOuter = FSoftObjectPath(FNameBuilder(OriginalOuterPathName).ToView()).ResolveObject();
				OriginalOuter && IsObjectPersistent(OriginalOuter))
			{
				return true;
			}
		}
		return false;
	};

	const ETransactionFilterResult FilterResult = ConcertClientTransactionBridgeUtil::ApplyCustomFilter(CustomFilters, InObject, InChangedPackage, InTransactionEvent);
	if (FilterResult != ETransactionFilterResult::UseDefault)
	{
		return FilterResult;
	}

	// Ignore compiled in package, as they are not considered Multi-user content.
	if (!InChangedPackage || InChangedPackage->HasAnyPackageFlags(PKG_CompiledIn))
	{
		return ETransactionFilterResult::ExcludeObject;
	}

	// Ignore transient packages and objects, unless the object was previously persistent
	if ((InChangedPackage == GetTransientPackage() || !IsObjectPersistent(InObject)) && !WasObjectPersistent(InTransactionEvent.GetOriginalObjectFlags(), InTransactionEvent.GetOriginalObjectOuterPathName()))
	{
		return ETransactionFilterResult::ExcludeObject;
	}

	// Ignore packages outside of known root paths (we ignore read-only roots here to skip things like unsaved worlds)
	if (!FPackageName::IsValidLongPackageName(InChangedPackage->GetName()))
	{
		return ETransactionFilterResult::ExcludeObject;
	}

	const UConcertSyncConfig* SyncConfig = GetDefault<UConcertSyncConfig>();

	// Run our exclude transaction filters: if a filter is matched on an object the whole transaction is excluded.
	if (SyncConfig->ExcludeTransactionClassFilters.Num() > 0 && RunTransactionFilters(SyncConfig->ExcludeTransactionClassFilters, InObject))
	{
		return ETransactionFilterResult::ExcludeTransaction;
	}

	// Run our include object filters: if the list is empty or we actively ignore the list then all objects are included,
	// otherwise a filter needs to be matched.
	if (SyncConfig->IncludeObjectClassFilters.Num() == 0 
		|| (CVarIgnoreTransactionIncludeFilter.GetValueOnAnyThread() > 0)
		|| RunTransactionFilters(SyncConfig->IncludeObjectClassFilters, InObject))
	{
		return ETransactionFilterResult::IncludeObject;
	}

	// Otherwise the object is excluded from the transaction
	return ETransactionFilterResult::ExcludeObject;
}

#if WITH_EDITOR
/** Utility struct to suppress editor transaction notifications and fire the correct delegates */
struct FEditorTransactionNotification
{
	FEditorTransactionNotification(FTransactionContext&& InTransactionContext)
		: TransactionContext(MoveTemp(InTransactionContext))
		, TransBuffer(GUnrealEd ? Cast<UTransBuffer>(GUnrealEd->Trans) : nullptr)
		, bOrigSquelchTransactionNotification(GEditor && GEditor->bSquelchTransactionNotification)
		, bOrigNotifyUndoRedoSelectionChange(GEditor && GEditor->bNotifyUndoRedoSelectionChange)
		, bOrigIgnoreSelectionChange(GEditor && GEditor->bIgnoreSelectionChange)
		, bOrigSuspendBroadcastPostUndoRedo(GEditor && GEditor->bSuspendBroadcastPostUndoRedo)
	{
	}

	bool IsTransactedObjectInSelection() const
	{
		if (!TransBuffer || !GEditor)
		{
			return false;
		}

		USelection* SelectedActors = GEditor->GetSelectedActors();
		for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
		{
			UObject* SelectionObject = *Iter;
			if (SelectionObject == TransactionContext.PrimaryObject)
			{
				return true;
			}
		}
		return false;
	}

	void PreUndo()
	{
		if (GEditor)
		{
			GEditor->bSquelchTransactionNotification = true;
			GEditor->bNotifyUndoRedoSelectionChange = true;
			if (TransBuffer)
			{
				if (CVarAlwaysSuspendBroadcastUndoRedo.GetValueOnAnyThread() > 0 || (ConcertSyncClientUtil::IsUserEditing() && !IsTransactedObjectInSelection()))
				{
					GEditor->bIgnoreSelectionChange = true;
					GEditor->bSuspendBroadcastPostUndoRedo = true;
				}

				TransBuffer->OnBeforeRedoUndo().Broadcast(TransactionContext);
			}
		}
	}

	void PostUndo()
	{
		if (GEditor)
		{
			if (TransBuffer)
			{
				TransBuffer->OnRedo().Broadcast(TransactionContext, true);
			}
			GEditor->bSquelchTransactionNotification = bOrigSquelchTransactionNotification;
			GEditor->bNotifyUndoRedoSelectionChange = bOrigNotifyUndoRedoSelectionChange;
			GEditor->bIgnoreSelectionChange = bOrigIgnoreSelectionChange;
			GEditor->bSuspendBroadcastPostUndoRedo = bOrigSuspendBroadcastPostUndoRedo;
		}
	}

	void HandleObjectTransacted(UObject* InTransactionObject, const FConcertExportedObject& InObjectUpdate, const TSharedPtr<ITransactionObjectAnnotation>& InTransactionAnnotation)
	{
		if (GUnrealEd)
		{
			FTransactionObjectEvent TransactionObjectEvent;
			{
				FTransactionObjectDeltaChange DeltaChange;
				DeltaChange.bHasNameChange = !InObjectUpdate.ObjectData.NewName.IsNone();
				DeltaChange.bHasOuterChange = !InObjectUpdate.ObjectData.NewOuterPathName.IsNone();
				DeltaChange.bHasExternalPackageChange = !InObjectUpdate.ObjectData.NewExternalPackageName.IsNone();
				DeltaChange.bHasPendingKillChange = InObjectUpdate.ObjectData.bIsPendingKill != !IsValid(InTransactionObject);
				DeltaChange.bHasNonPropertyChanges = InObjectUpdate.ObjectData.SerializedData.Num() > 0;
				for (const FConcertSerializedPropertyData& PropertyData : InObjectUpdate.PropertyDatas)
				{
					DeltaChange.ChangedProperties.Add(PropertyData.PropertyName);
				}
				TransactionObjectEvent = FTransactionObjectEvent(TransactionContext.TransactionId, TransactionContext.OperationId, ETransactionObjectEventType::UndoRedo, ETransactionObjectChangeCreatedBy::TransactionRecord,
					FTransactionObjectChange{ InObjectUpdate.ObjectId.ToTransactionObjectId(), (EObjectFlags)InObjectUpdate.ObjectId.ObjectPersistentFlags, MoveTemp(DeltaChange) }, InTransactionAnnotation);
			}
			GUnrealEd->HandleObjectTransacted(InTransactionObject, TransactionObjectEvent);
		}
	}

	FTransactionContext TransactionContext;
	UTransBuffer* TransBuffer;
	bool bOrigSquelchTransactionNotification;
	bool bOrigNotifyUndoRedoSelectionChange;
	bool bOrigIgnoreSelectionChange;
	bool bOrigSuspendBroadcastPostUndoRedo;
};
#endif


void ProcessTransactionEvent(const FConcertTransactionEventBase& InEvent, const FConcertSessionVersionInfo* InVersionInfo, const TArray<FName>& InPackagesToProcess, const FConcertLocalIdentifierTable* InLocalIdentifierTablePtr, const bool bIsSnapshot, const FConcertSyncWorldRemapper& WorldRemapper, const bool bIncludeEditorOnlyProperties, IConcertClientTransactionBridge::FRemoteTransactionResult* ResultInfo)
{
	// Transactions are applied in multiple-phases...
	//	0) Sort the objects to be processed; creation needs to happen parent->child, and update needs to happen child->parent
	//	1) Find or create all objects in the transaction (to handle object-interdependencies in the serialized data)
	//	2) Notify all objects that they are about to be changed (via PreEditUndo, parent->child)
	//	3) Update the state of all objects
	//	4) Notify all objects that they were changed (via PostEditUndo, child->parent)

	// --------------------------------------------------------------------------------------------------------------------
	// Phase 0
	// --------------------------------------------------------------------------------------------------------------------

#if WITH_EDITOR
	const bool bIsWorldPartitionWorldInGame = !GIsEditor && ConcertSyncClientUtil::IsWorldPartitionWorld();
#endif

	TArray<const FConcertExportedObject*, TInlineAllocator<32>> SortedExportedObjects;
	{
		SortedExportedObjects.Reserve(InEvent.ExportedObjects.Num());
		for (const FConcertExportedObject& ExportedObject : InEvent.ExportedObjects)
		{
#if WITH_EDITOR
			// If the exported object in a level instance and we are not in
			// -game or in a WP -game then skip processing. We can't process in
			// -game with WP due to constraints with cell generation and in
			// editor sessions the normal transaction flow will handle the proxy
			// object.
			//
			if (ExportedObject.bHasLevelInstanceObject && (GIsEditor || bIsWorldPartitionWorldInGame))
			{
				continue;
			}
#endif
			SortedExportedObjects.Add(&ExportedObject);
		}

		SortedExportedObjects.StableSort([](const FConcertExportedObject& One, const FConcertExportedObject& Two) -> bool
		{
			return One.ObjectPathDepth < Two.ObjectPathDepth;
		});
	}

	// --------------------------------------------------------------------------------------------------------------------
	// Phase 1
	// --------------------------------------------------------------------------------------------------------------------
	bool bObjectsDeleted = false;
#if WITH_EDITOR
	TArray<AActor*> ResurrectedActors;
#endif
	TArray<ConcertSyncClientUtil::FGetObjectResult, TInlineAllocator<32>> TransactionObjects;
	{
		TSet<const UObject*> NewlyCreatedObjects;
		TSortedMap<const UConcertClientObjectFactory*, TArray<UObject*>> ObjectsPendingInitialization;

		// Find or create each object in parent->child order
		TransactionObjects.AddDefaulted(SortedExportedObjects.Num());
		for (int32 ObjectIndex = 0; ObjectIndex < SortedExportedObjects.Num(); ++ObjectIndex)
		{
			ConcertSyncClientUtil::FGetObjectResult& TransactionObjectRef = TransactionObjects[ObjectIndex];
			const FConcertExportedObject& ObjectUpdate = *SortedExportedObjects[ObjectIndex];

			// Is this object excluded? We exclude certain packages when re-applying live transactions on a package load
			if (InPackagesToProcess.Num() > 0)
			{
				// if we have an assigned package, use that to validate package, or should we?
				FName ObjectPackageName = ObjectUpdate.ObjectData.NewPackageName.IsNone() ? ObjectUpdate.ObjectId.ObjectPackageName : ObjectUpdate.ObjectData.NewPackageName;
				if (!InPackagesToProcess.Contains(ObjectPackageName))
				{
					continue;
				}
			}

			// Find or create the object
			TransactionObjectRef = ConcertSyncClientUtil::GetObject(ObjectUpdate.ObjectId, ObjectUpdate.ObjectData.NewName, ObjectUpdate.ObjectData.NewOuterPathName, ObjectUpdate.ObjectData.NewExternalPackageName, ObjectUpdate.ObjectData.SourceObject, ObjectUpdate.ObjectData.bAllowCreate);
			bObjectsDeleted |= (ObjectUpdate.ObjectData.bIsPendingKill || TransactionObjectRef.NeedsGC());

			if (TransactionObjectRef.Obj)
			{
				if (TransactionObjectRef.NewlyCreated())
				{
					// If this object was created from a factory, then it may need further initialization once every object has been created
					if (TransactionObjectRef.Factory)
					{
						ObjectsPendingInitialization.FindOrAdd(TransactionObjectRef.Factory).Add(TransactionObjectRef.Obj);
					}

					// If this is a new component then we need to apply its CreationMethod (if present) early, as it could affect the PreEdit behavior
					if (UActorComponent* Component = Cast<UActorComponent>(TransactionObjectRef.Obj))
					{
						static const FName NAME_CreationMethod = "CreationMethod";
						const FConcertSerializedPropertyData* CreationMethodPropertyData = ObjectUpdate.PropertyDatas.FindByPredicate([](const FConcertSerializedPropertyData& PropertyData) { return PropertyData.PropertyName == NAME_CreationMethod; });
						if (CreationMethodPropertyData && !CreationMethodPropertyData->SerializedData.IsEmpty())
						{
							if (FProperty* CreationMethodProperty = FindFProperty<FProperty>(TransactionObjectRef.Obj->GetClass(), CreationMethodPropertyData->PropertyName))
							{
								FConcertSyncObjectReader ObjectReader(InLocalIdentifierTablePtr, WorldRemapper, InVersionInfo, TransactionObjectRef.Obj, CreationMethodPropertyData->SerializedData);
								ObjectReader.SerializeProperty(CreationMethodProperty, TransactionObjectRef.Obj);
							}
						}
					}

					// Track this object (and any inner objects, as they must also be new) as newly created
					NewlyCreatedObjects.Add(TransactionObjectRef.Obj);
					ForEachObjectWithOuter(TransactionObjectRef.Obj, [&NewlyCreatedObjects](UObject* InnerObj)
					{
						NewlyCreatedObjects.Add(InnerObj);
					});
				}
				
				// Update the pending kill state
				const bool bWasPendingKill = !IsValid(TransactionObjectRef.Obj);
				ConcertSyncClientUtil::UpdatePendingKillState(TransactionObjectRef.Obj, ObjectUpdate.ObjectData.bIsPendingKill);

				if (ObjectUpdate.ObjectData.bResetExisting && !NewlyCreatedObjects.Contains(TransactionObjectRef.Obj))
				{
					ConcertSyncUtil::ResetObjectPropertiesToArchetypeValues(TransactionObjectRef.Obj, bIncludeEditorOnlyProperties);
				}

#if WITH_EDITOR
				// We do not need to handle the resurrection case outside the editor as it involves objects being partially active due to the transaction buffer, a feature exclusive to the editor.
				// Remember: This case would happen if you undo the deletion of an actor... that does not happen in games!
				const bool bWasResurrected = !TransactionObjectRef.NewlyCreated() && !ObjectUpdate.ObjectData.bIsPendingKill && bWasPendingKill;
				if (bWasResurrected)
				{
					// If we're bringing this actor back to life, then make sure any SCS/UCS components exist in a clean state
					// We have to do this as some of the actor components may have been GC'd when using "AllowEliminatingReferences(false)" (eg, within the transaction buffer)
					if (AActor* Actor = Cast<AActor>(TransactionObjectRef.Obj))
					{
						ResurrectedActors.Add(Actor);

						TSet<const UObject*> ExistingSubObjects;
						ForEachObjectWithOuter(Actor, [&ExistingSubObjects](UObject* InnerObj)
						{
							ExistingSubObjects.Add(InnerObj);
						});

						Actor->RerunConstructionScripts();

						// Track any reconstructed objects as newly created
						ForEachObjectWithOuter(Actor, [&NewlyCreatedObjects, &ExistingSubObjects](UObject* InnerObj)
						{
							if (!ExistingSubObjects.Contains(InnerObj))
							{
								NewlyCreatedObjects.Add(InnerObj);
							}
						});
					}
				}
#endif
			}
		}

		// Track the extended result info for the objects, if requested
		if (ResultInfo)
		{
			for (int32 ObjectIndex = 0; ObjectIndex < SortedExportedObjects.Num(); ++ObjectIndex)
			{
				ConcertSyncClientUtil::FGetObjectResult& TransactionObjectRef = TransactionObjects[ObjectIndex];
				const FConcertExportedObject& ObjectUpdate = *SortedExportedObjects[ObjectIndex];

				if (TransactionObjectRef.Obj)
				{
					IConcertClientTransactionBridge::FRemoteTransactionObjectResult& ObjectResultInfo = ResultInfo->Objects.AddDefaulted_GetRef();
					ObjectResultInfo.ObjectPath = TransactionObjectRef.Obj;
					ObjectResultInfo.ObjectIndex = &ObjectUpdate - InEvent.ExportedObjects.GetData();
					ObjectResultInfo.bNewlyCreated = NewlyCreatedObjects.Contains(TransactionObjectRef.Obj);
				}
			}
		}

		// Run any deferred object factory initialization
		if (ObjectsPendingInitialization.Num() > 0)
		{
			// If factories have initialization order dependencies, then make sure and respect those
			TArray<const UConcertClientObjectFactory*, TInlineAllocator<4>> SortedFactories;
			ObjectsPendingInitialization.GetKeys(SortedFactories);
			Algo::Sort(SortedFactories,
				[](const UConcertClientObjectFactory* ThisFactory, const UConcertClientObjectFactory* OtherFactory)
				{
					return ThisFactory->ShouldInitializeObjectsBefore(OtherFactory);
				});

			for (const UConcertClientObjectFactory* Factory : SortedFactories)
			{
				const TArray<UObject*>& Objects = ObjectsPendingInitialization.FindChecked(Factory);
				Factory->InitializeObjects(Objects);
			}
		}
	}

#if WITH_EDITOR
	UObject* PrimaryObject = InEvent.PrimaryObjectId.ObjectName.IsNone() ? nullptr : ConcertSyncClientUtil::GetObject(InEvent.PrimaryObjectId, FName(), FName(), FName(), FSoftObjectPath(), /*bAllowCreate*/false).Obj;
	FEditorTransactionNotification EditorTransactionNotification(FTransactionContext(InEvent.TransactionId, InEvent.OperationId, LOCTEXT("ConcertTransactionEvent", "Concert Transaction Event"), TEXT("Concert Transaction Event"), PrimaryObject));
	if (!bIsSnapshot)
	{
		EditorTransactionNotification.PreUndo();
	}
#endif

	// --------------------------------------------------------------------------------------------------------------------
	// Phase 2
	// --------------------------------------------------------------------------------------------------------------------
#if WITH_EDITOR
	TArray<TSharedPtr<ITransactionObjectAnnotation>, TInlineAllocator<32>> TransactionAnnotations;
	TransactionAnnotations.AddDefaulted(SortedExportedObjects.Num());
	for (int32 ObjectIndex = 0; ObjectIndex < SortedExportedObjects.Num(); ++ObjectIndex)
	{
		const ConcertSyncClientUtil::FGetObjectResult& TransactionObjectRef = TransactionObjects[ObjectIndex];
		const FConcertExportedObject& ObjectUpdate = *SortedExportedObjects[ObjectIndex];

		UObject* TransactionObject = TransactionObjectRef.Obj;
		if (!TransactionObject)
		{
			continue;
		}

		// Restore its annotation data
		TSharedPtr<ITransactionObjectAnnotation>& TransactionAnnotation = TransactionAnnotations[ObjectIndex];
		if (ObjectUpdate.SerializedAnnotationData.Num() > 0)
		{
			FConcertSyncObjectReader AnnotationReader(InLocalIdentifierTablePtr, WorldRemapper, InVersionInfo, TransactionObject, ObjectUpdate.SerializedAnnotationData);
			TransactionAnnotation = TransactionObject->CreateAndRestoreTransactionAnnotation(AnnotationReader);
			UE_CLOGF(!TransactionAnnotation.IsValid(), LogConcert, Warning, "Object '%ls' had transaction annotation data that failed to restore!", *TransactionObject->GetPathName());
		}

		// Notify before changing anything
		if (!bIsSnapshot || TransactionAnnotation)
		{
			ULevel* Level = Cast<ULevel>(TransactionObject->GetOuter());
			const bool bLevelIsDirty = Level ? Level->GetPackage()->IsDirty() : false;

			// Transaction annotations require us to invoke the redo flow (even for snapshots!) as that's the only thing that can apply the annotation
			{
				TGuardValue<bool> IsTransactingGuard(GIsTransacting, true);
				TransactionObject->PreEditUndo();
			}

			// Levels are immune from dirty changes when using external objects.  See ULevel::PreEditUndo() If we
			// modified any dirty flags as a result of the PreEditUndo then restore it back here as if it didn't happen.
			//
			if (Level && Level->IsUsingExternalObjects() && TransactionObject->IsPackageExternal())
			{
				Level->GetPackage()->SetDirtyFlag(bLevelIsDirty);
			}
		}

		// We need to manually call OnPreObjectPropertyChanged as PreEditUndo calls the PreEditChange version that skips it, but we have things that rely on it being called
		// For snapshot events this also triggers PreEditChange directly since we can skip the call to PreEditUndo
		for (const FConcertSerializedPropertyData& PropertyData : ObjectUpdate.PropertyDatas)
		{
			FProperty* TransactionProp = FindFProperty<FProperty>(TransactionObject->GetClass(), PropertyData.PropertyName);
			if (TransactionProp)
			{
				if (bIsSnapshot)
				{
					TGuardValue<bool> IsTransactingGuard(GIsTransacting, true);
					// Prevent FlushRenderingCommands from running when in -game. It can cause performance issues / hitching during user interaction.
					TGuardValue<bool> ShouldFlushRenderingCommands(GFlushRenderingCommandsOnPreEditChange, GIsEditor);					
					TransactionObject->PreEditChange(TransactionProp);
				}

				FEditPropertyChain PropertyChain;
				PropertyChain.AddHead(TransactionProp);
				FCoreUObjectDelegates::OnPreObjectPropertyChanged.Broadcast(TransactionObject, PropertyChain);
			}
		}
	}
#endif

	// --------------------------------------------------------------------------------------------------------------------
	// Phase 3
	// --------------------------------------------------------------------------------------------------------------------
	for (int32 ObjectIndex = SortedExportedObjects.Num() - 1; ObjectIndex >= 0; --ObjectIndex)
	{
		const ConcertSyncClientUtil::FGetObjectResult& TransactionObjectRef = TransactionObjects[ObjectIndex];
		const FConcertExportedObject& ObjectUpdate = *SortedExportedObjects[ObjectIndex];

		UObject* TransactionObject = TransactionObjectRef.Obj;
		if (!TransactionObject)
		{
			continue;
		}

		TransactionObject->Modify();

		// Apply the new data
		if (ObjectUpdate.ObjectData.SerializedData.Num() > 0)
		{
			FConcertSyncObjectReader ObjectReader(InLocalIdentifierTablePtr, WorldRemapper, InVersionInfo, TransactionObject, ObjectUpdate.ObjectData.SerializedData);
			ObjectReader.SerializeObject(TransactionObject);
		}
		else
		{
			for (const FConcertSerializedPropertyData& PropertyData : ObjectUpdate.PropertyDatas)
			{
				FProperty* TransactionProp = FindFProperty<FProperty>(TransactionObject->GetClass(), PropertyData.PropertyName);
				if (TransactionProp)
				{
					FConcertSyncObjectReader ObjectReader(InLocalIdentifierTablePtr, WorldRemapper, InVersionInfo, TransactionObject, PropertyData.SerializedData);
					ObjectReader.SerializeProperty(TransactionProp, TransactionObject);
				}
			}
		}
	}

	// --------------------------------------------------------------------------------------------------------------------
	// Phase 4
	// --------------------------------------------------------------------------------------------------------------------
	TArray<AActor*> DeletedActorsForSelectionUpdate;
	TArray<UActorComponent*> DeletedActorComponentForSelectionUpdate;
	for (int32 ObjectIndex = SortedExportedObjects.Num() - 1; ObjectIndex >= 0; --ObjectIndex)
	{
		const ConcertSyncClientUtil::FGetObjectResult& TransactionObjectRef = TransactionObjects[ObjectIndex];
		const FConcertExportedObject& ObjectUpdate = *SortedExportedObjects[ObjectIndex];

		UObject* TransactionObject = TransactionObjectRef.Obj;
		if (!TransactionObject)
		{
			continue;
		}

		if (ObjectUpdate.ObjectData.bIsPendingKill)
		{
			if (AActor* TransactionActor = Cast<AActor>(TransactionObject))
			{
				DeletedActorsForSelectionUpdate.Add(TransactionActor);
			}
			else if(UActorComponent* TransactionActorComponent = Cast<UActorComponent>(TransactionObject))
			{
				DeletedActorComponentForSelectionUpdate.Add(TransactionActorComponent);
			}
		}

#if WITH_EDITOR
		// Ensure any scene components have an up-to-date ComponentToWorld before invoking their PostEdit logic, as this 
		// update may have dirtied it, and PostEdit may try and read it (eg, to get the world position of an actor)
		if (USceneComponent* TransactionSceneComponent = Cast<USceneComponent>(TransactionObject))
		{
			TransactionSceneComponent->UpdateComponentToWorld();
		}

		// We need to manually call OnObjectPropertyChanged as PostEditUndo calls the PostEditChange version that skips it, but we have things that rely on it being called
		// For snapshot events this also triggers PostEditChange directly since we can skip the call to PostEditUndo
		for (const FConcertSerializedPropertyData& PropertyData : ObjectUpdate.PropertyDatas)
		{
			FProperty* TransactionProp = FindFProperty<FProperty>(TransactionObject->GetClass(), PropertyData.PropertyName);
			if (TransactionProp)
			{
				if (bIsSnapshot)
				{
					TGuardValue<bool> IsTransactingGuard(GIsTransacting, true);
					TransactionObject->PostEditChange();
				}

				FEditPropertyChain PropertyChain;
				PropertyChain.AddHead(TransactionProp);
				FPropertyChangedEvent PropertyChangedEvent(TransactionProp, bIsSnapshot ? EPropertyChangeType::Interactive : EPropertyChangeType::Unspecified);
				FPropertyChangedChainEvent PropertyChangedChainEvent (PropertyChain, PropertyChangedEvent);
				FCoreUObjectDelegates::OnPostObjectPropertyChanged.Broadcast(TransactionObject, PropertyChangedChainEvent);
			}
		}

		// Notify after changing everything
		const TSharedPtr<ITransactionObjectAnnotation>& TransactionAnnotation = TransactionAnnotations[ObjectIndex];
		if (TransactionAnnotation)
		{
			// Transaction annotations require us to invoke the redo flow (even for snapshots!) as that's the only thing that can apply the annotation
			TGuardValue<bool> IsTransactingGuard(GIsTransacting, true);
			TransactionObject->PostEditUndo(TransactionAnnotation);
		}
		else if (!bIsSnapshot)
		{
			TGuardValue<bool> IsTransactingGuard(GIsTransacting, true);
			TransactionObject->PostEditUndo();
		}

		// Notify the editor that a transaction happened, as some things rely on this being called
		// We need to call this ourselves as we aren't actually going through the full transaction redo that the editor hooks in to to generate these notifications
		if (!bIsSnapshot)
		{
			EditorTransactionNotification.HandleObjectTransacted(TransactionObject, ObjectUpdate, TransactionAnnotation);
		}
#endif
	}

#if WITH_EDITOR
	if (!bIsSnapshot)
	{
		EditorTransactionNotification.PostUndo();
	}
	
	// Ensure that any actors we restored from the dead are added back to the actors array of their owner level
	for (AActor* Actor : ResurrectedActors)
	{
		ConcertSyncClientUtil::AddActorToOwnerLevel(Actor);
	}
#endif

	DeselectActorsAndActorComponents(DeletedActorsForSelectionUpdate, DeletedActorComponentForSelectionUpdate);

	// TODO: This can sometimes cause deadlocks - need to investigate why
	if (bObjectsDeleted)
	{
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS, false);
	}

#if WITH_EDITOR
	if (GUnrealEd)
	{
		if (bIsSnapshot)
		{
			GUnrealEd->UpdatePivotLocationForSelection();
		}
		GUnrealEd->RedrawLevelEditingViewports();
	}
#endif
}

}	// namespace ConcertClientTransactionBridgeUtil

TUniquePtr<IConcertClientTransactionBridge> IConcertClientTransactionBridge::NewInstance()
{
	return MakeUnique<FConcertClientTransactionBridge>();
}

FConcertClientTransactionBridge::FConcertClientTransactionBridge()
	: bHasBoundUnderlyingLocalTransactionEvents(false)
	, bIgnoreLocalTransactions(false)
	, bIncludeEditorOnlyProperties(true)
	, bIncludeNonPropertyObjectData(true)
	, bIncludeAnnotationObjectChanges(GetDefault<UConcertSyncConfig>()->bIncludeAnnotationObjectChanges)
{
	FCoreDelegates::OnEndFrame.AddRaw(this, &FConcertClientTransactionBridge::OnEndFrame);
}

FConcertClientTransactionBridge::~FConcertClientTransactionBridge()
{
	StopBridge();
	FCoreDelegates::OnEndFrame.RemoveAll(this);
}

void FConcertClientTransactionBridge::SetIncludeEditorOnlyProperties(const bool InIncludeEditorOnlyProperties)
{
	bIncludeEditorOnlyProperties = InIncludeEditorOnlyProperties;
}

void FConcertClientTransactionBridge::SetIncludeNonPropertyObjectData(const bool InIncludeNonPropertyObjectData)
{
	bIncludeNonPropertyObjectData = InIncludeNonPropertyObjectData;
}

void FConcertClientTransactionBridge::SetIncludeAnnotationObjectChanges(const bool InIncludeAnnotationObjectChanges)
{
	bIncludeAnnotationObjectChanges = InIncludeAnnotationObjectChanges;
}

FOnConcertClientLocalTransactionSnapshot& FConcertClientTransactionBridge::OnLocalTransactionSnapshot()
{
	return OnLocalTransactionSnapshotDelegate;
}

FOnConcertClientLocalTransactionFinalized& FConcertClientTransactionBridge::OnLocalTransactionFinalized()
{
	return OnLocalTransactionFinalizedDelegate;
}

bool FConcertClientTransactionBridge::CanApplyRemoteTransaction() const
{
	return ConcertSyncClientUtil::CanPerformBlockingAction();
}

FOnApplyTransaction& FConcertClientTransactionBridge::OnApplyTransaction()
{
	return OnApplyTransactionDelegate;
}

FOnConcertConflictResolutionForPendingSend& FConcertClientTransactionBridge::OnConflictResolutionForPendingSend()
{
	return OnConflictResolutionForPendingSendDelegate;
}


void FConcertClientTransactionBridge::ApplyRemoteTransaction(const FConcertTransactionEventBase& InEvent, const FConcertSessionVersionInfo* InVersionInfo, const TArray<FName>& InPackagesToProcess, const FConcertLocalIdentifierTable* InLocalIdentifierTablePtr, const bool bIsSnapshot, FRemoteTransactionResult* ResultInfo)
{
	ConcertClientTransactionBridgeUtil::ProcessTransactionEvent(InEvent, InVersionInfo, InPackagesToProcess, InLocalIdentifierTablePtr, bIsSnapshot, FConcertSyncWorldRemapper(), bIncludeEditorOnlyProperties, ResultInfo);
}

void FConcertClientTransactionBridge::ApplyRemoteTransaction(const FConcertTransactionEventBase& InEvent, const FConcertSessionVersionInfo* InVersionInfo, const TArray<FName>& InPackagesToProcess, const FConcertLocalIdentifierTable* InLocalIdentifierTablePtr, const bool bIsSnapshot, const FConcertSyncWorldRemapper& ConcertSyncWorldRemapper, FRemoteTransactionResult* ResultInfo)
{
	ConcertClientTransactionBridgeUtil::ProcessTransactionEvent(InEvent, InVersionInfo, InPackagesToProcess, InLocalIdentifierTablePtr, bIsSnapshot, ConcertSyncWorldRemapper, bIncludeEditorOnlyProperties, ResultInfo);
}

bool& FConcertClientTransactionBridge::GetIgnoreLocalTransactionsRef()
{
	return bIgnoreLocalTransactions;
}

void FConcertClientTransactionBridge::HandleTransactionStateChanged(const FTransactionContext& InTransactionContext, const ETransactionStateEventType InTransactionState)
{
	if (bIgnoreLocalTransactions)
	{
		return;
	}

	{
		const TCHAR* TransactionStateString = TEXT("");
		switch (InTransactionState)
		{
#define ENUM_TO_STRING(ENUM)						\
		case ETransactionStateEventType::ENUM:		\
			TransactionStateString = TEXT(#ENUM);	\
				break;
		ENUM_TO_STRING(TransactionStarted)
		ENUM_TO_STRING(TransactionCanceled)
		ENUM_TO_STRING(TransactionFinalized)
		ENUM_TO_STRING(UndoRedoStarted)
		ENUM_TO_STRING(UndoRedoFinalized)
#undef ENUM_TO_STRING
		default:
			break;
		}

		UE_LOGF(LogConcert, VeryVerbose, "Transaction %ls (%ls): %ls", *InTransactionContext.TransactionId.ToString(), *InTransactionContext.OperationId.ToString(), TransactionStateString);
	}

	// Create, finalize, or remove an ongoing transaction
	if (InTransactionState == ETransactionStateEventType::TransactionStarted || InTransactionState == ETransactionStateEventType::UndoRedoStarted)
	{
		// Start a new ongoing transaction
		check(!OngoingTransactions.Contains(InTransactionContext.OperationId));
		OngoingTransactionsOrder.Add(InTransactionContext.OperationId);
		OngoingTransactions.Add(InTransactionContext.OperationId, FOngoingTransaction(InTransactionContext.Title, InTransactionContext.TransactionId, InTransactionContext.OperationId, InTransactionContext.PrimaryObject));
	}
	else if (InTransactionState == ETransactionStateEventType::TransactionFinalized || InTransactionState == ETransactionStateEventType::UndoRedoFinalized)
	{
		// Finalize an existing ongoing transaction
		FOngoingTransaction& OngoingTransaction = OngoingTransactions.FindChecked(InTransactionContext.OperationId);
		OngoingTransaction.CommonData.TransactionTitle = InTransactionContext.Title;
		OngoingTransaction.CommonData.PrimaryObject = InTransactionContext.PrimaryObject;
		OngoingTransaction.bIsFinalized = true;
	}
	else if (InTransactionState == ETransactionStateEventType::TransactionCanceled)
	{
		// We receive an object undo event before a transaction is canceled to restore the object to its original state
		// We need to keep this update if we notified of any snapshots for this transaction (to undo the snapshot changes), otherwise we can just drop this transaction as no changes have notified
		FOngoingTransaction& OngoingTransaction = OngoingTransactions.FindChecked(InTransactionContext.OperationId);
		if (OngoingTransaction.bHasNotifiedSnapshot)
		{
			// Finalize the transaction
			OngoingTransaction.CommonData.TransactionTitle = InTransactionContext.Title;
			OngoingTransaction.CommonData.PrimaryObject = InTransactionContext.PrimaryObject;
			OngoingTransaction.bIsFinalized = true;
			OngoingTransaction.FinalizedData.bWasCanceled = true;
		}
		else
		{
			// Note: We don't remove this from OngoingTransactionsOrder as we just skip transactions missing from the map (assuming they've been canceled).
			OngoingTransactions.Remove(InTransactionContext.OperationId);
		}
	}
}


namespace ConcertClientTransactionBridgeUtil
{
void LogTransactionEventDetails(UObject* InObject, const FTransactionObjectEvent& InTransactionEvent, ETransactionFilterResult FilterResult, bool bIsTracked)
{
	const TCHAR* ObjectEventString = TEXT("");
	switch (InTransactionEvent.GetEventType())
	{
#define ENUM_TO_STRING(ENUM)						\
		case ETransactionObjectEventType::ENUM:		\
			ObjectEventString = TEXT(#ENUM);		\
				break;
		ENUM_TO_STRING(UndoRedo)
		ENUM_TO_STRING(Finalized)
		ENUM_TO_STRING(Snapshot)
#undef ENUM_TO_STRING
		default:
			break;
	}

	UE_LOGF(LogConcert, VeryVerbose,
		   "%ls Transaction %ls (%ls, %ls):%ls %ls:%ls (%ls property changes, %ls object changes)",
		   (bIsTracked ? TEXT("Tracked") : TEXT("Untracked")),
		   *InTransactionEvent.GetTransactionId().ToString(),
		   *InTransactionEvent.GetOperationId().ToString(),
		   ObjectEventString,
		   (FilterResult == ETransactionFilterResult::ExcludeObject ? TEXT(" FILTERED OBJECT: ") : TEXT("")),
		   *InObject->GetClass()->GetName(),
		   *InObject->GetPathName(),
		   (InTransactionEvent.HasPropertyChanges() ? TEXT("has") : TEXT("no")),
		   (InTransactionEvent.HasNonPropertyChanges() ? TEXT("has") : TEXT("no"))
			);
}

UClass* GetModifiedClass(UObject* InObject)
{
	if (Cast<ALevelInstance>(InObject))
	{
		return AActor::StaticClass();
	}
	if (Cast<ULevelInstanceComponent>(InObject))
	{
		return ULevelInstanceComponent::StaticClass();
	}
	if (Cast<USceneComponent>(InObject))
	{
		return USceneComponent::StaticClass();
	}
	if (Cast<UPrimitiveComponent>(InObject))
	{
		return UPrimitiveComponent::StaticClass();
	}
	return nullptr;
}

ALevelInstance* GetLevelInstance(UObject* InObject)
{
	ALevelInstance* LevelInstance = Cast<ALevelInstance>(InObject);
	if (!LevelInstance && InObject)
	{
		UObject* Owner = InObject->GetOuter();
		LevelInstance = Cast<ALevelInstance>(Owner);
	}

	return LevelInstance;
}

void ApplyForAllRelevantObjects(UObject* InObject, TFunctionRef<void(UObject*)> InFunc, UClass* InClass = nullptr)
{
	if (ALevelInstance* LevelInstance = GetLevelInstance(InObject))
	{
		UClass* TheClass = InClass ? InClass : GetModifiedClass(InObject);

		// We should always have a class at this point.
		check(TheClass);
		TSet<AActor*> Actors;
		LevelInstance->EditorGetUnderlyingActors(Actors);
		for (AActor* Actor : Actors)
		{
			UObject* ObjectToApply = Actor->FindComponentByClass(TheClass);
			if (ObjectToApply)
			{
				InFunc(ObjectToApply);
				ApplyForAllRelevantObjects(Actor, InFunc, TheClass);
			}
		}
	}
}

void GetCanCreateOrRenameObject(UObject* InObject, bool& bOutCanCreateObject, bool& bOutCanRenameObject)
{
	if (const UActorComponent* Component = Cast<UActorComponent>(InObject))
	{
		bool bIsOwnedByChildActor = false;
		if (const AActor* ComponentOwner = Component->GetOwner())
		{
			bIsOwnedByChildActor = ComponentOwner->IsChildActor();
		}

		// Components that are managed by a construction script cannot be created
		bOutCanCreateObject = !bIsOwnedByChildActor && !Component->IsCreatedByConstructionScript();

		// Components that are managed by a native or Blueprint class cannot be renamed, so we only allow "instance" components to be created or renamed
		bOutCanRenameObject = !bIsOwnedByChildActor && Component->CreationMethod == EComponentCreationMethod::Instance;
	}
}

struct FTransactedObjectState
{
	using FOngoingTransaction = FConcertClientTransactionBridge::FOngoingTransaction;

	FName NewObjectPackageName;
	FName NewObjectName;
	FName NewObjectOuterPathName;
	FName NewObjectExternalPackageName;

	FWeakObjectPtr MainObjectPtr;
	TArray<const FProperty*> ExportedProperties;
	TSharedPtr<ITransactionObjectAnnotation> TransactionAnnotation;
	bool bUseSerializedAnnotationData;

	bool bIncludeNonPropertyObjectData = false;
	bool bIncludeEditorOnlyProperties = false;

    bool bCanCreateObject = true;
	bool bCanRenameObject = true;

	// Transaction event flags
	bool bHasIdOrPendingKillChanges = false;
	bool bHasPendingKillChange = false;
	bool bHasNonPropertyChanges = false;
	bool bIsSnapshot = false;

	FTransactedObjectState() = delete;
    FTransactedObjectState(UObject* InObject, const FTransactionObjectEvent& InTransactionEvent,
						   bool bInIncludeNonPropertyObjectData,
						   bool bInIncludeEditorOnlyProperties,
						   bool bIncludeAnnotationObjectChanges)
		: MainObjectPtr(InObject)
    {
		bIncludeNonPropertyObjectData =  bInIncludeNonPropertyObjectData;
		bIncludeEditorOnlyProperties = bInIncludeEditorOnlyProperties;

        ConcertClientTransactionBridgeUtil::GetCanCreateOrRenameObject(InObject, bCanCreateObject, bCanRenameObject);

        if (bCanRenameObject)
        {
            if (InTransactionEvent.HasOuterChange() || InTransactionEvent.HasExternalPackageChange())
            {
                NewObjectPackageName = InObject->GetPackage()->GetFName();
            }
            if (InTransactionEvent.HasNameChange())
            {
                NewObjectName = InObject->GetFName();
            }
            if (InTransactionEvent.HasOuterChange() && InObject->GetOuter())
            {
                NewObjectOuterPathName = FName(*InObject->GetOuter()->GetPathName()) ;
            }
            if (InTransactionEvent.HasExternalPackageChange() && InObject->GetExternalPackage())
            {
                NewObjectExternalPackageName = InObject->GetExternalPackage()->GetFName();
            }
        }

        ExportedProperties = ConcertSyncClientUtil::GetExportedProperties(InObject->GetClass(), InTransactionEvent.GetChangedProperties(), bIncludeEditorOnlyProperties);

		bHasIdOrPendingKillChanges = InTransactionEvent.HasIdOrPendingKillChanges();
		bHasPendingKillChange = InTransactionEvent.HasPendingKillChange();
		bHasNonPropertyChanges = InTransactionEvent.HasNonPropertyChanges(/*SerializationOnly*/true);
		bIsSnapshot = InTransactionEvent.GetEventType() == ETransactionObjectEventType::Snapshot;

        TransactionAnnotation = InTransactionEvent.GetAnnotation();
        bUseSerializedAnnotationData = !bIncludeAnnotationObjectChanges
            || !TransactionAnnotation
            || !TransactionAnnotation->SupportsAdditionalObjectChanges();
    }

	bool IsValid() const
    {
        const bool bObjectHasChangesToSend = bHasIdOrPendingKillChanges
            || (bIncludeNonPropertyObjectData && bHasNonPropertyChanges)
            || ExportedProperties.Num() > 0
            || (TransactionAnnotation && bUseSerializedAnnotationData);
        return bObjectHasChangesToSend;
    }

	void CaptureSnapshot(UObject* InObject, FConcertExportedObject* ObjectUpdatePtr)
	{
		if (TransactionAnnotation && bUseSerializedAnnotationData)
		{
			ObjectUpdatePtr->SerializedAnnotationData.Reset();
			FConcertSyncObjectWriter AnnotationWriter(nullptr, InObject, ObjectUpdatePtr->SerializedAnnotationData, bIncludeEditorOnlyProperties, true);
			TransactionAnnotation->Serialize(AnnotationWriter);
		}

		// Find or add an update for each property
		for (const FProperty* ExportedProperty : ExportedProperties)
		{
			FConcertSerializedPropertyData* PropertyDataPtr = ObjectUpdatePtr->PropertyDatas.FindByPredicate([ExportedProperty](FConcertSerializedPropertyData& PropertyData)
			{
				return ExportedProperty->GetFName() == PropertyData.PropertyName;
			});
			if (!PropertyDataPtr)
			{
				PropertyDataPtr = &ObjectUpdatePtr->PropertyDatas.AddDefaulted_GetRef();
				PropertyDataPtr->PropertyName = ExportedProperty->GetFName();
			}

			PropertyDataPtr->SerializedData.Reset();
			ConcertSyncClientUtil::SerializeProperty(nullptr, InObject, ExportedProperty, bIncludeEditorOnlyProperties, PropertyDataPtr->SerializedData);
		}
	}

	void CaptureFinalized(FConcertExportedObject& ObjectUpdate, FConcertLocalIdentifierTable& LocalIdentifierTable, const FConcertObjectId& ObjectId, UObject* InObject)
	{
		const bool bIsNewlyCreated = bHasPendingKillChange && ::IsValid(InObject);

		ObjectUpdate.ObjectId = ObjectId;
		ObjectUpdate.ObjectPathDepth = ConcertSyncClientUtil::GetObjectPathDepth(InObject);
		ObjectUpdate.ObjectData.bAllowCreate = bIsNewlyCreated && bCanCreateObject;
		ObjectUpdate.ObjectData.bResetExisting = bIsNewlyCreated;
		ObjectUpdate.ObjectData.bIsPendingKill = !::IsValid(InObject);
		ObjectUpdate.ObjectData.NewPackageName = NewObjectPackageName;
		ObjectUpdate.ObjectData.NewName = NewObjectName;
		ObjectUpdate.ObjectData.NewOuterPathName = NewObjectOuterPathName;
		ObjectUpdate.ObjectData.NewExternalPackageName = NewObjectExternalPackageName;


		if (TransactionAnnotation && bUseSerializedAnnotationData)
		{
			FConcertSyncObjectWriter AnnotationWriter(&LocalIdentifierTable, InObject, ObjectUpdate.SerializedAnnotationData, bIncludeEditorOnlyProperties, false);
			TransactionAnnotation->Serialize(AnnotationWriter);
		}

		if (bIncludeNonPropertyObjectData && bHasNonPropertyChanges)
		{
			// The 'non-property changes' refers to custom data added by a deriver UObject before and/or after the standard serialized data. Since this is a custom
			// data format, we don't know what changed, call the object to re-serialize this part, but still send the delta for the generic reflected properties (in RootPropertyNames).
			ConcertSyncClientUtil::SerializeObject(&LocalIdentifierTable, InObject, &ExportedProperties, bIncludeEditorOnlyProperties, ObjectUpdate.ObjectData.SerializedData);

			// Track which properties changed. Not used to apply the transaction on the receiving side, the object specific serialization function will be used for that, but
			// to be able to display, in the transaction detail view, which 'properties' changed in the transaction as transaction data is otherwise opaque to UI.
			for (const FProperty* ExportedProperty : ExportedProperties)
			{
				FConcertSerializedPropertyData& PropertyData = ObjectUpdate.PropertyDatas.AddDefaulted_GetRef();
				PropertyData.PropertyName = ExportedProperty->GetFName();
			}
		}
		else // Its possible to optimize the transaction payload, only sending a 'delta' update.
		{
			ConcertSyncClientUtil::SerializeProperties(&LocalIdentifierTable, InObject, ExportedProperties, bIncludeEditorOnlyProperties, ObjectUpdate.PropertyDatas);
		}
	}

	void TrackPackageChanges(UPackage* ChangedPackage, FOngoingTransaction& OngoingTransaction, const FTransactionObjectEvent& InTransactionEvent)
	{
		// Track which packages were changed
		OngoingTransaction.CommonData.ModifiedPackages.AddUnique(ChangedPackage->GetFName());

		// if there was an outer change, track that outer package
		if (InTransactionEvent.HasOuterChange())
		{
			FName OriginalOuterPackageName = *FPackageName::ObjectPathToPackageName(InTransactionEvent.GetOriginalObjectOuterPathName().ToString());
			OngoingTransaction.CommonData.ModifiedPackages.AddUnique(OriginalOuterPackageName);
		}

		// if there was an package change, track that package
		if (InTransactionEvent.HasExternalPackageChange())
		{
			FName OriginalPackageName = InTransactionEvent.GetOriginalObjectPackageName().IsNone() ? *FPackageName::ObjectPathToPackageName(InTransactionEvent.GetOriginalObjectOuterPathName().ToString()) : InTransactionEvent.GetOriginalObjectPackageName();
			OngoingTransaction.CommonData.ModifiedPackages.AddUnique(OriginalPackageName);
		}
	}

	bool HasDataToCapture() const
	{
		return (ExportedProperties.Num() > 0 || TransactionAnnotation.IsValid());
	}

	TArray<FConcertExportedObject> AddLevelSequenceObjectsForFinalized(FConcertLocalIdentifierTable& LocalIdentifierTable)
	{
		TArray<FConcertExportedObject> ExportedObjects;
		if (MainObjectPtr.IsValid())
		{
			auto Capture = [&LocalIdentifierTable, &ExportedObjects, this](UObject* InObject)
			{
				FConcertExportedObject& ExportedObject = ExportedObjects.AddDefaulted_GetRef();
				ExportedObject.bHasLevelInstanceObject = true;
				FConcertObjectId NewId = FConcertObjectId(InObject);
				CaptureFinalized(ExportedObject, LocalIdentifierTable, NewId, InObject);
			};
			ApplyForAllRelevantObjects(MainObjectPtr.Get(), Capture);
		}
		return ExportedObjects;
	}
};

using FTransactedObjectStateMap = TMap<FConcertObjectId, TPimplPtr<FTransactedObjectState> >;

void AppendLevelInstanceObjects(FConcertLocalIdentifierTable& LocalIdentifierTable, const FTransactedObjectStateMap& InObjectPtrs, TArray<FConcertExportedObject>& ExportedObjects)
{
	TArray<FConcertExportedObject> NewExportedObjects;
	for (FConcertExportedObject& ExportedObject : ExportedObjects)
	{
		const TPimplPtr<ConcertClientTransactionBridgeUtil::FTransactedObjectState>* ObjectState = InObjectPtrs.Find(ExportedObject.ObjectId);
		if (ObjectState && ObjectState->IsValid())
		{
			NewExportedObjects.Append(ObjectState->Get()->AddLevelSequenceObjectsForFinalized(LocalIdentifierTable));
		}
	}
	ExportedObjects.Append(NewExportedObjects);
}

}

bool FConcertClientTransactionBridge::CanHandleObjectTransacted(UObject* InObject, const FTransactionObjectEvent& InTransactionEvent) const
{
	if (bIgnoreLocalTransactions)
	{
		return false;
	}

	if (!bIncludeAnnotationObjectChanges && InTransactionEvent.GetObjectChangeCreatedBy() == ETransactionObjectChangeCreatedBy::TransactionAnnotation)
	{
		return false;
	}

	if (!bIncludeEditorOnlyProperties && InObject->IsEditorOnly())
	{
		return false;
	}

	return true;
}

void FConcertClientTransactionBridge::HandleObjectTransacted(UObject* InObject, const FTransactionObjectEvent& InTransactionEvent)
{
	LLM_SCOPE_BYTAG(Concert_ConcertClientTransactionBridge);
	SCOPED_CONCERT_TRACE(FConcertClientTransactionBridge_HandleObjectTransacted);

	if (!CanHandleObjectTransacted(InObject, InTransactionEvent))
	{
		return;
	}

	UPackage* ChangedPackage = InObject->GetOutermost();
	const ETransactionFilterResult FilterResult = ConcertClientTransactionBridgeUtil::ApplyTransactionFilters(TransactionFilters, InObject, ChangedPackage, InTransactionEvent);
	FOngoingTransaction* TrackedTransaction = OngoingTransactions.Find(InTransactionEvent.GetOperationId());

	// TODO: This needs to send both editor-only and non-editor-only payload
	// data to the server, which will forward only the correct part to cooked
	// and non-cooked clients
	{
		ConcertClientTransactionBridgeUtil::LogTransactionEventDetails(InObject, InTransactionEvent, FilterResult, TrackedTransaction != nullptr);
	}

	if (TrackedTransaction == nullptr)
	{
		// No tracked transaction so nothing to handle.
		return;
	}

	const FConcertObjectId ObjectId = FConcertObjectId(InTransactionEvent.GetOriginalObjectId(), InObject->GetFlags());
	FOngoingTransaction& OngoingTransaction = *TrackedTransaction;

	// If the object is excluded or exclude the whole transaction add it to the excluded list
	if (FilterResult != ETransactionFilterResult::IncludeObject)
	{
		OngoingTransaction.CommonData.bIsExcluded |= FilterResult == ETransactionFilterResult::ExcludeTransaction;
		OngoingTransaction.CommonData.ExcludedObjectUpdates.Add(ObjectId);
		return;
	}

	FOngoingTransaction::FTransactedObjectStatePtr TransactedObjectState = MakePimpl<ConcertClientTransactionBridgeUtil::FTransactedObjectState>(InObject, InTransactionEvent, bIncludeNonPropertyObjectData, bIncludeEditorOnlyProperties,  bIncludeAnnotationObjectChanges);

	if (!TransactedObjectState->IsValid())
	{
		// This object has no changes to send
		return;
	}

	TransactedObjectState->TrackPackageChanges(ChangedPackage, OngoingTransaction, InTransactionEvent);
	// Add this object change to its pending transaction
	if (InTransactionEvent.GetEventType() == ETransactionObjectEventType::Snapshot)
	{
		if (OnLocalTransactionSnapshotDelegate.IsBound() && TransactedObjectState->HasDataToCapture())
		{
			// Find or add an entry for this object
			auto CaptureForSnapshot = [&TransactedObjectState, &OngoingTransaction, InObject, &ObjectId](UObject* NewObject)
			{
				FConcertObjectId NewId = (NewObject == nullptr || NewObject == InObject) ? ObjectId : FConcertObjectId(NewObject);
				FConcertExportedObject* ObjectUpdatePtr = OngoingTransaction.SnapshotData.SnapshotObjectUpdates.FindByPredicate([&NewId](FConcertExportedObject& ObjectUpdate)
					{
						return ConcertSyncClientUtil::ObjectIdsMatch(NewId, ObjectUpdate.ObjectId);
					});

				if (!ObjectUpdatePtr)
				{
					ObjectUpdatePtr = &OngoingTransaction.SnapshotData.SnapshotObjectUpdates.AddDefaulted_GetRef();
					ObjectUpdatePtr->ObjectId = NewId;
					ObjectUpdatePtr->ObjectPathDepth = ConcertSyncClientUtil::GetObjectPathDepth(InObject);
					ObjectUpdatePtr->ObjectData.bIsPendingKill = !IsValid(InObject);
					ObjectUpdatePtr->bHasLevelInstanceObject = NewObject != InObject;
				}
				TransactedObjectState->CaptureSnapshot(NewObject, ObjectUpdatePtr);
			};

			CaptureForSnapshot(InObject);
			// Now traverse the object to see if we need to add any additional objects to the snapshot.
			//
			ConcertClientTransactionBridgeUtil::ApplyForAllRelevantObjects(InObject, CaptureForSnapshot);
		}
	}
	else if (OnLocalTransactionFinalizedDelegate.IsBound())
	{
		FConcertLocalIdentifierTable& LocalIdentifierTable = OngoingTransaction.FinalizedData.FinalizedLocalIdentifierTable;
		FConcertExportedObject& ObjectUpdate = OngoingTransaction.FinalizedData.FinalizedObjectUpdates.AddDefaulted_GetRef();
		TransactedObjectState->CaptureFinalized(ObjectUpdate, LocalIdentifierTable, ObjectId, InObject);

		// We don't scan for additional object to add to the level snapshot due to undo flow.  On undo, with level instances,
		// we cannot access actor list.
		//
		OngoingTransaction.TransactedStatePtrs.FindOrAdd(ObjectId,MoveTemp(TransactedObjectState));
	}
}

void FConcertClientTransactionBridge::ConditionalBindUnderlyingLocalTransactionEvents()
{
	LLM_SCOPE_BYTAG(Concert_ConcertClientTransactionBridge);

	if (bHasBoundUnderlyingLocalTransactionEvents)
	{
		return;
	}

#if WITH_EDITOR
	// If the bridge is created while a transaction is ongoing, add it as pending
	if (GUndo)
	{
		// Start a new pending transaction
		HandleTransactionStateChanged(GUndo->GetContext(), ETransactionStateEventType::TransactionStarted);
	}

	// Register Object Transaction events
	if (GUnrealEd)
	{
		if (UTransBuffer* TransBuffer = Cast<UTransBuffer>(GUnrealEd->Trans))
		{
			bHasBoundUnderlyingLocalTransactionEvents = true;
			TransBuffer->OnTransactionStateChanged().AddRaw(this, &FConcertClientTransactionBridge::HandleTransactionStateChanged);
			FCoreUObjectDelegates::OnObjectTransacted.AddRaw(this, &FConcertClientTransactionBridge::HandleObjectTransacted);
		}
	}
#endif
}

void FConcertClientTransactionBridge::StartBridge()
{
	if (GIsRunning)
	{
		ConditionalBindUnderlyingLocalTransactionEvents();
	}
	else
	{
		// Wait until the engine init has been completed to bind to transaction events.
		FCoreDelegates::OnFEngineLoopInitComplete.AddRaw(this, &FConcertClientTransactionBridge::OnEngineInitComplete);
	}
}


void FConcertClientTransactionBridge::StopBridge()
{
	FCoreDelegates::OnFEngineLoopInitComplete.RemoveAll(this);

	if (!bHasBoundUnderlyingLocalTransactionEvents)
	{
		return;
	}
	
#if WITH_EDITOR
	// Unregister Object Transaction events
	if (GUnrealEd)
	{
		if (UTransBuffer* TransBuffer = Cast<UTransBuffer>(GUnrealEd->Trans))
		{
			TransBuffer->OnTransactionStateChanged().RemoveAll(this);
		}
	}
	FCoreUObjectDelegates::OnObjectTransacted.RemoveAll(this);
#endif
	bHasBoundUnderlyingLocalTransactionEvents = false;
}

void FConcertClientTransactionBridge::OnEngineInitComplete()
{
	ConditionalBindUnderlyingLocalTransactionEvents();
}

void FConcertClientTransactionBridge::OnEndFrame()
{
	SCOPED_CONCERT_TRACE(FConcertClientTransactionBridge_OnEndFrame);

	for (auto OngoingTransactionsOrderIter = OngoingTransactionsOrder.CreateIterator(); OngoingTransactionsOrderIter; ++OngoingTransactionsOrderIter)
	{
		FOngoingTransaction* OngoingTransactionPtr = OngoingTransactions.Find(*OngoingTransactionsOrderIter);
		if (!OngoingTransactionPtr)
		{
			// Missing transaction, must have been canceled...
			OngoingTransactionsOrderIter.RemoveCurrent();
			continue;
		}


		if (OngoingTransactionPtr->bIsFinalized)
		{
			ConcertClientTransactionBridgeUtil::AppendLevelInstanceObjects(OngoingTransactionPtr->FinalizedData.FinalizedLocalIdentifierTable, OngoingTransactionPtr->TransactedStatePtrs, OngoingTransactionPtr->FinalizedData.FinalizedObjectUpdates);

			OnLocalTransactionFinalizedDelegate.Broadcast(OngoingTransactionPtr->CommonData, OngoingTransactionPtr->FinalizedData);

			OngoingTransactions.Remove(OngoingTransactionPtr->CommonData.OperationId);
			OngoingTransactionsOrderIter.RemoveCurrent();
			continue;
		}
		else if (OngoingTransactionPtr->SnapshotData.SnapshotObjectUpdates.Num() > 0)
		{
			OnLocalTransactionSnapshotDelegate.Broadcast(OngoingTransactionPtr->CommonData, OngoingTransactionPtr->SnapshotData);

			OngoingTransactionPtr->bHasNotifiedSnapshot = true;
			OngoingTransactionPtr->SnapshotData.SnapshotObjectUpdates.Reset();
		}
	}
}

void FConcertClientTransactionBridge::RegisterTransactionFilter(FName FilterName, FOnFilterTransactionDelegate FilterDelegate)
{
	LLM_SCOPE_BYTAG(Concert_ConcertClientTransactionBridge);
	check(TransactionFilters.Find(FilterName) == nullptr);
	TransactionFilters.Add(FilterName, MoveTemp(FilterDelegate));
}

void FConcertClientTransactionBridge::UnregisterTransactionFilter(FName FilterName)
{
	TransactionFilters.FindAndRemoveChecked(FilterName);
}

#undef LOCTEXT_NAMESPACE
