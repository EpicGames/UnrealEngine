// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IConcertClientTransactionBridge.h"
#include "Misc/ITransaction.h"
#include "ConcertSyncArchives.h"

namespace ConcertClientTransactionBridgeUtil
{
	struct FTransactedObjectState;
}

class FConcertClientTransactionBridge : public IConcertClientTransactionBridge
{
public:
	FConcertClientTransactionBridge();
	virtual ~FConcertClientTransactionBridge();

	//~ IConcertClientTransactionBridge interface
	virtual void SetIncludeEditorOnlyProperties(const bool InIncludeEditorOnlyProperties) override;
	virtual void SetIncludeNonPropertyObjectData(const bool InIncludeNonPropertyObjectData) override;
	virtual void SetIncludeAnnotationObjectChanges(const bool InIncludeAnnotationObjectChanges) override;
	virtual FOnConcertClientLocalTransactionSnapshot& OnLocalTransactionSnapshot() override;
	virtual FOnConcertClientLocalTransactionFinalized& OnLocalTransactionFinalized() override;
	virtual bool CanApplyRemoteTransaction() const override;
	virtual FOnApplyTransaction& OnApplyTransaction() override;
	virtual FOnConcertConflictResolutionForPendingSend& OnConflictResolutionForPendingSend() override;

	virtual void ApplyRemoteTransaction(const FConcertTransactionEventBase& InEvent, const FConcertSessionVersionInfo* InVersionInfo, const TArray<FName>& InPackagesToProcess, const FConcertLocalIdentifierTable* InLocalIdentifierTablePtr, const bool bIsSnapshot, FRemoteTransactionResult* ResultInfo = nullptr) override;
	virtual void ApplyRemoteTransaction(const FConcertTransactionEventBase& InEvent, const FConcertSessionVersionInfo* InVersionInfo, const TArray<FName>& InPackagesToProcess, const FConcertLocalIdentifierTable* InLocalIdentifierTablePtr, const bool bIsSnapshot, const FConcertSyncWorldRemapper& ConcertSyncWorldRemapper, FRemoteTransactionResult* ResultInfo = nullptr) override;
	virtual bool& GetIgnoreLocalTransactionsRef() override;

	virtual void RegisterTransactionFilter(FName FilterName, FOnFilterTransactionDelegate FilterDelegate) override;
	virtual void UnregisterTransactionFilter(FName FilterName) override;

	virtual void StartBridge() final;
	virtual void StopBridge() final;

private:
	/** Called to handle a transaction state change */
	void HandleTransactionStateChanged(const FTransactionContext& InTransactionContext, const ETransactionStateEventType InTransactionState);

	/** Confirms that we can handle the transaction event. */
	bool CanHandleObjectTransacted(UObject *InObject, const FTransactionObjectEvent& InTransactionEvent) const;

	/** Called to handle an object being transacted */
	void HandleObjectTransacted(UObject* InObject, const FTransactionObjectEvent& InTransactionEvent);

	/** Attempt to bind to the underlying local transaction events, if they are available and have not yet been bound */
	void ConditionalBindUnderlyingLocalTransactionEvents();

	/** Called once Engine initialization has finished, to bind to the underlying local transaction events */
	void OnEngineInitComplete();

	/** Called at the end of the frame to notify of any transaction updates */
	void OnEndFrame();

	struct FOngoingTransaction
	{
		FOngoingTransaction(const FText InTransactionTitle, const FGuid& InTransactionId, const FGuid& InOperationId, UObject* InPrimaryObject)
			: CommonData(InTransactionTitle, InTransactionId, InOperationId, InPrimaryObject)
		{
		}

		FConcertClientLocalTransactionCommonData CommonData;
		FConcertClientLocalTransactionSnapshotData SnapshotData;
		FConcertClientLocalTransactionFinalizedData FinalizedData;

		using FTransactedObjectStatePtr = TPimplPtr<ConcertClientTransactionBridgeUtil::FTransactedObjectState>;
		TMap<FConcertObjectId, FTransactedObjectStatePtr> TransactedStatePtrs;

		bool bIsFinalized = false;
		bool bHasNotifiedSnapshot = false;
	};

	friend struct ConcertClientTransactionBridgeUtil::FTransactedObjectState;

	/** Array of transaction IDs in the order they should be notified (maps to OngoingTransactions, although canceled transactions may be missing from the map) */
	TArray<FGuid> OngoingTransactionsOrder;

	/** Map of transaction IDs to the transaction that may be notified in the future */
	TMap<FGuid, FOngoingTransaction> OngoingTransactions;

	/** Map of named transaction filters that can override what is included / excluded by transaction bridge*/
	TMap<FName, FOnFilterTransactionDelegate> TransactionFilters;

	/** Called when an ongoing transaction is updated via a snapshot */
	FOnConcertClientLocalTransactionSnapshot OnLocalTransactionSnapshotDelegate;

	/** Called when an transaction is finalized */
	FOnConcertClientLocalTransactionFinalized OnLocalTransactionFinalizedDelegate;

	/** Delegeate called when a inbound transaction conflicts with a pending transaction to send.*/
	FOnConcertConflictResolutionForPendingSend OnConflictResolutionForPendingSendDelegate;

	/** Called when we are about to apply a transaction. */
	FOnApplyTransaction OnApplyTransactionDelegate;

	/** True if we have managed to bind to the underlying local transaction events, as they may not have been ready when this instance was started */
	bool bHasBoundUnderlyingLocalTransactionEvents;

	/** Flag to ignore transaction state change event, used when we do not want to record transaction we generate ourselves */
	bool bIgnoreLocalTransactions;

	/** Include non-cooked properties in object serialization */
	bool bIncludeEditorOnlyProperties;

	/** Include non-property object data in updates */
	bool bIncludeNonPropertyObjectData;

	/** Include object changes that have been generated via a transaction annotation (where possible) */
	bool bIncludeAnnotationObjectChanges;

	FConcertSyncWorldRemapper WorldRemapper = FConcertSyncWorldRemapper();
};
