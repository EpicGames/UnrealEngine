// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertClientTransactionManager.h"

#include "ConcertSyncSessionTypes.h"
#include "ConcertTransactionEvents.h"
#include "IConcertSession.h"
#include "ConcertSyncClientLiveSession.h"
#include "ConcertSyncSessionDatabase.h"
#include "ConcertLogGlobal.h"
#include "ConcertSyncSettings.h"
#include "ConcertSyncClientUtil.h"
#include "IConcertSyncClient.h"
#include "Scratchpad/ConcertScratchpad.h"

#include "Algo/AnyOf.h"
#include "Components/SceneComponent.h"
#include "Engine/World.h"
#include "Engine/Level.h"
#include "GameFramework/Actor.h"
#include "Internationalization/Internationalization.h"
#include "Misc/ScopedSlowTask.h"
#include "RenderingThread.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Package.h"
#include "UObject/UObjectHash.h"

#if WITH_EDITOR
	#include "Editor.h"
	#include "UnrealEdGlobals.h"
	#include "Editor/UnrealEdEngine.h"
	#include "Editor/TransBuffer.h"
#endif

#define LOCTEXT_NAMESPACE "ConcertClientTransactionManager"

FConcertClientTransactionManager::FConcertClientTransactionManager(TSharedRef<FConcertSyncClientLiveSession> InLiveSession, IConcertClientTransactionBridge* InTransactionBridge)
	: LiveSession(InLiveSession)
	, TransactionBridge(InTransactionBridge)
{
	check(LiveSession->IsValidSession());
	check(EnumHasAnyFlags(LiveSession->GetSessionFlags(), EConcertSyncSessionFlags::EnableTransactions));
	check(TransactionBridge);

	if (EnumHasAnyFlags(LiveSession->GetSessionFlags(), EConcertSyncSessionFlags::ShouldSendTransactionSnapshots))
	{
		TransactionBridge->OnLocalTransactionSnapshot().AddRaw(this, &FConcertClientTransactionManager::HandleLocalTransactionSnapshot);
	}
	TransactionBridge->OnLocalTransactionFinalized().AddRaw(this, &FConcertClientTransactionManager::HandleLocalTransactionFinalized);

	// Snapshots event are handled directly, finalized events are handled by the workspace (which calls HandleRemoteTransaction)
	LiveSession->GetSession().RegisterCustomEventHandler<FConcertTransactionSnapshotEvent>(this, &FConcertClientTransactionManager::HandleTransactionEvent<FConcertTransactionSnapshotEvent>);
	LiveSession->GetSession().RegisterCustomEventHandler<FConcertTransactionRejectedEvent>(this, &FConcertClientTransactionManager::HandleTransactionRejectedEvent);
}

FConcertClientTransactionManager::~FConcertClientTransactionManager()
{
	TransactionBridge->OnLocalTransactionSnapshot().RemoveAll(this);
	TransactionBridge->OnLocalTransactionFinalized().RemoveAll(this);

	LiveSession->GetSession().UnregisterCustomEventHandler<FConcertTransactionSnapshotEvent>(this);
	LiveSession->GetSession().UnregisterCustomEventHandler<FConcertTransactionRejectedEvent>(this);
}

bool FConcertClientTransactionManager::HasSessionChanges() const
{
	bool bHasLiveTransactions = false;
	LiveSession->GetSessionDatabase().EnumeratePackageNamesWithLiveTransactions([&bHasLiveTransactions](const FName PackageName)
	{
		bHasLiveTransactions = true;
		return false; // Stop enumeration
	});
	return bHasLiveTransactions;
}

bool FConcertClientTransactionManager::HasLiveTransactionSupport(class UPackage* InPackage) const
{
	const UConcertSyncConfig* SyncConfig = GetDefault<UConcertSyncConfig>();
	// If we have no include object filters everything has live transaction support
	if (SyncConfig->IncludeObjectClassFilters.Num() == 0)
	{
		return true;
	}

	// validate that one of the package top level object is one of the allowed outer for the include filters
	for (const FTransactionClassFilter& TransactionFilter : SyncConfig->IncludeObjectClassFilters)
	{
		UClass* TransactionOuterClass = TransactionFilter.ObjectOuterClass.TryLoadClass<UObject>();
		if (TransactionOuterClass && FindObjectWithOuter(InPackage, TransactionOuterClass))
		{
			return true;
		}
	}
	return false;
}

void FConcertClientTransactionManager::ReplayAllTransactions()
{
	TArray<int64> LiveTransactionEventIds;
	LiveSession->GetSessionDatabase().GetLiveTransactionEventIds(LiveTransactionEventIds);

	if (LiveTransactionEventIds.Num() > 0)
	{
		FScopedSlowTask SlowTask(LiveTransactionEventIds.Num(), LOCTEXT("ReplayingTransactions", "Replaying Transactions..."));
		SlowTask.MakeDialogDelayed(1.0f);

		FPendingTransactionToProcessContext TransactionContext;
		TransactionContext.bIsRequired = true;

		FConcertSyncTransactionEvent TransactionEvent;
		for (int64 TransactionEventId : LiveTransactionEventIds)
		{
			SlowTask.EnterProgressFrame(1.0f, FText::Format(LOCTEXT("ReplayingTransactionFmt", "Replaying Transaction {0}"), TransactionEventId));

			if (LiveSession->GetSessionDatabase().GetTransactionEvent(TransactionEventId, TransactionEvent))
			{
				PendingTransactionsToProcess.Emplace(TransactionContext, FConcertTransactionFinalizedEvent::StaticStruct(), &TransactionEvent.Transaction);
			}
		}
	}
}

void FConcertClientTransactionManager::ReplayTransactions(const FName InPackageName)
{
	TArray<int64> LiveTransactionEventIds;
	LiveSession->GetSessionDatabase().GetLiveTransactionEventIdsForPackage(InPackageName, LiveTransactionEventIds);

	if (LiveTransactionEventIds.Num() > 0)
	{
		FScopedSlowTask SlowTask(LiveTransactionEventIds.Num(), FText::Format(LOCTEXT("ReplayingTransactionsForPackageFmt", "Replaying Transactions for {0}..."), FText::FromName(InPackageName)));
		SlowTask.MakeDialogDelayed(1.0f);

		FPendingTransactionToProcessContext TransactionContext;
		TransactionContext.bIsRequired = true;
		TransactionContext.PackagesToProcess.Add(InPackageName);

		int64 PackageEventId = 0;
		LiveSession->GetSessionDatabase().GetPackageHeadEventFromName(InPackageName, PackageEventId);

		FConcertSyncActivity PackageActivity;
		LiveSession->GetSessionDatabase().GetActivityForEvent(PackageEventId, EConcertSyncActivityEventType::Package, PackageActivity);

		FConcertSyncTransactionEvent TransactionEvent;
		for (int64 TransactionEventId : LiveTransactionEventIds)
		{
			FConcertSyncActivity TransactionActivity;
			LiveSession->GetSessionDatabase().GetActivityForEvent(TransactionEventId, EConcertSyncActivityEventType::Transaction, TransactionActivity);
			if (TransactionActivity.ActivityId > PackageActivity.ActivityId)
			{
				SlowTask.EnterProgressFrame(1.0f, FText::Format(
												LOCTEXT("ReplayingTransactionForPackageFmt",
														"Replaying Transaction {0} for {1}"),
												TransactionEventId, FText::FromName(InPackageName)));

				if (LiveSession->GetSessionDatabase().GetTransactionEvent(TransactionEventId, TransactionEvent))
				{
					PendingTransactionsToProcess.Emplace(TransactionContext, FConcertTransactionFinalizedEvent::StaticStruct(), &TransactionEvent.Transaction);
				}
			}
		}
	}
}

void FConcertClientTransactionManager::HandleRemoteTransaction(const FGuid& InSourceEndpointId, const int64 InTransactionEventId, const bool bApply)
{
	// Ignore this transaction if we generated it
	if (InSourceEndpointId == LiveSession->GetSession().GetSessionClientEndpointId())
	{
		return;
	}

	if (!bApply)
	{
		return;
	}

	FConcertSyncTransactionEvent TransactionEvent;
	if (LiveSession->GetSessionDatabase().GetTransactionEvent(InTransactionEventId, TransactionEvent))
	{
		FPendingTransactionToProcessContext TransactionContext;
		TransactionContext.bIsRequired = true;
		PendingTransactionsToProcess.Emplace(TransactionContext, FConcertTransactionFinalizedEvent::StaticStruct(), &TransactionEvent.Transaction);
	}
}

void FConcertClientTransactionManager::ProcessPending()
{
	if (PendingTransactionsToProcess.Num() > 0)
	{
		if (CanProcessTransactionEvent())
		{
			for (auto It = PendingTransactionsToProcess.CreateIterator(); It; ++It)
			{
				// PendingTransaction is moved out of the array since `ProcessTransactionEvent` can add more PendingTransactions through loading packages which would make a reference dangle
				FPendingTransactionToProcess PendingTransaction = MoveTemp(*It);
				ProcessTransactionEvent(PendingTransaction.Context, PendingTransaction.EventData);
				It.RemoveCurrent();
			}
		}
		else
		{
			PendingTransactionsToProcess.RemoveAll([](const FPendingTransactionToProcess& PendingTransaction)
			{
				return !PendingTransaction.Context.bIsRequired;
			});
		}
	}

	if (CanSendTransactionEvents())
	{
		SendPendingTransactionEvents();
	}

}

template <typename EventType>
void FConcertClientTransactionManager::HandleTransactionEvent(const FConcertSessionContext& InEventContext, const EventType& InEvent)
{
	static_assert(TIsDerivedFrom<EventType, FConcertTransactionEventBase>::IsDerived, "HandleTransactionEvent can only be used with types deriving from FConcertTransactionEventBase");

	FPendingTransactionToProcessContext TransactionContext;
	TransactionContext.bIsRequired = EnumHasAnyFlags(InEventContext.MessageFlags, EConcertMessageFlags::ReliableOrdered);

	PendingTransactionsToProcess.Emplace(TransactionContext, EventType::StaticStruct(), &InEvent);
}

void FConcertClientTransactionManager::HandleTransactionRejectedEvent(const FConcertSessionContext& InEventContext, const FConcertTransactionRejectedEvent& InEvent)
{
#if WITH_EDITOR
	UTransBuffer* TransBuffer = GUnrealEd ? Cast<UTransBuffer>(GUnrealEd->Trans) : nullptr;
	if (TransBuffer == nullptr)
	{
		return;
	}

	// For this undo operation, squelch the notification, also prevent us from recording
	IConcertClientTransactionBridge::FScopedIgnoreLocalTransaction IgnoreLocalTransaction(*TransactionBridge);
	bool bOrigSquelchTransactionNotification = GEditor && GEditor->bSquelchTransactionNotification;
	if (GEditor)
	{
		GEditor->bSquelchTransactionNotification = true;
	}

	// if the transaction to undo is the current one, end it.
	if (GUndo && GUndo->GetContext().TransactionId == InEvent.TransactionId)
	{
		// Cancel doesn't entirely do what we want here as it will just, remove the current transaction without restoring object state
		// This shouldn't happen however, since we only undo finalized transaction
		ensureMsgf(false, TEXT("Received a Concert undo request for an ongoing transaction."));
		TransBuffer->End();
		TransBuffer->Undo(false);
	}
	// Otherwise undo operations until the requested transaction has been undone.
	else
	{
		int32 ReversedQueueIndex = TransBuffer->FindTransactionIndex(InEvent.TransactionId);
		if (ReversedQueueIndex != INDEX_NONE)
		{
			ReversedQueueIndex = TransBuffer->GetQueueLength() - TransBuffer->GetUndoCount() - ReversedQueueIndex;
			int32 UndoCount = 0;

			// if we get a positive number, then we need to undo
			if (ReversedQueueIndex > 0)
			{
				while (UndoCount < ReversedQueueIndex)
				{
					TransBuffer->Undo();
					++UndoCount;
				}
			}
			// Otherwise we need to redo, as the transaction has already been undone
			else
			{
				ReversedQueueIndex = -ReversedQueueIndex + 1;
				while (UndoCount < ReversedQueueIndex)
				{
					TransBuffer->Redo();
					++UndoCount;
				}
			}
		}
	}

	if (GEditor)
	{
		GEditor->bSquelchTransactionNotification = bOrigSquelchTransactionNotification;
	}
#endif
}

void FConcertClientTransactionManager::NotifyUserOfSendConflict(const FConcertConflictDescriptionBase& ConflictDescription)
{
	TransactionBridge->OnConflictResolutionForPendingSend().Broadcast(ConflictDescription);
}

bool FConcertClientTransactionManager::CanProcessTransactionEvent() const
{
	const bool bIsSuspended = LiveSession->GetSession().GetSendReceiveState() == EConcertSendReceiveState::SendOnly;
	return TransactionBridge->CanApplyRemoteTransaction() && !bIsSuspended;
}

bool FConcertClientTransactionManager::CanSendTransactionEvents() const
{
	const bool bCanSend = LiveSession->GetSession().GetSendReceiveState() != EConcertSendReceiveState::ReceiveOnly;
	return bCanSend;
}

void FConcertClientTransactionManager::ProcessTransactionEvent(const FPendingTransactionToProcessContext& InContext, const FStructOnScope& InEvent)
{
	const FConcertTransactionEventBase& TransactionEvent = *(const FConcertTransactionEventBase*)InEvent.GetStructMemory();
	if (!ShouldProcessTransactionEvent(TransactionEvent, InContext.bIsRequired))
	{
		UE_LOGF(LogConcert, VeryVerbose, "Dropping transaction for '%ls' (index %d) as it arrived out-of-order", *TransactionEvent.TransactionId.ToString(), TransactionEvent.TransactionUpdateIndex);
		return;
	}

#define PROCESS_OBJECT_UPDATE_EVENT(EventName)																\
	if (InEvent.GetStruct() == FConcert##EventName::StaticStruct())											\
	{																										\
		return Process##EventName(InContext, static_cast<const FConcert##EventName&>(TransactionEvent));	\
	}

	PROCESS_OBJECT_UPDATE_EVENT(TransactionFinalizedEvent)
	PROCESS_OBJECT_UPDATE_EVENT(TransactionSnapshotEvent)

#undef PROCESS_OBJECT_UPDATE_EVENT
}

void FConcertConflictDescriptionAggregate::AddObjectRemoved(const FConcertObjectId& ObjectIdRemoved)
{
	UE_LOGF(LogConcert, Warning, "Object %ls conflicts with in-bound transaction and has been removed}.",
		   *ObjectIdRemoved.ObjectName.ToString());
	ObjectsRemoved.Add(ObjectIdRemoved);
}

void FConcertConflictDescriptionAggregate::AddObjectRenamed(const FConcertObjectId& OldObjectId, const FConcertObjectId& NewObjectId)
{
	if (!ConcertSyncClientUtil::ObjectIdsMatch(OldObjectId, NewObjectId) &&
		OldObjectId.ObjectName.ToString() != NewObjectId.ObjectName.ToString())
	{
		UE_LOGF(LogConcert, Warning, "Object %ls conflicts with in-bound transaction and has been renamed to %ls.",
			   *OldObjectId.ObjectName.ToString(), *NewObjectId.ObjectName.ToString());
		ObjectsRenamed.Add({OldObjectId,NewObjectId});
	}
}

FText FConcertConflictDescriptionAggregate::GetConflictTitle() const
{
	return FText::Format(
		LOCTEXT("TransactionConflict", "Inbound transaction conflicts with transactions for pending send. Renamed {0} objects and removed {1} objects."),
		ObjectsRenamed.Num(), ObjectsRemoved.Num());
}

FText FConcertConflictDescriptionAggregate::GetConflictDetails() const
{
	FTextBuilder RunningTextObject;
	if (ObjectsRemoved.Num())
	{
		RunningTextObject.AppendLine(LOCTEXT("TransactionConflictRemovedObjects", "Removed Objects:"));
		for (const FConcertObjectId& RemovedObject : ObjectsRemoved)
		{
			RunningTextObject.AppendLine(FText::FromString(RemovedObject.ObjectName.ToString()));
		}
	}

	if (ObjectsRenamed.Num())
	{
		RunningTextObject.AppendLine(LOCTEXT("TransactionConflictRenamedObjects", "Renamed Objects:"));
		for (const TPair<FConcertObjectId, FConcertObjectId>& Item : ObjectsRenamed)
		{
			RunningTextObject.AppendLineFormat(LOCTEXT("ConflictRenamedObject", "{0} -> {1}"),
											   FText::FromString(Item.Get<0>().ObjectName.ToString()),
											   FText::FromString(Item.Get<1>().ObjectName.ToString()));
		}
	}
	return RunningTextObject.ToText();
}

namespace UE::ConcertClientTransactionManager::Private
{
/**
 * Given a UObject to rename we either need to rename this object or the outer for the object. Transactions can refer to
 * Component.Property modification. However, modifying the component does not address the actual conflicting object.
 * Instead it is more appropriate to resolve to the parent actor and rename that actor so that it does not globally
 * conflict anymore once the transaction has been sent to the server.
 *
 * @param InObject - UObject to compare.
 */
UObject* GetRealObjectToRename(UObject* InObject)
{
	auto IsAValidForRename = [](UObject* Obj)
	{
		return IsValid(Obj) && (Obj->IsA<AActor>() || Obj->IsA<UActorComponent>());
	};
	if (IsAValidForRename(InObject))
	{
		if (const UActorComponent* Component = Cast<UActorComponent>(InObject))
		{
			if (IsAValidForRename(Component->GetOwner()))
			{
				return Component->GetOwner();
			}
		}
		return InObject;
	}
	return nullptr;
}

/**
 * This structure holds the renaming as applied to an FConcertObjectId so that we can update any tables of the
 * underlying UObject.
 */
struct FRenameObjectResult
{
	FConcertObjectId OldObjectId;
	FConcertObjectId NewObjectId = {};

	FConcertObjectId OldParentObjectId = {};
	FConcertObjectId NewParentObjectId = {};
};

/**
 * Apply the rename to the UObject using unique identifiers to avoid any future conflict.
 */
void MakeObjectUniqueToAvoidCollision(UObject* InObject)
{
	FString NewName = InObject->GetFName().ToString() + FGuid::NewGuid().ToString(EGuidFormats::Short);
	InObject->Rename(*NewName, nullptr, REN_NonTransactional);
}

/**
 * Given the ObjectId to fixup apply that rename to the resolve UObject and then return the old / new FConcertObjectId
 * so that any caller can update corresponding data.
 */
FRenameObjectResult RenameObjectsToAvoidCollision(const FConcertObjectId& InObjectIdToRename)
{
	UObject* ObjectToRename = ConcertSyncClientUtil::GetObject(InObjectIdToRename, FName(), FName(), FName(), FSoftObjectPath(), false).Obj;
	// The object does not exist or is marked for GC.
	if (!IsValid(ObjectToRename))
	{
		return {InObjectIdToRename};
	}

	UObject* TargetObjectToRename = GetRealObjectToRename(ObjectToRename);
	if (TargetObjectToRename)
	{
		FConcertObjectId TargetObjectId(TargetObjectToRename);
		MakeObjectUniqueToAvoidCollision(TargetObjectToRename);
		if (ObjectToRename != TargetObjectToRename)
		{
			return {InObjectIdToRename, FConcertObjectId(ObjectToRename),
					TargetObjectId, FConcertObjectId(TargetObjectToRename)};
		}
		else
		{
			return {InObjectIdToRename, FConcertObjectId(ObjectToRename)};
		}
	}

	return {};
}
}

void FConcertClientTransactionManager::FixupObjectIdsInPendingSend(
	const FConcertObjectId& OldObjectId, const FConcertObjectId& NewObjectId, FConcertConflictDescriptionAggregate& ConflictAggregate)
{
	/** If we didn't receive a valid OldObjectId then there is nothing to rename. */
	if (!OldObjectId.IsValid())
	{
		return;
	}

	auto FixNames = [&OldObjectId, &NewObjectId](FPendingTransactionToSend* ToSend)
	{
		if (!ToSend)
		{
			return;
		}
		for (FConcertExportedObject& ExportedObject : ToSend->FinalizedData.FinalizedObjectUpdates)
		{
			if (ConcertSyncClientUtil::ObjectIdsMatch(ExportedObject.ObjectId,OldObjectId))
			{
				ExportedObject.ObjectId = NewObjectId;
			}
		}
	};

	TSet<FGuid>* TransactionsWithObject = NameLookupForPendingTransactionsToSend.Find(OldObjectId);

	if (!TransactionsWithObject)
	{
		return;
	}

	/*
	 * If the new object id is not valid then the target object is pending GC or no longer exists. So it should not
	 * be in our tables anymore.
	 */
	const bool bShouldRemove = !NewObjectId.IsValid();
	if (bShouldRemove)
	{
		ConflictAggregate.AddObjectRemoved(OldObjectId);
		for (const FGuid& Guid : *TransactionsWithObject)
		{
			RemovePendingToSend(Guid);
		}
	}
	else
	{
		ConflictAggregate.AddObjectRenamed(OldObjectId, NewObjectId);
		for (const FGuid& Guid : *TransactionsWithObject)
		{
			FixNames(PendingTransactionsToSend.Find(Guid));
		}
	}

	if (!bShouldRemove)
	{
		// Move the renamed FConcertObjectId to a new tracking table entry.
		NameLookupForPendingTransactionsToSend.Add(NewObjectId, *TransactionsWithObject);
	}
	// Remove the old entry named entry.
	NameLookupForPendingTransactionsToSend.Remove(OldObjectId);
}

void FConcertClientTransactionManager::CheckEventForSendConflicts(const FConcertTransactionFinalizedEvent& InEvent)
{
	if (CanSendTransactionEvents())
	{
		return;
	}

	auto IsObjectCreated = [this](const FGuid& Id)
	{
		if (FPendingTransactionToSend* PendingToSendPtr = PendingTransactionsToSend.Find(Id))
		{
			for (const FConcertExportedObject& Exported : PendingToSendPtr->FinalizedData.FinalizedObjectUpdates)
			{
				if (Exported.ObjectData.bAllowCreate)
				{
					return true;
				}
			}
		}
		return false;
	};

	FConcertConflictDescriptionAggregate ConflictAggregate;
	auto LookupAndFix = [this, &ConflictAggregate, &IsObjectCreated](const FConcertExportedObject& ExportedObject)
	{
		const FConcertObjectId& ObjectId = ExportedObject.ObjectId;
		if (TSet<FGuid>* NamedPrimary = NameLookupForPendingTransactionsToSend.Find(ObjectId))
		{
			// If the object is *new* during our receive only session then we need to go find every instance and
			// redo the references.
			if (Algo::AnyOf(*NamedPrimary, IsObjectCreated))
			{
				// This exported object is from a create. Therefore the conflict was from a create.
				// fixup the conflict by renaming the object.
				//
				UE::ConcertClientTransactionManager::Private::FRenameObjectResult Result =
					UE::ConcertClientTransactionManager::Private::RenameObjectsToAvoidCollision(ObjectId);
				FixupObjectIdsInPendingSend(Result.OldParentObjectId, Result.NewParentObjectId, ConflictAggregate);
				FixupObjectIdsInPendingSend(Result.OldObjectId, Result.NewObjectId, ConflictAggregate);
			}
			else
			{
				// Otherwise designate this change has a conflict and must be dropped from the pending list.
				// TODO UE-170906: We can improve this by using the serialization routines to per-property diff.
				//
				ConflictAggregate.AddObjectRemoved(ObjectId);
				for (const FGuid Id : *NamedPrimary)
				{
					RemovePendingToSend(Id);
				}
				NameLookupForPendingTransactionsToSend.Remove(ObjectId);
			}
		}
	};
	// For each exported object check for name conflicts.
	//
	for (const FConcertExportedObject& ExportedObject : InEvent.ExportedObjects)
	{
		LookupAndFix(ExportedObject);
	}
	if (ConflictAggregate.HasAnyResults())
	{
		NotifyUserOfSendConflict(ConflictAggregate);
	}
}

void FConcertClientTransactionManager::ProcessTransactionFinalizedEvent(const FPendingTransactionToProcessContext& InContext, const FConcertTransactionFinalizedEvent& InEvent)
{
	const FConcertSessionVersionInfo* VersionInfo = LiveSession->GetSession().GetSessionInfo().VersionInfos.IsValidIndex(InEvent.VersionIndex) ? &LiveSession->GetSession().GetSessionInfo().VersionInfos[InEvent.VersionIndex] : nullptr;
	FConcertLocalIdentifierTable LocalIdentifierTable(InEvent.LocalIdentifierState);
	TransactionBridge->OnApplyTransaction().Broadcast(ETransactionNotification::Begin, /*bIsSnapshot*/ false);
	CheckEventForSendConflicts(InEvent);
	TransactionBridge->ApplyRemoteTransaction(InEvent, VersionInfo, InContext.PackagesToProcess, &LocalIdentifierTable, /*bIsSnapshot*/false);

	TransactionBridge->OnApplyTransaction().Broadcast(ETransactionNotification::End, /*bIsSnapshot*/ false);
}

void FConcertClientTransactionManager::ProcessTransactionSnapshotEvent(const FPendingTransactionToProcessContext& InContext, const FConcertTransactionSnapshotEvent& InEvent)
{
	const FConcertSessionVersionInfo* VersionInfo = LiveSession->GetSession().GetSessionInfo().VersionInfos.IsValidIndex(InEvent.VersionIndex) ? &LiveSession->GetSession().GetSessionInfo().VersionInfos[InEvent.VersionIndex] : nullptr;
	TransactionBridge->OnApplyTransaction().Broadcast(ETransactionNotification::Begin, /*bIsSnapshot*/ true);
	TransactionBridge->ApplyRemoteTransaction(InEvent, VersionInfo, InContext.PackagesToProcess, nullptr, /*bIsSnapshot*/true);
	TransactionBridge->OnApplyTransaction().Broadcast(ETransactionNotification::End, /*bIsSnapshot*/ true);
}

FConcertClientTransactionManager::FPendingTransactionToSend&
FConcertClientTransactionManager::HandleLocalTransactionCommon(
	const FConcertClientLocalTransactionCommonData& InCommonData,
	const TArray<FConcertExportedObject>& ObjectUpdates)
{
	FPendingTransactionToSend* PendingTransactionPtr = PendingTransactionsToSend.Find(InCommonData.OperationId);
	if (PendingTransactionPtr)
	{
		PendingTransactionPtr->CommonData = InCommonData;
		return *PendingTransactionPtr;
	}
	return AddPendingToSend(InCommonData, ObjectUpdates);
}

FConcertClientTransactionManager::FPendingTransactionToSend&
FConcertClientTransactionManager::AddPendingToSend(
	const FConcertClientLocalTransactionCommonData& InCommonData,
	const TArray<FConcertExportedObject>& ObjectUpdates)
{
	PendingTransactionsToSendOrder.Add(InCommonData.OperationId);
	if (!CanSendTransactionEvents())
	{
		UObject* PrimaryObject = InCommonData.PrimaryObject.Get();
		if (PrimaryObject)
		{
			FConcertObjectId Id(PrimaryObject);
			TSet<FGuid>& NamedLookupGuids = NameLookupForPendingTransactionsToSend.FindOrAdd(Id);
			NamedLookupGuids.Add(InCommonData.OperationId);
		}
		for (const FConcertExportedObject& Exported : ObjectUpdates)
		{
			TSet<FGuid>& NamedLookupGuids = NameLookupForPendingTransactionsToSend.FindOrAdd(Exported.ObjectId);
			NamedLookupGuids.Add(InCommonData.OperationId);
		}
	}
	else
	{
		NameLookupForPendingTransactionsToSend.Reset();
	}
	return PendingTransactionsToSend.Emplace(InCommonData.OperationId, InCommonData);
}


void FConcertClientTransactionManager::RemovePendingToSend(const FGuid& InGuid)
{
	if (FPendingTransactionToSend* PendingToSend = PendingTransactionsToSend.Find(InGuid))
	{
		LiveSession->GetSession().GetSequencedEventManager().RemoveSequencedEvent(PendingToSend->SequencedEvent);
		PendingTransactionsToSend.Remove(InGuid);
	}
}

void FConcertClientTransactionManager::RemovePendingToSend(const FConcertClientLocalTransactionCommonData& InCommonData)
{
	RemovePendingToSend(InCommonData.OperationId);
	if (!CanSendTransactionEvents())
	{
		if (UObject* PrimaryObject = InCommonData.PrimaryObject.Get())
		{
			FConcertObjectId Id(PrimaryObject);
			TSet<FGuid>* NamedLookupForObjects = NameLookupForPendingTransactionsToSend.Find(Id);
			if (NamedLookupForObjects)
			{
				NamedLookupForObjects->Remove(InCommonData.OperationId);
			}
		}
	}
}

void FConcertClientTransactionManager::HandleLocalTransactionSnapshot(const FConcertClientLocalTransactionCommonData& InCommonData, const FConcertClientLocalTransactionSnapshotData& InSnapshotData)
{
	if (!CanSendTransactionEvents())
	{
		// Don't handle snapshot events when sending to clients.
		return;
	}

	if (InCommonData.bIsExcluded)
	{
		// Note: We don't remove this from PendingTransactionsToSendOrder as we just skip transactions missing from the
		// map (assuming they've been excluded).
		RemovePendingToSend(InCommonData);
		return;
	}

	FPendingTransactionToSend& PendingTransaction = HandleLocalTransactionCommon(InCommonData, InSnapshotData.SnapshotObjectUpdates);
	if (PendingTransaction.SnapshotData.SnapshotObjectUpdates.Num() == 0)
	{
		PendingTransaction.SnapshotData = InSnapshotData;
	}
	else
	{
		// Merge this snapshot with the current data
		for (const FConcertExportedObject& SnapshotObjectUpdate : InSnapshotData.SnapshotObjectUpdates)
		{
			// Find or add an entry for this object
			FConcertExportedObject* ObjectUpdatePtr = PendingTransaction.SnapshotData.SnapshotObjectUpdates.FindByPredicate([&SnapshotObjectUpdate](FConcertExportedObject& ObjectUpdate)
			{
				return ConcertSyncClientUtil::ObjectIdsMatch(SnapshotObjectUpdate.ObjectId, ObjectUpdate.ObjectId);
			});
			if (ObjectUpdatePtr)
			{
				// Apply any new annotation data
				ObjectUpdatePtr->SerializedAnnotationData = SnapshotObjectUpdate.SerializedAnnotationData;

				// Find or add an update for each property
				for (const FConcertSerializedPropertyData& SnapshotPropertyData : SnapshotObjectUpdate.PropertyDatas)
				{
					FConcertSerializedPropertyData* PropertyDataPtr = ObjectUpdatePtr->PropertyDatas.FindByPredicate([&SnapshotPropertyData](FConcertSerializedPropertyData& PropertyData)
					{
						return SnapshotPropertyData.PropertyName == PropertyData.PropertyName;
					});
					if (PropertyDataPtr)
					{
						PropertyDataPtr->SerializedData = SnapshotPropertyData.SerializedData;
					}
					else
					{
						ObjectUpdatePtr->PropertyDatas.Add(SnapshotPropertyData);
					}
				}
			}
			else
			{
				PendingTransaction.SnapshotData.SnapshotObjectUpdates.Add(SnapshotObjectUpdate);
			}
		}
	}
}

void FConcertClientTransactionManager::HandleLocalTransactionFinalized(const FConcertClientLocalTransactionCommonData& InCommonData, const FConcertClientLocalTransactionFinalizedData& InFinalizedData)
{
	if (InCommonData.bIsExcluded || InFinalizedData.FinalizedObjectUpdates.Num() == 0)
	{
		// Note: We don't remove this from PendingTransactionsToSendOrder as we just skip transactions missing from the map (assuming they've been excluded).
		RemovePendingToSend(InCommonData);
		return;
	}

	if (InFinalizedData.bWasCanceled)
	{
		// We only need to send this update if we sent any snapshot updates for this transaction (to undo the snapshot changes), otherwise we can just drop this transaction as no changes have propagated
		FPendingTransactionToSend* PendingTransactionPtr = PendingTransactionsToSend.Find(InCommonData.OperationId);
		if (PendingTransactionPtr && PendingTransactionPtr->LastSnapshotTimeSeconds == 0.0)
		{
			// Note: We don't remove this from PendingTransactionsToSendOrder as we just skip transactions missing from
			// the map (assuming they've been canceled).
			RemovePendingToSend(InCommonData);
			return;
		}
	}

	FPendingTransactionToSend& PendingTransaction = HandleLocalTransactionCommon(InCommonData, InFinalizedData.FinalizedObjectUpdates);
	PendingTransaction.FinalizedData = InFinalizedData;
	PendingTransaction.bIsFinalized = true;
	PendingTransaction.SequencedEvent = LiveSession->GetSession().GetSequencedEventManager().AddSequencedCustomEvent();
}

void FConcertClientTransactionManager::SendTransactionFinalizedEvent(const FGuid& InTransactionId, const FGuid& InOperationId, UObject* InPrimaryObject, const TArray<FName>& InModifiedPackages, const TArray<FConcertExportedObject>& InObjectUpdates, const FConcertLocalIdentifierTable& InLocalIdentifierTable, const FConcertSequencedCustomEvent& SequenceId, const FText& InTitle)
{
	FConcertTransactionFinalizedEvent TransactionFinalizedEvent;
	FillTransactionEvent(InTransactionId, InOperationId, InModifiedPackages, TransactionFinalizedEvent);
	TransactionFinalizedEvent.PrimaryObjectId = InPrimaryObject ? FConcertObjectId(InPrimaryObject) : FConcertObjectId();
	TransactionFinalizedEvent.ExportedObjects = InObjectUpdates;
	InLocalIdentifierTable.GetState(TransactionFinalizedEvent.LocalIdentifierState);
	TransactionFinalizedEvent.Title = InTitle;

	LiveSession->GetSession().SendCustomEvent(TransactionFinalizedEvent, LiveSession->GetSession().GetSessionServerEndpointId(), EConcertMessageFlags::ReliableOrdered, SequenceId);
}

void FConcertClientTransactionManager::SendTransactionSnapshotEvent(const FGuid& InTransactionId, const FGuid& InOperationId, UObject* InPrimaryObject, const TArray<FName>& InModifiedPackages, const TArray<FConcertExportedObject>& InObjectUpdates)
{
	check(EnumHasAnyFlags(LiveSession->GetSessionFlags(), EConcertSyncSessionFlags::ShouldSendTransactionSnapshots));

	FConcertTransactionSnapshotEvent TransactionSnapshotEvent;
	FillTransactionEvent(InTransactionId, InOperationId, InModifiedPackages, TransactionSnapshotEvent);
	TransactionSnapshotEvent.PrimaryObjectId = InPrimaryObject ? FConcertObjectId(InPrimaryObject) : FConcertObjectId();
	TransactionSnapshotEvent.ExportedObjects = InObjectUpdates;

	LiveSession->GetSession().SendCustomEvent(TransactionSnapshotEvent, LiveSession->GetSession().GetSessionServerEndpointId(), EConcertMessageFlags::None);
}

void FConcertClientTransactionManager::SendPendingTransactionEvents()
{
	const double SnapshotEventDelaySeconds = 1.0 / FMath::Max(GetDefault<UConcertSyncConfig>()->SnapshotTransactionsPerSecond, KINDA_SMALL_NUMBER);

	const double CurrentTimeSeconds = FPlatformTime::Seconds();

	for (auto PendingTransactionsToSendOrderIter = PendingTransactionsToSendOrder.CreateIterator(); PendingTransactionsToSendOrderIter; ++PendingTransactionsToSendOrderIter)
	{
		FPendingTransactionToSend* PendingTransactionPtr = PendingTransactionsToSend.Find(*PendingTransactionsToSendOrderIter);
		if (!PendingTransactionPtr)
		{
			// Missing transaction, must have been canceled/excluded...
			PendingTransactionsToSendOrderIter.RemoveCurrent();
			continue;
		}

		// if the transaction isn't excluded, send updates
		if (!PendingTransactionPtr->CommonData.bIsExcluded)
		{
			UObject* PrimaryObject = PendingTransactionPtr->CommonData.PrimaryObject.Get(/*bEvenIfPendingKill*/true);
			if (PendingTransactionPtr->bIsFinalized)
			{
				// Process this transaction
				if (PendingTransactionPtr->FinalizedData.FinalizedObjectUpdates.Num() > 0)
				{
					SendTransactionFinalizedEvent(
						PendingTransactionPtr->CommonData.TransactionId, 
						PendingTransactionPtr->CommonData.OperationId, 
						PrimaryObject, 
						PendingTransactionPtr->CommonData.ModifiedPackages, 
						PendingTransactionPtr->FinalizedData.FinalizedObjectUpdates, 
						PendingTransactionPtr->FinalizedData.FinalizedLocalIdentifierTable,
						PendingTransactionPtr->SequencedEvent,
						PendingTransactionPtr->CommonData.TransactionTitle
						);
				}
				// TODO: Warn about excluded objects?

				RemovePendingToSend(PendingTransactionPtr->CommonData);
				PendingTransactionsToSendOrderIter.RemoveCurrent();
				continue;
			}
			else if (PendingTransactionPtr->SnapshotData.SnapshotObjectUpdates.Num() > 0 && CurrentTimeSeconds > PendingTransactionPtr->LastSnapshotTimeSeconds + SnapshotEventDelaySeconds)
			{
				// Process this snapshot
				SendTransactionSnapshotEvent(
					PendingTransactionPtr->CommonData.TransactionId,
					PendingTransactionPtr->CommonData.OperationId,
					PrimaryObject, 
					PendingTransactionPtr->CommonData.ModifiedPackages,
					PendingTransactionPtr->SnapshotData.SnapshotObjectUpdates
					);

				PendingTransactionPtr->SnapshotData.SnapshotObjectUpdates.Reset();
				PendingTransactionPtr->LastSnapshotTimeSeconds = CurrentTimeSeconds;
			}
		}
		// Once the excluded transaction is finalized, broadcast and remove it.
		else if (PendingTransactionPtr->bIsFinalized)
		{
			// TODO: Broadcast delegate

			RemovePendingToSend(PendingTransactionPtr->CommonData.TransactionId);
			PendingTransactionsToSendOrderIter.RemoveCurrent();
			continue;
		}
	}
}

bool FConcertClientTransactionManager::ShouldProcessTransactionEvent(const FConcertTransactionEventBase& InEvent, const bool InIsRequired) const
{
	const FName TransactionKey = *FString::Printf(TEXT("TransactionManager.TransactionId:%s"), *InEvent.TransactionId.ToString());

	FConcertScratchpadPtr SenderScratchpad = LiveSession->GetSession().GetClientScratchpad(InEvent.TransactionEndpointId);
	if (SenderScratchpad.IsValid())
	{
		// If the event is required then we have to process it (it may have been received after a newer non-required transaction update, which is why we skip the update order check)
		if (InIsRequired)
		{
			SenderScratchpad->SetValue<uint8>(TransactionKey, InEvent.TransactionUpdateIndex);
			return true;
		}
		
		// If the event isn't required, then we can drop it if its update index is older than the last update we processed
		if (uint8* TransactionUpdateIndexPtr = LiveSession->GetSession().GetScratchpad()->GetValue<uint8>(TransactionKey))
		{
			uint8& TransactionUpdateIndex = *TransactionUpdateIndexPtr;
			const bool bShouldProcess = InEvent.TransactionUpdateIndex >= TransactionUpdateIndex + 1; // Note: We +1 before doing the check to handle overflow
			TransactionUpdateIndex = InEvent.TransactionUpdateIndex;
			return bShouldProcess;
		}

		// First update for this transaction, just process it
		SenderScratchpad->SetValue<uint8>(TransactionKey, InEvent.TransactionUpdateIndex);
		return true;
	}

	return true;
}

void FConcertClientTransactionManager::FillTransactionEvent(const FGuid& InTransactionId, const FGuid& InOperationId, const TArray<FName>& InModifiedPackages, FConcertTransactionEventBase& OutEvent) const
{
	const FName TransactionKey = *FString::Printf(TEXT("TransactionManager.TransactionId:%s"), *InTransactionId.ToString());

	OutEvent.TransactionId = InTransactionId;
	OutEvent.OperationId = InOperationId;
	OutEvent.TransactionEndpointId = LiveSession->GetSession().GetSessionClientEndpointId();
	OutEvent.VersionIndex = LiveSession->GetSession().GetSessionInfo().VersionInfos.Num() - 1;
	OutEvent.TransactionUpdateIndex = 0;
	OutEvent.ModifiedPackages = InModifiedPackages;

	if (uint8* TransactionUpdateIndexPtr = LiveSession->GetSession().GetScratchpad()->GetValue<uint8>(TransactionKey))
	{
		uint8& TransactionUpdateIndex = *TransactionUpdateIndexPtr;
		OutEvent.TransactionUpdateIndex = TransactionUpdateIndex++;
	}
	else
	{
		LiveSession->GetSession().GetScratchpad()->SetValue<uint8>(TransactionKey, OutEvent.TransactionUpdateIndex);
	}
}

#undef LOCTEXT_NAMESPACE
