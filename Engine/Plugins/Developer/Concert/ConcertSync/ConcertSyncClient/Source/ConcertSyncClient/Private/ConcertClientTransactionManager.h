// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertTransactionEvents.h"
#include "CoreMinimal.h"
#include "Misc/ITransaction.h"
#include "UObject/Class.h"
#include "ConcertMessages.h"
#include "IConcertSession.h"
#include "IConcertSessionHandler.h"
#include "IConcertClientTransactionBridge.h"
#include "UObject/StructOnScope.h"
#include "ConcertSyncClientUtil.h"

class FConcertSyncClientLiveSession;

struct FConcertConflictDescriptionAggregate : public FConcertConflictDescriptionBase
{
	virtual ~FConcertConflictDescriptionAggregate() override = default;
	virtual FText GetConflictDetails() const override;
	virtual FText GetConflictTitle() const override;

	void AddObjectRemoved(const FConcertObjectId& ObjectIdRemoved);
	void AddObjectRenamed(const FConcertObjectId& OldObjectId, const FConcertObjectId& NewObjectId);

	bool HasAnyResults() const
	{
		return !ObjectsRemoved.IsEmpty() || !ObjectsRenamed.IsEmpty();
	}

private:
	TArray<FConcertObjectId> ObjectsRemoved;
	TArray<TPair<FConcertObjectId, FConcertObjectId>> ObjectsRenamed;
};

class FConcertClientTransactionManager
{
public:
	FConcertClientTransactionManager(TSharedRef<FConcertSyncClientLiveSession> InLiveSession, IConcertClientTransactionBridge* InTransactionBridge);
	~FConcertClientTransactionManager();

	/**
	 * @return true if there are any packages with live transactions right now.
	 */
	bool HasSessionChanges() const;

	/**
	 * Indicate if a particular package is supported for live transactions
	 * based on transaction filters.
	 *
	 * @param InPackage The package to check for support.
	 * @return true if the package support live transactions.
	 */
	bool HasLiveTransactionSupport(class UPackage* InPackage) const;

	/**
	 * Called to replay any live transactions for all packages.
	 */
	void ReplayAllTransactions();

	/**
	 * Called to replay live transactions for the given package.
	 */
	void ReplayTransactions(const FName InPackageName);

	/**
	 * Called to handle a remote transaction being received.
	 */
	void HandleRemoteTransaction(const FGuid& InSourceEndpointId, const int64 InTransactionEventId, const bool bApply);

	/**
	 * Called to process any pending transaction events (sending or receiving).
	 */
	void ProcessPending();

private:
	/**
	 * Context object for transactions that are to be processed.
	 */
	struct FPendingTransactionToProcessContext
	{
		/** Is this transaction required? */
		bool bIsRequired = false;

		/** Optional list of packages to process transactions for, or empty to process transactions for all packages */
		TArray<FName> PackagesToProcess;
	};

	/** Lookup set key functions. */
	struct FConcertObjectIdFuncs : public TDefaultMapKeyFuncs<FConcertObjectId, TSet<FGuid>, false>
	{
		FORCEINLINE static bool Matches(KeyInitType A, KeyInitType B)
		{
			return ConcertSyncClientUtil::ObjectIdsMatch(A, B);
		}

		FORCEINLINE static uint32 GetKeyHash(KeyInitType InObjectId)
		{
			return HashCombine(GetTypeHash(InObjectId.ObjectName), GetTypeHash(InObjectId.ObjectOuterPathName));
		}
	};

	/**
	 * A received pending transaction event that was queued for processing later.
	 */
	struct FPendingTransactionToProcess
	{
		FPendingTransactionToProcess(const FPendingTransactionToProcessContext& InContext, const UScriptStruct* InEventStruct, const void* InEventData)
			: Context(InContext)
			, EventData(InEventStruct)
		{
			InEventStruct->CopyScriptStruct(EventData.GetStructMemory(), InEventData);
		}

		FPendingTransactionToProcess(const FPendingTransactionToProcessContext& InContext, FStructOnScope&& InEvent)
			: Context(InContext)
			, EventData(MoveTemp(InEvent))
		{
			check(EventData.OwnsStructMemory());
		}

		FPendingTransactionToProcessContext Context;
		FStructOnScope EventData;
	};

	/**
	 * A pending transaction that may be sent in the future (when finalized).
	 */
	struct FPendingTransactionToSend
	{
		explicit FPendingTransactionToSend(const FConcertClientLocalTransactionCommonData& InCommonData)
			: CommonData(InCommonData)
		{
		}

		FConcertClientLocalTransactionCommonData CommonData;
		FConcertClientLocalTransactionSnapshotData SnapshotData;
		FConcertClientLocalTransactionFinalizedData FinalizedData;

		// Sequencing id for this pending transaction.
		FConcertSequencedCustomEvent SequencedEvent;

		bool bIsFinalized = false;
		double LastSnapshotTimeSeconds = 0.0;
	};

	/**
	 * Handle a transaction event by queueing it for processing at the end of the current frame.
	 */
	template <typename EventType>
	void HandleTransactionEvent(const FConcertSessionContext& InEventContext, const EventType& InEvent);

	/**
	 * Handle a rejected transaction event, those are sent by the server when a transaction is refused.
	 */
	void HandleTransactionRejectedEvent(const FConcertSessionContext& InEventContext, const FConcertTransactionRejectedEvent& InEvent);

	/**
	 * Can we currently process transaction events?
	 * True if we are neither suspended nor unable to perform a blocking action, false otherwise.
	 */
	bool CanProcessTransactionEvent() const;

	/**
	 * Can we send transaction events to remote clients.
	 */
	bool CanSendTransactionEvents() const;

	/**
	 * Process a transaction event.
	 */
	void ProcessTransactionEvent(const FPendingTransactionToProcessContext& InContext, const FStructOnScope& InEvent);

	/**
	 * Process a transaction finalized event.
	 */
	void ProcessTransactionFinalizedEvent(const FPendingTransactionToProcessContext& InContext, const FConcertTransactionFinalizedEvent& InEvent);

	/**
	 * Process a transaction snapshot event.
	 */
	void ProcessTransactionSnapshotEvent(const FPendingTransactionToProcessContext& InContext, const FConcertTransactionSnapshotEvent& InEvent);

	/**
	 * Handle a local transaction, ensuring that there is a pending entry to be sent.
	 */
	FPendingTransactionToSend& HandleLocalTransactionCommon(const FConcertClientLocalTransactionCommonData& InCommonData, const TArray<FConcertExportedObject>& ObjectUpdates);

	/**
	 * Handle a local transaction being snapshot, queueing it for send later.
	 */
	void HandleLocalTransactionSnapshot(const FConcertClientLocalTransactionCommonData& InCommonData, const FConcertClientLocalTransactionSnapshotData& InSnapshotData);
	
	/**
	 * Handle a local transaction being finalized, queueing it for send later.
	 */
	void HandleLocalTransactionFinalized(const FConcertClientLocalTransactionCommonData& InCommonData, const FConcertClientLocalTransactionFinalizedData& InFinalizedData);

	/**
	 * Send a transaction finalized event.
	 */
	void SendTransactionFinalizedEvent(const FGuid& InTransactionId, const FGuid& InOperationId, UObject* InPrimaryObject, const TArray<FName>& InModifiedPackages, const TArray<FConcertExportedObject>& InObjectUpdates, const FConcertLocalIdentifierTable& InLocalIdentifierTable, const FConcertSequencedCustomEvent& SequenceId, const FText& InTitle);

	/**
	 * Send a transaction snapshot event.
	 */
	void SendTransactionSnapshotEvent(const FGuid& InTransactionId, const FGuid& InOperationId, UObject* InPrimaryObject, const TArray<FName>& InModifiedPackages, const TArray<FConcertExportedObject>& InObjectUpdates);

	/**
	 * Send any pending transaction events that qualify.
	 */
	void SendPendingTransactionEvents();

	/**
	 * Check the inbound transaction event for conflicts in the pending send. If we are receive only
	 * and a conflict is detected then we have to remove pending send object.
	 */
	void CheckEventForSendConflicts(const FConcertTransactionFinalizedEvent& InEvent);

	/**
	 * Fixup any objects from the old ObjectId a given new ObjectId.  The routine will check that the
	 * given object ids are valid.
	 *
	 * @param OldObjectId to search.
	 * @param NewObjectId to replace.
	 * @param Aggregate for conflict information.
	 */
	void FixupObjectIdsInPendingSend(const FConcertObjectId& OldObjectId, const FConcertObjectId& NewObjectId, FConcertConflictDescriptionAggregate& ConflictAggregate);

	/**
	 * Remove the given pending to send object.
	 */
	void RemovePendingToSend(const FConcertClientLocalTransactionCommonData& InCommonData);

	/**
	 * Remove the given pending to send object by Guid.
	 */
	void RemovePendingToSend(const FGuid& InCommonData);

	/**
	 * Remove the given pending to send object.
	 */
	FPendingTransactionToSend& AddPendingToSend(const FConcertClientLocalTransactionCommonData& InCommonData, const TArray<FConcertExportedObject>& ObjectUpdates);

	/**
	 * Should process this transaction event?
	 */
	bool ShouldProcessTransactionEvent(const FConcertTransactionEventBase& InEvent, const bool InIsRequired) const;

	/**
	 * Fill in the transaction event based on the given GUID.
	 */
	void FillTransactionEvent(const FGuid& InTransactionId, const FGuid& InOperationId, const TArray<FName>& InModifiedPackages, FConcertTransactionEventBase& OutEvent) const;

	/**
	 * Notify externally that a transaction conflict has occurred.
	 */
	void NotifyUserOfSendConflict(const FConcertConflictDescriptionBase& ConflictDescription);

	/**
	 * Session instance this transaction manager was created for.
	 */
	TSharedPtr<FConcertSyncClientLiveSession> LiveSession;

	/**
	 * Transaction bridge used by this manager.
	 */
	IConcertClientTransactionBridge* TransactionBridge;

	/**
	 * Array of pending transaction events in the order they were received.
	 * Events are queued in this array while the session is suspended or the user is interacting,
	 * and any queued transactions will be processed on the next Tick.
	 */
	TArray<FPendingTransactionToProcess> PendingTransactionsToProcess;

	/**
	 * Array of transaction IDs in the order they should be sent (maps to PendingTransactionsToSend, although canceled transactions may be missing from the map).
	 */
	TArray<FGuid> PendingTransactionsToSendOrder;

	/**
	 * Map of transaction IDs to the pending transaction that may be sent in the future (when finalized).
	 */
	TMap<FGuid, FPendingTransactionToSend> PendingTransactionsToSend;

	/**
	 * Named lookup for pending transactions to send.  This is so we can cross check the pending for send against
	 * remote transactions that may conflict.
	 */
	TMap<FConcertObjectId, TSet<FGuid>, FDefaultSetAllocator, FConcertObjectIdFuncs> NameLookupForPendingTransactionsToSend;
};
