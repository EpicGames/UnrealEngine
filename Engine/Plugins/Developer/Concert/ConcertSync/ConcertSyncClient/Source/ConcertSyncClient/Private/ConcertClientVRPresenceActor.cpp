// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertClientVRPresenceActor.h"
#include "UObject/ConstructorHelpers.h"
#include "EngineGlobals.h"
#include "Components/StaticMeshComponent.h"
#include "Components/TextRenderComponent.h"
#include "ConcertClientPresenceMode.h"
#include "Engine/Engine.h"
#include "ConcertAssetContainer.h"
#include "Materials/MaterialInstanceDynamic.h"
#if WITH_EDITOR
#include "IXRTrackingSystem.h"
#endif 
#include "ConcertClientPresenceManager.h"
#include "Components/SplineMeshComponent.h"
#include "Components/SplineComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConcertClientVRPresenceActor)

#define LOCTEXT_NAMESPACE "ConcertClientVRPresenceActor"

//////////////////////////////////////////////////////////////////////////
// AConcertClientVRPresenceActor

AConcertClientVRPresenceActor::AConcertClientVRPresenceActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	LeftControllerMeshComponent = CreateDefaultSubobject<UStaticMeshComponent>(TEXT("Left"));
	AddOwnedComponent(LeftControllerMeshComponent);
	LeftControllerMeshComponent->SetupAttachment(RootComponent);

	RightControllerMeshComponent = CreateDefaultSubobject<UStaticMeshComponent>(TEXT("Right"));
	AddOwnedComponent(RightControllerMeshComponent);
	RightControllerMeshComponent->SetupAttachment(RootComponent);

	LeftLaserSplineComponent = CreateDefaultSubobject<USplineComponent>(TEXT("LeftSpline"));
	AddOwnedComponent(LeftLaserSplineComponent);
	LeftLaserSplineComponent->SetupAttachment(RootComponent);
	LeftLaserSplineComponent->SetVisibility(false);

	RightLaserSplineComponent = CreateDefaultSubobject<USplineComponent>(TEXT("RightSpline"));
	AddOwnedComponent(RightLaserSplineComponent);
	RightLaserSplineComponent->SetupAttachment(RootComponent);
	RightLaserSplineComponent->SetVisibility(false);

	bIsRightControllerVisible = true;
	bIsLeftControllerVisible = true;

	bIsLeftLaserVisible = false;
	bIsRightLaserVisible = false;
}

void AConcertClientVRPresenceActor::HandleEvent(const FStructOnScope& InEvent)
{
	if (InEvent.GetStruct() == FConcertClientVRPresenceUpdateEvent::StaticStruct())
	{
		if (const FConcertClientVRPresenceUpdateEvent* Event = (const FConcertClientVRPresenceUpdateEvent*)InEvent.GetStructMemory())
		{
			const double TimestampSeconds = FPlatformTime::Seconds();
			const double LocationUpdateFrequency = FConcertClientPresenceManager::GetLocationUpdateFrequency();

			// left controller
			const FTransform LeftControllerTransform(Event->LeftMotionControllerOrientation, Event->LeftMotionControllerPosition);
			if (LeftControllerTransform.Equals(FTransform::Identity))
			{
				HideLeftController();
			}
			else
			{
				if (!LeftControllerMovement.IsSet())
				{
					LeftControllerMovement = FConcertClientMovement(LocationUpdateFrequency, TimestampSeconds, Event->LeftMotionControllerPosition, Event->LeftMotionControllerOrientation);
				}
				else
				{
					LeftControllerMovement->UpdateLastKnownLocation(TimestampSeconds, Event->LeftMotionControllerPosition, &Event->LeftMotionControllerOrientation);
				}
			}

			// right controller
			const FTransform RightControllerTransform(Event->RightMotionControllerOrientation, Event->RightMotionControllerPosition);
			if (RightControllerTransform.Equals(FTransform::Identity))
			{
				HideRightController();
			}
			else
			{
				if (!RightControllerMovement.IsSet())
				{
					RightControllerMovement = FConcertClientMovement(LocationUpdateFrequency, TimestampSeconds, Event->RightMotionControllerPosition, Event->RightMotionControllerOrientation);
				}
				else
				{
					RightControllerMovement->UpdateLastKnownLocation(TimestampSeconds, Event->RightMotionControllerPosition, &Event->RightMotionControllerOrientation);
				}
			}

			// lasers
			auto UpdateLaserLastKnownLocation = [TimestampSeconds, LocationUpdateFrequency](TOptional<FConcertClientMovement>& MovementObject, const FVector& Position)
			{
				if (!MovementObject.IsSet())
				{
					MovementObject = FConcertClientMovement(LocationUpdateFrequency, TimestampSeconds, Position);
				}
				else
				{
					MovementObject->UpdateLastKnownLocation(TimestampSeconds, Position);
				}
			};

			//  left laser
			if (Event->Lasers[(int32)EControllerHand::Left].IsValid())
			{
				const FConcertLaserData& Laser = Event->Lasers[(int32)EControllerHand::Left];
				UpdateLaserLastKnownLocation(LeftLaserStartMovement, Laser.LaserStart);
				UpdateLaserLastKnownLocation(LeftLaserEndMovement, Laser.LaserEnd);

			}
			else
			{
				HideLeftLaser();
			}

			// right laser
			if (Event->Lasers[(int32)EControllerHand::Right].IsValid())
			{
				const FConcertLaserData& Laser = Event->Lasers[(int32)EControllerHand::Right];
				UpdateLaserLastKnownLocation(RightLaserStartMovement, Laser.LaserStart);
				UpdateLaserLastKnownLocation(RightLaserEndMovement, Laser.LaserEnd);

			}
			else
			{
				HideRightLaser();
			}
		}
	}
	else
	{
		Super::HandleEvent(InEvent);
	}
}

void AConcertClientVRPresenceActor::Tick(float DeltaSeconds)
{
	Super::Tick(DeltaSeconds);

	// Left Controller
	if (LeftControllerMovement.IsSet())
	{
		if (!bIsLeftControllerVisible)
		{
			ShowLeftController();
		}

		FVector LeftControllerPosition;
		FQuat LeftControllerOrientation;
		LeftControllerMovement->MoveSmooth(DeltaSeconds, LeftControllerPosition, &LeftControllerOrientation);

		const FTransform LeftControllerTransform(LeftControllerOrientation, LeftControllerPosition);
		LeftControllerMeshComponent->SetWorldTransform(LeftControllerTransform);
	}

	// Right Controller
	if (RightControllerMovement.IsSet())
	{
		if (!bIsRightControllerVisible)
		{
			ShowRightController();
		}

		FVector RightControllerPosition;
		FQuat RightControllerOrientation;
		RightControllerMovement->MoveSmooth(DeltaSeconds, RightControllerPosition, &RightControllerOrientation);

		const FTransform RightControllerTransform(RightControllerOrientation, RightControllerPosition);
		RightControllerMeshComponent->SetWorldTransform(RightControllerTransform);
		RightControllerMeshComponent->SetRelativeScale3D(FVector(1.0f, -1.0f, 1.0f));
	}

	// Left Laser
	if (LeftLaserSplineComponent && LeftLaserStartMovement.IsSet() && LeftLaserEndMovement.IsSet())
	{
		if (!bIsLeftLaserVisible)
		{
			ShowLeftLaser();
		}

		FVector LaserStartPosition;
		FVector LaserEndPosition;
		LeftLaserStartMovement->MoveSmooth(DeltaSeconds, LaserStartPosition);
		LeftLaserEndMovement->MoveSmooth(DeltaSeconds, LaserEndPosition);
		UpdateSplineLaser(LeftLaserSplineComponent, LeftLaserSplineMeshComponents, LaserStartPosition, LaserEndPosition);
	}

	// Right Laser
	if (RightLaserSplineComponent && RightLaserStartMovement.IsSet() && RightLaserEndMovement.IsSet())
	{
		if (!bIsRightLaserVisible)
		{
			ShowRightLaser();
		}

		FVector LaserStartPosition;
		FVector LaserEndPosition;
		RightLaserStartMovement->MoveSmooth(DeltaSeconds, LaserStartPosition);
		RightLaserEndMovement->MoveSmooth(DeltaSeconds, LaserEndPosition);
		UpdateSplineLaser(RightLaserSplineComponent, RightLaserSplineMeshComponents, LaserStartPosition, LaserEndPosition);
	}
}

void AConcertClientVRPresenceActor::InitPresence(const class UConcertAssetContainer& InAssetContainer, FName DeviceType)
{
	Super::InitPresence(InAssetContainer, DeviceType);

	// To do, send data about these through the event.
	UStaticMesh* ControllerMesh = PresenceDeviceType == FName(TEXT("OculusHMD")) ? InAssetContainer.OculusControllerMesh : InAssetContainer.VivePreControllerMesh;

	LeftControllerMeshComponent->SetStaticMesh(ControllerMesh);
	LeftControllerMeshComponent->SetMobility(EComponentMobility::Movable);
	LeftControllerMeshComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
	LeftControllerMeshComponent->bSelectable = false;
	LeftControllerMeshComponent->SetCastShadow(false);

	RightControllerMeshComponent->SetStaticMesh(ControllerMesh);
	RightControllerMeshComponent->SetMobility(EComponentMobility::Movable);
	RightControllerMeshComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
	RightControllerMeshComponent->bSelectable = false;
	RightControllerMeshComponent->SetCastShadow(false);

	PresenceMeshComponent->SetMaterial(0, PresenceMID);
	LeftControllerMeshComponent->SetMaterial(0, PresenceMID);
	RightControllerMeshComponent->SetMaterial(0, PresenceMID);

	const int32 NumLaserSplinePoints = 12;

	UStaticMesh* MiddleSplineMesh = InAssetContainer.LaserPointerMesh;
	UStaticMesh* StartSplineMesh = InAssetContainer.LaserPointerStartMesh;
	UStaticMesh* EndSplineMesh = InAssetContainer.LaserPointerEndMesh;

	UMaterialInterface* LaserMaterial = InAssetContainer.LaserMaterial;
	LaserMid = UMaterialInstanceDynamic::Create(LaserMaterial, this);

	UMaterialInterface* LaserCoreMaterial = InAssetContainer.LaserCoreMaterial;
	LaserCoreMid = UMaterialInstanceDynamic::Create(LaserCoreMaterial, this);

	auto AddSplinePoints = [this, StartSplineMesh, MiddleSplineMesh, EndSplineMesh, NumLaserSplinePoints](TArray<USplineMeshComponent*>& LaserSplineMeshComponents)
	{
		for (int32 i = 0; i < NumLaserSplinePoints; i++)
		{
			USplineMeshComponent* SplineSegment = NewObject<USplineMeshComponent>(this);
			SplineSegment->SetMobility(EComponentMobility::Movable);
			SplineSegment->SetCollisionEnabled(ECollisionEnabled::NoCollision);
			SplineSegment->SetbNeverNeedsCookedCollisionData(true);
			SplineSegment->SetSplineUpDir(FVector::UpVector, false);

			UStaticMesh* StaticMesh = nullptr;
			if (i == 0)
			{
				StaticMesh = StartSplineMesh;
			}
			else if (i == NumLaserSplinePoints - 1)
			{
				StaticMesh = EndSplineMesh;
			}
			else
			{
				StaticMesh = MiddleSplineMesh;
			}

			SplineSegment->SetStaticMesh(StaticMesh);
			SplineSegment->bTickInEditor = true;
			SplineSegment->bCastDynamicShadow = false;
			SplineSegment->CastShadow = false;
			SplineSegment->SetMaterial(0, LaserCoreMid);
			SplineSegment->SetMaterial(1, LaserMid);
			SplineSegment->SetVisibility(true);
			SplineSegment->RegisterComponent();

			LaserSplineMeshComponents.Add(SplineSegment);
		}
	};
	AddSplinePoints(MutableView(LeftLaserSplineMeshComponents));
	AddSplinePoints(MutableView(RightLaserSplineMeshComponents));
}

void AConcertClientVRPresenceActor::SetPresenceColor(const FLinearColor& InColor)
{
	static const FName LaserColorParam(TEXT("UserColor"));
	static const FName BodyColorParam(TEXT("Color"));
	static const FName ColorParamName(TEXT("Color"));

	LaserMid->SetVectorParameterValue(LaserColorParam, InColor);
	LaserCoreMid->SetVectorParameterValue(LaserColorParam, InColor);
	PresenceMID->SetVectorParameterValue(BodyColorParam, InColor);
	TextMID->SetVectorParameterValue(ColorParamName, InColor);
}

void AConcertClientVRPresenceActor::UpdateSplineLaser(USplineComponent* InLaserSplineComponent, const TArray<USplineMeshComponent*>& InLaserSplineMeshComponents, const FVector& InStartLocation, const FVector& InEndLocation)
{
	if (InLaserSplineComponent)
	{
		// Clear the segments before updating it
		InLaserSplineComponent->ClearSplinePoints(true);

		const FVector SmoothLaserDirection = InEndLocation - InStartLocation;
		float Distance = SmoothLaserDirection.Size();
		const FVector StraightLaserEndLocation = InEndLocation;
		const int32 NumLaserSplinePoints = InLaserSplineMeshComponents.Num();

		InLaserSplineComponent->AddSplinePoint(InStartLocation, ESplineCoordinateSpace::Local, false);
		for (int32 Index = 1; Index < NumLaserSplinePoints; Index++)
		{
			float Alpha = (float)Index / (float)NumLaserSplinePoints;
			Alpha = FMath::Sin(Alpha * PI * 0.5f);
			const FVector PointOnStraightLaser = FMath::Lerp(InStartLocation, StraightLaserEndLocation, Alpha);
			const FVector PointOnSmoothLaser = FMath::Lerp(InStartLocation, InEndLocation, Alpha);
			const FVector PointBetweenLasers = FMath::Lerp(PointOnStraightLaser, PointOnSmoothLaser, Alpha);
			InLaserSplineComponent->AddSplinePoint(PointBetweenLasers, ESplineCoordinateSpace::Local, false);
		}
		InLaserSplineComponent->AddSplinePoint(InEndLocation, ESplineCoordinateSpace::Local, false);

		// Update all the segments of the spline
		InLaserSplineComponent->UpdateSpline();

		const float LaserPointerRadius = LaserThickness;
		Distance *= 0.0001f;
		for (int32 Index = 0; Index < NumLaserSplinePoints; Index++)
		{
			USplineMeshComponent* SplineMeshComponent = InLaserSplineMeshComponents[Index];
			check(SplineMeshComponent != nullptr);

			FVector StartLoc, StartTangent, EndLoc, EndTangent;
			InLaserSplineComponent->GetLocationAndTangentAtSplinePoint(Index, StartLoc, StartTangent, ESplineCoordinateSpace::Local);
			InLaserSplineComponent->GetLocationAndTangentAtSplinePoint(Index + 1, EndLoc, EndTangent, ESplineCoordinateSpace::Local);

			const float AlphaIndex = (float)Index / (float)NumLaserSplinePoints;
			const float AlphaDistance = Distance * AlphaIndex;
			float Radius = LaserPointerRadius * ((AlphaIndex * AlphaDistance) + 1);
			FVector2D LaserScale(Radius, Radius);
			SplineMeshComponent->SetStartScale(LaserScale, false);

			const float NextAlphaIndex = (float)(Index + 1) / (float)NumLaserSplinePoints;
			const float NextAlphaDistance = Distance * NextAlphaIndex;
			Radius = LaserPointerRadius * ((NextAlphaIndex * NextAlphaDistance) + 1);
			LaserScale = FVector2D(Radius, Radius);
			SplineMeshComponent->SetEndScale(LaserScale, false);

			SplineMeshComponent->SetStartAndEnd(StartLoc, StartTangent, EndLoc, EndTangent, true);
		}
	}
}

void AConcertClientVRPresenceActor::HideLeftController()
{
	bIsLeftControllerVisible = false;
	LeftControllerMeshComponent->SetVisibility(false, true);

	if (LeftControllerMovement.IsSet())
	{
		LeftControllerMovement.Reset();
	}
}

void AConcertClientVRPresenceActor::ShowLeftController()
{
	bIsLeftControllerVisible = true;
	LeftControllerMeshComponent->SetVisibility(true, true);
}

void AConcertClientVRPresenceActor::HideRightController()
{
	bIsRightControllerVisible = false;
	RightControllerMeshComponent->SetVisibility(false, true);

	if (RightControllerMovement.IsSet())
	{
		RightControllerMovement.Reset();
	}
}

void AConcertClientVRPresenceActor::ShowRightController()
{
	bIsRightControllerVisible = true;
	RightControllerMeshComponent->SetVisibility(true, true);
}

void AConcertClientVRPresenceActor::HideLeftLaser()
{
	bIsLeftLaserVisible = false;
	for (USplineMeshComponent* LaserSplineMeshSegmentCmp : LeftLaserSplineMeshComponents)
	{
		LaserSplineMeshSegmentCmp->SetVisibility(false, true);
	}
	
	if (LeftLaserStartMovement.IsSet())
	{
		LeftLaserStartMovement.Reset();
	}

	if (LeftLaserEndMovement.IsSet())
	{
		LeftLaserEndMovement.Reset();
	}
}

void AConcertClientVRPresenceActor::ShowLeftLaser()
{
	bIsLeftLaserVisible = true;
	for (USplineMeshComponent* LaserSplineMeshSegmentCmp : LeftLaserSplineMeshComponents)
	{
		LaserSplineMeshSegmentCmp->SetVisibility(true, true);
	}
}

void AConcertClientVRPresenceActor::HideRightLaser()
{
	bIsRightLaserVisible = false;
	for (USplineMeshComponent* LaserSplineMeshSegmentCmp : RightLaserSplineMeshComponents)
	{
		LaserSplineMeshSegmentCmp->SetVisibility(false, true);
	}

	if (RightLaserStartMovement.IsSet())
	{
		RightLaserStartMovement.Reset();
	}

	if (RightLaserEndMovement.IsSet())
	{
		RightLaserEndMovement.Reset();
	}
}

void AConcertClientVRPresenceActor::ShowRightLaser()
{
	bIsRightLaserVisible = true;
	for (USplineMeshComponent* LaserSplineMeshSegmentCmp : RightLaserSplineMeshComponents)
	{
		LaserSplineMeshSegmentCmp->SetVisibility(true, true);
	}
}


#undef LOCTEXT_NAMESPACE


