// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertClientWorkspace.h"

#include "ConcertClientTransactionManager.h"
#include "ConcertClientPackageManager.h"
#include "ConcertClientLockManager.h"
#include "IConcertClientPackageBridge.h"
#include "IConcertClient.h"
#include "IConcertClientWorkspace.h"
#include "IConcertSyncClient.h"
#include "IConcertSyncClientModule.h"

#include "ConcertSyncClientLiveSession.h"
#include "ConcertSyncSessionDatabase.h"
#include "ConcertClientSettings.h"
#include "ConcertSyncClientUtil.h"
#include "ConcertLogGlobal.h"
#include "ConcertWorkspaceData.h"
#include "ConcertWorkspaceMessages.h"
#include "ConcertClientDataStore.h"
#include "ConcertClientLiveTransactionAuthors.h"
#include "IConcertSession.h"
#include "IConcertFileSharingService.h"
#include "Replication/Messages/ReplicationActivity.h"

#include "Algo/AllOf.h"
#include "Containers/ArrayBuilder.h"
#include "UObject/Package.h"
#include "UObject/SavePackage.h"
#include "UObject/StructOnScope.h"
#include "Misc/PackageName.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "Misc/CoreDelegates.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/FeedbackContext.h"
#include "Serialization/MemoryReader.h"
#include "StructDeserializer.h"
#include "StructSerializer.h"
#include "Backends/JsonStructDeserializerBackend.h"
#include "Backends/JsonStructSerializerBackend.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/World.h"
#include "Engine/Engine.h"

#if WITH_EDITOR
	#include "Editor.h"
	#include "UnrealEdGlobals.h"
	#include "Editor/EditorEngine.h"
	#include "Editor/UnrealEdEngine.h"
	#include "Editor/TransBuffer.h"
#endif

LLM_DEFINE_TAG(Concert_ConcertClientWorkspace);
#define LOCTEXT_NAMESPACE "ConcertClientWorkspace"

/** Provides a workaround for scoped slow tasks that are push/pop out of order by Concert. Concert doesn't use FScopedSlowTask as designed when syncing
    workspaces. The tasks are likely to run over many engine loops. DisasterRecovery and MultiUser, based on Concert, may both try to run the same
    slow tasks to sync their workspaces. Syncing uses network messages and the duration depends of how much needs to be synced, so those slow tasks ends
    in an arbitrary order rather than the strict order expected by the FSlowTask design, sometimes triggering an ensure(). */
class FConcertSlowTaskStackWorkaround
{
public:
	static FConcertSlowTaskStackWorkaround& Get()
	{
		static FConcertSlowTaskStackWorkaround Instance; // Static instance shared by DisasterRecovery and MultiUser clients.
		return Instance;
	}

	void PushTask(TUniquePtr<FScopedSlowTask>& Task)
	{
		TaskStack.Push(Task.Get()); // Track the expected order as FScopedSlowTask does.
	}

	void PopTask(TUniquePtr<FScopedSlowTask> Discarded)
	{
		if (!Discarded)
		{
			return;
		}

		if (Discarded.Get() == TaskStack.Top()) // If the slow task completed in the FScopedSlowTask expected order?
		{
			TaskStack.Pop();
			Discarded.Reset();
			while (TaskStack.Num() && ExtendedTaskLife.Num()) // Check if a task for which the life was extended is now on top (and can be discarded).
			{
				FScopedSlowTask* Top = TaskStack.Top();
				if (ExtendedTaskLife.RemoveAll([Top](const TUniquePtr<FScopedSlowTask>& Task) { return Task.Get() == Top; }) != 0)
				{
					TaskStack.Pop();
				}
				else
				{
					return;
				}
			}
		}
		else // Out of order deletion, need to extend the life of this task until it gets to the top of the stack.
		{
			ExtendedTaskLife.Add(MoveTemp(Discarded));
		}
	}

private:
	FConcertSlowTaskStackWorkaround() = default;

	TArray<FScopedSlowTask*> TaskStack;
	TArray<TUniquePtr<FScopedSlowTask>> ExtendedTaskLife;
};

namespace UE::ConcertWorkspace::Private
{
	template <size_t N>
	struct CVarString
	{
		TCHAR Storage[N+1]{};
		constexpr CVarString(TCHAR const* InStr)
		{
			for (size_t Index = 0; Index != N; Index++)
			{
				Storage[Index] = InStr[Index];
			}
		}
		constexpr operator TCHAR const*() const {return Storage;}
	};
	template<size_t N> CVarString(TCHAR const (&)[N]) -> CVarString<N - 1>;

	template<CVarString T, bool bEditorOnly>
	struct FCVarBool
	{
		static constexpr TCHAR const* CvarName = T;
		void SetValueOnce(bool bInValue, bool bForce = false)
		{
#if WITH_EDITOR
			const bool bShouldSetValue = bEditorOnly ? GEditor != nullptr : true;
			if (bShouldSetValue && (!bHasBeenSet || bForce))
			{
				int32 ValueAsInt = !!bInValue;
				SetValue(ValueAsInt);
			}
#endif
		}

		void Reset()
		{
#if WITH_EDITOR
			if (bHasBeenSet)
			{
				SetValue(bOriginalValue);
				bHasBeenSet = false;
			}
#endif
		}

		bool IsEnabled() const
		{
#if WITH_EDITOR
			static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(CvarName);
			if (CVar)
			{
				return !!CVar->GetInt();
			}
#endif
			return false;
		}

	private:
		void SetValue(int32 InValue)
		{
			static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(CvarName);
			if (CVar)
			{
				bOriginalValue = CVar->GetInt();
				CVar->Set(InValue);
				bHasBeenSet = true;
			}
		}

		int32 bOriginalValue = false;
		bool bHasBeenSet = false;
	};

	static FCVarBool<TEXT("LevelInstance.ForceEditorWorldMode"), false /*Editor Only*/> LevelInstanceForceEditorWorldMode;
	static FCVarBool<TEXT("EditorPaths.Enabled"), false /*Editor Only*/> EditorPaths;
}

struct FConcertWorkspaceConsoleCommands
{
	FConcertWorkspaceConsoleCommands() :
		EnableRemoteVerboseLogging(TEXT("Concert.SetRemoteLoggingOn"), TEXT("Send logging event to enable verbose logging on server."),
								   FConsoleCommandDelegate::CreateRaw(this, &FConcertWorkspaceConsoleCommands::EnableRemoteLogging)),
		DisableRemoteVerboseLogging(TEXT("Concert.SetRemoteLoggingOff"), TEXT("Send logging event to disable verbose logging on server."),
									FConsoleCommandDelegate::CreateRaw(this, &FConcertWorkspaceConsoleCommands::DisableRemoteLogging))
	{
	};

	TSharedPtr<IConcertClientWorkspace> GetConnectedWorkspace()
	{
		if (TSharedPtr<IConcertSyncClient> ConcertSyncClient = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser")))
		{
			TSharedPtr<IConcertClientWorkspace> Workspace = ConcertSyncClient->GetWorkspace();
			IConcertClientSession& Session = Workspace->GetSession();
			if (Session.GetConnectionStatus() == EConcertConnectionStatus::Connected)
			{
				return Workspace;
			}
		}
		return nullptr;
	}

	void SendForceResend()
	{
		TSharedPtr<IConcertClientWorkspace> Workspace = GetConnectedWorkspace();
		if (Workspace)
		{
			IConcertClientSession& Session = Workspace->GetSession();
			FConcertSendResendPending Resend;
			Session.SendCustomEvent(Resend, Session.GetSessionServerEndpointId(), EConcertMessageFlags::None);
		}
	}

	void SendRemoteLogging(bool bInValue)
	{
		TSharedPtr<IConcertClientWorkspace> Workspace = GetConnectedWorkspace();
		if (Workspace)
		{
			IConcertClientSession& Session = Workspace->GetSession();
			FConcertServerLogging Logging;
			Logging.bLoggingEnabled = bInValue;
			Session.SendCustomEvent(Logging, Session.GetSessionServerEndpointId(), EConcertMessageFlags::None);
		}
	}

	void EnableRemoteLogging()
	{
		SendRemoteLogging(true);
	}

	void DisableRemoteLogging()
	{
		SendRemoteLogging(false);
	}

	/** Console command enable verbose logging command to server. */
	FAutoConsoleCommand EnableRemoteVerboseLogging;

	/** Console command disable verbose logging command to server. */
	FAutoConsoleCommand DisableRemoteVerboseLogging;
};

FConcertClientWorkspace::FConcertClientWorkspace(const UE::ConcertSyncClient::FSessionBindArgs& SessionBindArgs, TSharedPtr<IConcertFileSharingService> InFileSharingService, IConcertSyncClient* InOwnerSyncClient)
	: OwnerSyncClient(InOwnerSyncClient)
	, FileSharingService(MoveTemp(InFileSharingService))
{
	static FConcertWorkspaceConsoleCommands ConsoleCommands;

	check(OwnerSyncClient);
	BindSession(SessionBindArgs);
}

FConcertClientWorkspace::~FConcertClientWorkspace()
{
	UnbindSession();
}

IConcertClientSession& FConcertClientWorkspace::GetSession() const
{
	return LiveSession->GetSession();
}

FGuid FConcertClientWorkspace::GetWorkspaceLockId() const
{
	return LockManager ? LockManager->GetWorkspaceLockId() : FGuid();
}

FGuid FConcertClientWorkspace::GetResourceLockId(const FName InResourceName) const
{
	return LockManager ? LockManager->GetResourceLockId(InResourceName) : FGuid();
}

bool FConcertClientWorkspace::AreResourcesLockedBy(TArrayView<const FName> ResourceNames, const FGuid& ClientId)
{
	return !LockManager || LockManager->AreResourcesLockedBy(ResourceNames, ClientId);
}

TFuture<FConcertResourceLockResponse> FConcertClientWorkspace::LockResources(TArray<FName> InResourceNames)
{
	if (LockManager)
	{
		return LockManager->LockResources(InResourceNames);
	}

	FConcertResourceLockResponse DummyResponse;
	DummyResponse.LockType = EConcertResourceLockType::Lock;
	return MakeFulfilledPromise<FConcertResourceLockResponse>(MoveTemp(DummyResponse)).GetFuture();
}

TFuture<FConcertResourceLockResponse> FConcertClientWorkspace::UnlockResources(TArray<FName> InResourceNames)
{
	if (LockManager)
	{
		return LockManager->UnlockResources(InResourceNames);
	}

	FConcertResourceLockResponse DummyResponse;
	DummyResponse.LockType = EConcertResourceLockType::Unlock;
	return MakeFulfilledPromise<FConcertResourceLockResponse>(MoveTemp(DummyResponse)).GetFuture();
}

bool FConcertClientWorkspace::HasSessionChanges() const
{
	return (TransactionManager && TransactionManager->HasSessionChanges()) || (PackageManager && PackageManager->HasSessionChanges());
}

TArray<FName> FConcertClientWorkspace::GatherSessionChanges(bool IgnorePersisted)
{
	TSet<FName> SessionChangedPackageNames;

	// Gather the packages with live transactions
	LiveSession->GetSessionDatabase().EnumeratePackageNamesWithLiveTransactions([&SessionChangedPackageNames](FName PackageName)
	{
		SessionChangedPackageNames.Add(PackageName);
		return true;
	});

	// Gather the packages with a non persisted head revision events
	LiveSession->GetSessionDatabase().EnumeratePackageNamesWithHeadRevision([&SessionChangedPackageNames](FName PackageName)
	{
		SessionChangedPackageNames.Add(PackageName);
		return true;
	}, IgnorePersisted);

	return SessionChangedPackageNames.Array();
}

TOptional<FString> FConcertClientWorkspace::GetValidPackageSessionPath(FName PackageName) const
{
#if WITH_EDITOR
	if (PackageManager)
	{
		return PackageManager->GetValidPackageSessionPath(MoveTemp(PackageName));
	}
#endif
	return {};
}


FPersistResult FConcertClientWorkspace::PersistSessionChanges(FPersistParameters InParam)
{
	bool bSuccess = false;
#if WITH_EDITOR
	if (PackageManager)
	{
		TArray<FName> PackageNames;
		for (const FName& PackageName : InParam.PackagesToPersist)
		{
			SaveLiveTransactionsToPackage(PackageName);
			PackageNames.Add(PackageName);
		}

		FPersistResult Result = PackageManager->PersistSessionChanges(MoveTemp(InParam));
		// if we successfully persisted the files, record persist events for them in the db
		if (Result.PersistStatus == EPersistStatus::Success)
		{
			int64 PersistEventId = 0;
			for (const FName& PackageName : PackageNames)
			{
				LiveSession->GetSessionDatabase().AddPersistEventForHeadRevision(
					PackageName, PersistEventId);
			}
		}
		return Result;
	}
#endif
	return {};
}

bool FConcertClientWorkspace::HasLiveTransactionSupport(UPackage* InPackage) const
{
	return TransactionManager && TransactionManager->HasLiveTransactionSupport(InPackage);
}

bool FConcertClientWorkspace::ShouldIgnorePackageDirtyEvent(class UPackage* InPackage) const
{
	return PackageManager && PackageManager->ShouldIgnorePackageDirtyEvent(InPackage);
}

bool FConcertClientWorkspace::FindTransactionEvent(const int64 TransactionEventId, FConcertSyncTransactionEvent& OutTransactionEvent, const bool bMetaDataOnly) const
{
	bool bFound = LiveSession->GetSessionDatabase().GetTransactionEvent(TransactionEventId, OutTransactionEvent, bMetaDataOnly);
	return bFound && (bMetaDataOnly || !IsTransactionEventPartiallySynced(OutTransactionEvent)); // Avoid succeeding if the event is partially sync but full event data was requested.
}

TFuture<TOptional<FConcertSyncTransactionEvent>> FConcertClientWorkspace::FindOrRequestTransactionEvent(const int64 TransactionEventId, const bool bMetaDataOnly)
{
	FConcertSyncTransactionEvent TransactionEvent;

	// Check if the event exist in the database.
	if (LiveSession->GetSessionDatabase().GetTransactionEvent(TransactionEventId, TransactionEvent, bMetaDataOnly))
	{
		// If the transaction data is required and the event has only the meta data (partially synced, the event was superseded by another).
		if (!bMetaDataOnly && IsTransactionEventPartiallySynced(TransactionEvent))
		{
			FConcertSyncEventRequest SyncEventRequest{EConcertSyncActivityEventType::Transaction, TransactionEventId };
			TWeakPtr<FConcertSyncClientLiveSession> WeakLiveSession = LiveSession;
			return LiveSession->GetSession().SendCustomRequest<FConcertSyncEventRequest, FConcertSyncEventResponse>(SyncEventRequest, LiveSession->GetSession().GetSessionServerEndpointId()).Next([WeakLiveSession, TransactionEventId](const FConcertSyncEventResponse& Response)
			{
				if (Response.Event.PayloadSize > 0) // Some data was sent back?
				{
					// Extract the payload as FConcertSyncTransactionEvent.
					FStructOnScope EventPayload;
					Response.Event.GetPayload(EventPayload);
					check(EventPayload.IsValid() && EventPayload.GetStruct()->IsChildOf(FConcertSyncTransactionEvent::StaticStruct()));
					FConcertSyncTransactionEvent* TransactionEvent = (FConcertSyncTransactionEvent*)EventPayload.GetStructMemory();

					// Update the database, caching the event to avoid syncing again.
					if (TSharedPtr<FConcertSyncClientLiveSession> LiveSessionPin = WeakLiveSession.Pin())
					{
						LiveSessionPin->GetSessionDatabase().UpdateTransactionEvent(TransactionEventId, *TransactionEvent);
						// NOTE: PostActivityUpdated() could be called, but the activity did not change, more info was simply cached locally. Unless a use case requires it, don't call it.
					}

					return TOptional<FConcertSyncTransactionEvent>(MoveTemp(*TransactionEvent));
				}
				else
				{
					return TOptional<FConcertSyncTransactionEvent>(); // The server did not return any data.
				}
			});
		}
		else
		{
			return MakeFulfilledPromise<TOptional<FConcertSyncTransactionEvent>>(MoveTemp(TransactionEvent)).GetFuture(); // All required data was already available locally.
		}
	}

	return MakeFulfilledPromise<TOptional<FConcertSyncTransactionEvent>>().GetFuture(); // Not found.
}

bool FConcertClientWorkspace::FindPackageEvent(const int64 PackageEventId, FConcertSyncPackageEventMetaData& OutPackageEvent) const
{
	return LiveSession->GetSessionDatabase().GetPackageEventMetaData(PackageEventId, OutPackageEvent.PackageRevision, OutPackageEvent.PackageInfo);
}

bool FConcertClientWorkspace::FindReplicationEvent(const int64 ReplicationEventId, FConcertSyncReplicationEvent& OutReplicationEvent) const
{
	return LiveSession->GetSessionDatabase().GetReplicationEvent(ReplicationEventId, OutReplicationEvent);
}

bool FConcertClientWorkspace::IsTransactionEventPartiallySynced(const FConcertSyncTransactionEvent& TransactionEvent) const
{
	return TransactionEvent.Transaction.ExportedObjects.Num() == 0;
}

void FConcertClientWorkspace::GetActivities(const int64 FirstActivityIdToFetch, const int64 MaxNumActivities, TMap<FGuid, FConcertClientInfo>& OutEndpointClientInfoMap, TArray<FConcertSessionActivity>& OutActivities) const
{
	OutEndpointClientInfoMap.Reset();
	OutActivities.Reset();
	LiveSession->GetSessionDatabase().EnumerateActivitiesInRange(FirstActivityIdToFetch, MaxNumActivities, [this, &OutEndpointClientInfoMap, &OutActivities](FConcertSyncActivity&& InActivity)
	{
		if (!OutEndpointClientInfoMap.Contains(InActivity.EndpointId))
		{
			FConcertSyncEndpointData EndpointData;
			if (LiveSession->GetSessionDatabase().GetEndpoint(InActivity.EndpointId, EndpointData))
			{
				OutEndpointClientInfoMap.Add(InActivity.EndpointId, EndpointData.ClientInfo);
			}
		}

		FStructOnScope ActivitySummary;
		if (InActivity.EventSummary.GetPayload(ActivitySummary))
		{
			OutActivities.Emplace(MoveTemp(InActivity), MoveTemp(ActivitySummary));
		}

		return EBreakBehavior::Continue;
	});
}

int64 FConcertClientWorkspace::GetLastActivityId() const
{
	int64 ActivityMaxId = 0;
	LiveSession->GetSessionDatabase().GetActivityMaxId(ActivityMaxId);
	return ActivityMaxId;
}

FOnActivityAddedOrUpdated& FConcertClientWorkspace::OnActivityAddedOrUpdated()
{
	return OnActivityAddedOrUpdatedDelegate;
}

FOnWorkspaceSynchronized& FConcertClientWorkspace::OnWorkspaceSynchronized()
{
	return OnWorkspaceSyncedDelegate;
}

FOnFinalizeWorkspaceSyncCompleted& FConcertClientWorkspace::OnFinalizeWorkspaceSyncCompleted()
{
	return OnFinalizeWorkspaceSyncCompletedDelegate;
}

FOnWorkspaceEndFrameCompleted& FConcertClientWorkspace::OnWorkspaceEndFrameCompleted()
{
	return OnWorkspaceEndFrameCompletedDelegate;
}

IConcertClientDataStore& FConcertClientWorkspace::GetDataStore()
{
	return *DataStore;
}

void FConcertClientWorkspace::BindSession(const UE::ConcertSyncClient::FSessionBindArgs& SessionBindArgs)
{
	check(SessionBindArgs.IsValid());

	UnbindSession();
	LiveSession = SessionBindArgs.LiveSession;
	PackageBridge = SessionBindArgs.Bridges.PackageBridge;

	LoadSessionData();

	bHasSyncedWorkspace = false;
	bFinalizeWorkspaceSyncRequested = false;

	// Provide access to the data store (shared by session clients) maintained by the server.
	DataStore = MakeUnique<FConcertClientDataStore>(LiveSession.ToSharedRef());

	// Create Transaction Manager
	if (EnumHasAnyFlags(LiveSession->GetSessionFlags(), EConcertSyncSessionFlags::EnableTransactions))
	{
		TransactionManager = MakeUnique<FConcertClientTransactionManager>(LiveSession.ToSharedRef(), SessionBindArgs.Bridges.TransactionBridge);
	}

	// Create Package Manager
	if (EnumHasAnyFlags(LiveSession->GetSessionFlags(), EConcertSyncSessionFlags::EnablePackages))
	{
		PackageManager = MakeUnique<FConcertClientPackageManager>(
			LiveSession.ToSharedRef(), SessionBindArgs.Bridges.PackageBridge, FileSharingService, OwnerSyncClient->GetConcertClient()->GetRole()
			);
	}

	// Create Lock Manager
	if (EnumHasAnyFlags(LiveSession->GetSessionFlags(), EConcertSyncSessionFlags::EnableLocking))
	{
		LockManager = MakeUnique<FConcertClientLockManager>(LiveSession.ToSharedRef());
	}

	// Register Session events
	LiveSession->GetSession().OnConnectionChanged().AddRaw(this, &FConcertClientWorkspace::HandleConnectionChanged);

#if WITH_EDITOR
	if (EnumHasAnyFlags(LiveSession->GetSessionFlags(), EConcertSyncSessionFlags::EnableTransactions))
	{
		// Register Asset Load Events
		FCoreUObjectDelegates::OnAssetLoaded.AddRaw(this, &FConcertClientWorkspace::HandleAssetLoaded);

		if (EnumHasAnyFlags(LiveSession->GetSessionFlags(), EConcertSyncSessionFlags::ShouldDiscardTransactionsOnPackageUnload))
		{
			// Register Package Discarded Events
			PackageBridge->OnLocalPackageDiscarded().AddRaw(this, &FConcertClientWorkspace::HandlePackageDiscarded);
		}
	}

	// Register PIE/SIE Events
	FEditorDelegates::PostPIEStarted.AddRaw(this, &FConcertClientWorkspace::HandlePostPIEStarted);
	FEditorDelegates::OnSwitchBeginPIEAndSIE.AddRaw(this, &FConcertClientWorkspace::HandleSwitchBeginPIEAndSIE);
	FEditorDelegates::EndPIE.AddRaw(this, &FConcertClientWorkspace::HandleEndPIE);
#endif

	// Register OnEndFrame events
	FCoreDelegates::OnEndFrame.AddRaw(this, &FConcertClientWorkspace::OnEndFrame);

	// Register workspace event
	LiveSession->GetSession().RegisterCustomEventHandler<FConcertWorkspaceSyncEndpointEvent>(this, &FConcertClientWorkspace::HandleWorkspaceSyncEndpointEvent);
	LiveSession->GetSession().RegisterCustomEventHandler<FConcertWorkspaceSyncActivityEvent>(this, &FConcertClientWorkspace::HandleWorkspaceSyncActivityEvent);
	LiveSession->GetSession().RegisterCustomEventHandler<FConcertWorkspaceSyncLockEvent>(this, &FConcertClientWorkspace::HandleWorkspaceSyncLockEvent);
	LiveSession->GetSession().RegisterCustomEventHandler<FConcertWorkspaceSyncCompletedEvent>(this, &FConcertClientWorkspace::HandleWorkspaceSyncCompletedEvent);
}

void FConcertClientWorkspace::UnbindSession()
{
	if (LiveSession)
	{
		SaveSessionData();

		// Destroy Transaction Authors
		LiveTransactionAuthors.Reset();

		// Destroy Lock Manager
		LockManager.Reset();

		// Destroy Package Manager
		PackageManager.Reset();

		// Destroy Transaction Manager
		TransactionManager.Reset();

		// Unregister Session events
		LiveSession->GetSession().OnConnectionChanged().RemoveAll(this);

#if WITH_EDITOR
		// Unregister Asset Load Events
		FCoreUObjectDelegates::OnAssetLoaded.RemoveAll(this);

		// Unregister Package Discarded Events
		PackageBridge->OnLocalPackageDiscarded().RemoveAll(this);

		// Unregister PIE/SIE Events
		FEditorDelegates::PostPIEStarted.RemoveAll(this);
		FEditorDelegates::OnSwitchBeginPIEAndSIE.RemoveAll(this);
		FEditorDelegates::EndPIE.RemoveAll(this);
#endif

		// Unregister OnEndFrame events
		FCoreDelegates::OnEndFrame.RemoveAll(this);

		// Unregister workspace event
		LiveSession->GetSession().UnregisterCustomEventHandler<FConcertWorkspaceSyncEndpointEvent>(this);
		LiveSession->GetSession().UnregisterCustomEventHandler<FConcertWorkspaceSyncActivityEvent>(this);
		LiveSession->GetSession().UnregisterCustomEventHandler<FConcertWorkspaceSyncLockEvent>(this);
		LiveSession->GetSession().UnregisterCustomEventHandler<FConcertWorkspaceSyncCompletedEvent>(this);

		DataStore.Reset();
		LiveSession.Reset();
		PackageBridge = nullptr;
	}
}

void FConcertClientWorkspace::LoadSessionData()
{
	FString ClientWorkspaceDataPath = LiveSession->GetSession().GetSessionWorkingDirectory() / TEXT("WorkspaceData.json");
	if (TUniquePtr<FArchive> FileReader = TUniquePtr<FArchive>(IFileManager::Get().CreateFileReader(*ClientWorkspaceDataPath)))
	{
		FJsonStructDeserializerBackend Backend(*FileReader);
		FStructDeserializer::Deserialize<FConcertClientWorkspaceData>(SessionData, Backend);
		FileReader->Close();
	}
	// if the loaded session data doesn't match the session clear everything
	if (SessionData.SessionIdentifier != LiveSession->GetSession().GetSessionServerEndpointId())
	{
		SessionData.SessionIdentifier.Invalidate();
	}
}

void FConcertClientWorkspace::SaveSessionData()
{
	SessionData.SessionIdentifier = LiveSession->GetSession().GetSessionServerEndpointId();
	
	FString ClientWorkspaceDataPath = LiveSession->GetSession().GetSessionWorkingDirectory() / TEXT("WorkspaceData.json");
	if (TUniquePtr<FArchive> FileWriter = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*ClientWorkspaceDataPath)))
	{
		FJsonStructSerializerBackend Backend(*FileWriter, EStructSerializerBackendFlags::Default);
		FStructSerializer::Serialize<FConcertClientWorkspaceData>(SessionData, Backend);
		FileWriter->Close();
	}
}

void FConcertClientWorkspace::HandleConnectionChanged(IConcertClientSession& InSession, EConcertConnectionStatus Status)
{
	check(&LiveSession->GetSession() == &InSession);

	if (Status == EConcertConnectionStatus::Connected)
	{
		if (FileSharingService)
		{
			const FConcertSessionInfo& SessionInfo = InSession.GetSessionInfo();
			FileSharingService->SetEnabled(SessionInfo.FileSharing.bFileShareEnabled);
			if (SessionInfo.FileSharing.bFileShareEnabled)
			{
				FString WorkspacePath = SessionInfo.FileSharing.FileShareRoot / SessionInfo.SessionName / LiveSession->GetSession().GetSessionClientEndpointId().ToString();
				FileSharingService->SetPath(WorkspacePath);
			}

			UE_LOGF(LogConcert, Display, "File sharing is %ls for session workspace.", FileSharingService->IsEnabled() ? TEXT("enabled") : TEXT("disabled") );
		}
		
		bHasSyncedWorkspace = false;
		bFinalizeWorkspaceSyncRequested = false;
		InitialSyncSlowTask = MakeUnique<FScopedSlowTask>(1.0f, LOCTEXT("SynchronizingSession", "Synchronizing Session..."));
		InitialSyncSlowTask->MakeDialogDelayed(1.0f);

		FConcertSlowTaskStackWorkaround::Get().PushTask(InitialSyncSlowTask);

		// Request our initial workspace sync for any new activity since we last joined
		{
			FConcertWorkspaceSyncRequestedEvent SyncRequestedEvent;
			LiveSession->GetSessionDatabase().GetActivityMaxId(SyncRequestedEvent.FirstActivityIdToSync);
			SyncRequestedEvent.FirstActivityIdToSync++;
			SyncRequestedEvent.bEnableLiveSync = EnumHasAnyFlags(LiveSession->GetSessionFlags(), EConcertSyncSessionFlags::EnableLiveSync);
			LiveSession->GetSession().SendCustomEvent(SyncRequestedEvent, LiveSession->GetSession().GetSessionServerEndpointId(), EConcertMessageFlags::ReliableOrdered);
		}

#if WITH_EDITOR
		if (GUnrealEd)
		{
			if (FWorldContext* PIEWorldContext = GUnrealEd->GetPIEWorldContext())
			{
				if (UWorld* PIEWorld = PIEWorldContext->World())
				{
					// Track open PIE/SIE sessions so the server can discard them once everyone leaves
					FConcertPlaySessionEvent PlaySessionEvent;
					PlaySessionEvent.EventType = EConcertPlaySessionEventType::BeginPlay;
					PlaySessionEvent.PlayEndpointId = LiveSession->GetSession().GetSessionClientEndpointId();
					PlaySessionEvent.PlayPackageName = PIEWorld->GetOutermost()->GetFName();
					PlaySessionEvent.bIsSimulating = GUnrealEd->bIsSimulatingInEditor;
					LiveSession->GetSession().SendCustomEvent(PlaySessionEvent, LiveSession->GetSession().GetSessionServerEndpointId(), EConcertMessageFlags::ReliableOrdered);
				}
			}
		}
		if (GEditor == nullptr)
		{
			const bool bLevelInstanceEditorWorldMode = UE::ConcertWorkspace::Private::LevelInstanceForceEditorWorldMode.IsEnabled();
			UE_CLOGF( !bLevelInstanceEditorWorldMode, LogConcert, Warning, "Level Instance Editor World Mode is off. Ensure LevelInstance.ForceEditorWorldMode=1 on startup. Non-editor world mode in -game may cause issues with transaction playback on Level Instance Actors. Forcing this mode to be ON." );
		}

		UE::ConcertWorkspace::Private::LevelInstanceForceEditorWorldMode.SetValueOnce(true);
		UE::ConcertWorkspace::Private::EditorPaths.SetValueOnce(true);
#endif
	}
	else if (Status == EConcertConnectionStatus::Disconnected)
	{
		bHasSyncedWorkspace = false;
		bFinalizeWorkspaceSyncRequested = false;
		FConcertSlowTaskStackWorkaround::Get().PopTask(MoveTemp(InitialSyncSlowTask));
		UE::ConcertWorkspace::Private::LevelInstanceForceEditorWorldMode.Reset();
		UE::ConcertWorkspace::Private::EditorPaths.Reset();

		if (FileSharingService && FileSharingService->IsEnabled())
		{
			IConcertClientRef ConcertClient = OwnerSyncClient->GetConcertClient();
			if (ConcertClient->GetConfiguration()->ClientSettings.bCleanSessionFilesOnDisconnect)
			{
				FileSharingService->Delete(FileSharingService->GetPath());
			}
		}
	}
}

#if WITH_EDITOR

void FConcertClientWorkspace::SaveLiveTransactionsToPackages()
{
	// Save any packages that have live transactions
	if (GEditor && TransactionManager)
	{
		LiveSession->GetSessionDatabase().EnumeratePackageNamesWithLiveTransactions([this](const FName PackageName)
		{
			SaveLiveTransactionsToPackage(PackageName);
			return true;
		});
	}
}

void FConcertClientWorkspace::SaveLiveTransactionsToPackage(const FName PackageName)
{
	if (GEditor && TransactionManager)
	{
		bool bHasLiveTransactions = false;
		if (LiveSession->GetSessionDatabase().PackageHasLiveTransactions(PackageName, bHasLiveTransactions)
			&& bHasLiveTransactions)
		{
			// Ignore these package saves as the other clients should already be in-sync
			IConcertClientPackageBridge::FScopedIgnoreLocalSave IgnorePackageSaveScope(*PackageBridge);

			const FString PackageNameStr = PackageName.ToString();
			UPackage* Package = LoadPackage(nullptr, *PackageNameStr, LOAD_None);
			if (Package)
			{
				// Load package will queue live transaction, process them before saving the file
				TransactionManager->ProcessPending();

				UWorld* World = UWorld::FindWorldInPackage(Package);
				FString PackageFilename;
				if (!FPackageName::DoesPackageExist(PackageNameStr, &PackageFilename))
				{
					PackageFilename = FPackageName::LongPackageNameToFilename(PackageNameStr, World ? FPackageName::GetMapPackageExtension() : FPackageName::GetAssetPackageExtension());
				}

				FSavePackageArgs SaveArgs;
				SaveArgs.TopLevelFlags = RF_Standalone;
				SaveArgs.Error = GWarn;
				if (GEditor->SavePackage(Package, World, *PackageFilename, SaveArgs))
				{
					// Add a dummy package entry to trim the live transaction for the saved package but ONLY if we're tracking package saves (ie, we have a package manager)
					// This is added ONLY on this client, and will be CLOBBERED by any future saves of this package from the server!
					if (PackageManager)
					{
						int64 PackageEventId = 0;
						LiveSession->GetSessionDatabase().AddDummyPackageEvent(PackageName, PackageEventId);
					}
				}
				else
				{
					UE_LOGF(LogConcert, Warning, "Failed to save package '%ls' when persiting sandbox state!", *PackageNameStr);
				}
			}
		}
	}
}

void FConcertClientWorkspace::HandleAssetLoaded(UObject* InAsset)
{
	if (TransactionManager && bHasSyncedWorkspace)
	{
		const FName LoadedPackageName = InAsset->GetOutermost()->GetFName();
		TransactionManager->ReplayTransactions(LoadedPackageName);
	}
}

void FConcertClientWorkspace::HandlePackageDiscarded(UPackage* InPackage)
{
	if (bHasSyncedWorkspace && EnumHasAllFlags(LiveSession->GetSessionFlags(), EConcertSyncSessionFlags::EnableTransactions | EConcertSyncSessionFlags::ShouldDiscardTransactionsOnPackageUnload))
	{
		const FName PackageName = InPackage->GetFName();

		// Add a dummy package entry to trim the live transaction for the discarded world
		// This is added ONLY on this client, and will be CLOBBERED by any future saves of this package from the server!
		// We always do this, even if the client is tracking package changes, as we may be in the middle of an action that 
		// needs to fence transactions immediately and can't wait for the activity to be returned from the server
		int64 PackageEventId = 0;
		LiveSession->GetSessionDatabase().AddDummyPackageEvent(PackageName, PackageEventId);

		// Client is tracking package events, so also discard the changes made to this package for everyone in the session
		if (PackageManager)
		{
			PackageManager->HandlePackageDiscarded(InPackage);
		}
	}
}

void FConcertClientWorkspace::HandlePostPIEStarted(const bool InIsSimulating)
{
	if (FWorldContext* PIEWorldContext = GUnrealEd->GetPIEWorldContext())
	{
		if (UWorld* PIEWorld = PIEWorldContext->World())
		{
			// Track open PIE/SIE sessions so the server can discard them once everyone leaves
			FConcertPlaySessionEvent PlaySessionEvent;
			PlaySessionEvent.EventType = EConcertPlaySessionEventType::BeginPlay;
			PlaySessionEvent.PlayEndpointId = LiveSession->GetSession().GetSessionClientEndpointId();
			PlaySessionEvent.PlayPackageName = PIEWorld->GetOutermost()->GetFName();
			PlaySessionEvent.bIsSimulating = InIsSimulating;
			LiveSession->GetSession().SendCustomEvent(PlaySessionEvent, LiveSession->GetSession().GetSessionServerEndpointId(), EConcertMessageFlags::ReliableOrdered);

			// Apply transactions to the PIE/SIE world
			HandleAssetLoaded(PIEWorld);
		}
	}
}

void FConcertClientWorkspace::HandleSwitchBeginPIEAndSIE(const bool InIsSimulating)
{
	if (FWorldContext* PIEWorldContext = GUnrealEd->GetPIEWorldContext())
	{
		if (UWorld* PIEWorld = PIEWorldContext->World())
		{
			// Track open PIE/SIE sessions so the server can discard them once everyone leaves
			FConcertPlaySessionEvent PlaySessionEvent;
			PlaySessionEvent.EventType = EConcertPlaySessionEventType::SwitchPlay;
			PlaySessionEvent.PlayEndpointId = LiveSession->GetSession().GetSessionClientEndpointId();
			PlaySessionEvent.PlayPackageName = PIEWorld->GetOutermost()->GetFName();
			PlaySessionEvent.bIsSimulating = InIsSimulating;
			LiveSession->GetSession().SendCustomEvent(PlaySessionEvent, LiveSession->GetSession().GetSessionServerEndpointId(), EConcertMessageFlags::ReliableOrdered);
		}
	}
}

void FConcertClientWorkspace::HandleEndPIE(const bool InIsSimulating)
{
	if (FWorldContext* PIEWorldContext = GUnrealEd->GetPIEWorldContext())
	{
		if (UWorld* PIEWorld = PIEWorldContext->World())
		{
			// Track open PIE/SIE sessions so the server can discard them once everyone leaves
			FConcertPlaySessionEvent PlaySessionEvent;
			PlaySessionEvent.EventType = EConcertPlaySessionEventType::EndPlay;
			PlaySessionEvent.PlayEndpointId = LiveSession->GetSession().GetSessionClientEndpointId();
			PlaySessionEvent.PlayPackageName = PIEWorld->GetOutermost()->GetFName();
			PlaySessionEvent.bIsSimulating = InIsSimulating;
			LiveSession->GetSession().SendCustomEvent(PlaySessionEvent, LiveSession->GetSession().GetSessionServerEndpointId(), EConcertMessageFlags::ReliableOrdered);
		}
	}
}

#endif	// WITH_EDITOR

void FConcertClientWorkspace::OnEndFrame()
{
	LLM_SCOPE_BYTAG(Concert_ConcertClientWorkspace);
	SCOPED_CONCERT_TRACE(FConcertClientWorkspace_OnEndFrame);

	if (CanFinalize())
	{
		bFinalizeWorkspaceSyncRequested = false;

		// Start tracking changes made by other users
		check(!LiveTransactionAuthors);
		LiveTransactionAuthors = MakeUnique<FConcertClientLiveTransactionAuthors>(LiveSession.ToSharedRef());

		// Make sure any new packages are loaded
		if (InitialSyncSlowTask.IsValid())
		{
			InitialSyncSlowTask->EnterProgressFrame(0.0f, LOCTEXT("ApplyingSynchronizedPackages", "Applying Synchronized Packages..."));
		}
		if (PackageManager)
		{
			PackageManager->QueueDirtyPackagesForReload();
			PackageManager->ApplyAllHeadPackageData();
			PackageManager->SynchronizeInMemoryPackages();
		}

		// Replay any "live" transactions
		if (InitialSyncSlowTask.IsValid())
		{
			InitialSyncSlowTask->EnterProgressFrame(0.0f, LOCTEXT("ApplyingSynchronizedTransactions", "Applying Synchronized Transactions..."));
		}
		if (TransactionManager)
		{
			TransactionManager->ReplayAllTransactions();

			// We process all pending transactions we just replayed before finalizing the sync to prevent package being loaded as a result to trigger replaying transactions again
			TransactionManager->ProcessPending();
		}

		// Finalize the sync
		bHasSyncedWorkspace = true;
		FConcertSlowTaskStackWorkaround::Get().PopTask(MoveTemp(InitialSyncSlowTask));
		OnFinalizeWorkspaceSyncCompletedDelegate.Broadcast();

		// Notify the server that we've finalized our workspace.
		LiveSession->GetSession().SendCustomEvent(FConcertWorkspaceSyncAndFinalizeCompletedEvent(), LiveSession->GetSession().GetSessionServerEndpointId(), EConcertMessageFlags::ReliableOrdered);
	}

	if (bHasSyncedWorkspace && CanProcessPendingPackages() && !ConcertSyncClientUtil::ShouldDelayTransaction())
	{
		if (PackageManager)
		{
			PackageManager->SynchronizeInMemoryPackages();
		}

		if (TransactionManager)
		{
			TransactionManager->ProcessPending();
		}

		if (bPendingStopIgnoringActivityOnRestore)
		{
			FConcertIgnoreActivityStateChangedEvent StateChangeEvent{LiveSession->GetSession().GetSessionClientEndpointId(), /*bIgnore*/false};
			LiveSession->GetSession().SendCustomEvent(StateChangeEvent, LiveSession->GetSession().GetSessionServerEndpointId(), EConcertMessageFlags::ReliableOrdered);
			bPendingStopIgnoringActivityOnRestore = false;
		}

		OnWorkspaceEndFrameCompletedDelegate.Broadcast();
	}
	LiveSession->GetSessionDatabase().UpdateAsynchronousTasks();
	IConcertClientRef ConcertClient = OwnerSyncClient->GetConcertClient();
}

void FConcertClientWorkspace::HandleWorkspaceSyncEndpointEvent(const FConcertSessionContext& Context, const FConcertWorkspaceSyncEndpointEvent& Event)
{
	// Update slow task dialog
	if (InitialSyncSlowTask.IsValid())
	{
		InitialSyncSlowTask->TotalAmountOfWork = InitialSyncSlowTask->CompletedWork + Event.NumRemainingSyncEvents + 1;
		InitialSyncSlowTask->EnterProgressFrame(FMath::Min<float>(Event.NumRemainingSyncEvents, 1.0f), FText::Format(LOCTEXT("SynchronizedEndpointFmt", "Synchronized User {0}..."), FText::AsCultureInvariant(Event.Endpoint.EndpointData.ClientInfo.DisplayName)));
	}

	// Set endpoint in database
	SetEndpoint(Event.Endpoint.EndpointId, Event.Endpoint.EndpointData);
}

void FConcertClientWorkspace::HandleWorkspaceSyncActivityEvent(const FConcertSessionContext& Context, const FConcertWorkspaceSyncActivityEvent& Event)
{
	SCOPED_CONCERT_TRACE(FConcertClientWorkspace_HandleWorkspaceSyncActivityEvent);

	FStructOnScope ActivityPayload;
	Event.Activity.GetPayload(ActivityPayload);

	check(ActivityPayload.IsValid() && ActivityPayload.GetStruct()->IsChildOf(FConcertSyncActivity::StaticStruct()));
	FConcertSyncActivity* Activity = (FConcertSyncActivity*)ActivityPayload.GetStructMemory();
	ensureAlwaysMsgf((Activity->Flags & EConcertSyncActivityFlags::Muted) == EConcertSyncActivityFlags::None, TEXT("Clients are not supposed to receive muted activities!"));

	// Update slow task dialog
	if (InitialSyncSlowTask.IsValid())
	{
		InitialSyncSlowTask->TotalAmountOfWork = InitialSyncSlowTask->CompletedWork + Event.NumRemainingSyncEvents + 1;
		InitialSyncSlowTask->EnterProgressFrame(FMath::Min<float>(Event.NumRemainingSyncEvents, 1.0f), FText::Format(LOCTEXT("SynchronizedActivityFmt", "Synchronized Activity {0}..."), Activity->ActivityId));
	}

	// Handle the activity correctly
	static_assert(static_cast<uint8>(EConcertSyncActivityEventType::Count) == 6, "If you added an EConcertSyncActivityEventType entry, update this switch");
	switch (Activity->EventType)
	{
	case EConcertSyncActivityEventType::Connection:
		check(ActivityPayload.GetStruct()->IsChildOf(FConcertSyncConnectionActivity::StaticStruct()));
		SetConnectionActivity(*(FConcertSyncConnectionActivity*)Activity);
		break;

	case EConcertSyncActivityEventType::Lock:
		check(ActivityPayload.GetStruct()->IsChildOf(FConcertSyncLockActivity::StaticStruct()));
		SetLockActivity(*(FConcertSyncLockActivity*)Activity);
		break;

	case EConcertSyncActivityEventType::Transaction:
		check(ActivityPayload.GetStruct()->IsChildOf(FConcertSyncTransactionActivity::StaticStruct()));
		SetTransactionActivity(*(FConcertSyncTransactionActivity*)Activity);
		break;

	case EConcertSyncActivityEventType::Package:
		check(ActivityPayload.GetStruct()->IsChildOf(FConcertSyncPackageActivity::StaticStruct()));
		SetPackageActivity(*(FConcertSyncPackageActivity*)Activity);
		break;

	case EConcertSyncActivityEventType::Replication:
		check(ActivityPayload.GetStruct()->IsChildOf(FConcertSyncReplicationActivity::StaticStruct()));
		SetReplicationActivity(*(FConcertSyncReplicationActivity*)Activity);
		break;

	default:
		checkf(false, TEXT("Unhandled EConcertSyncActivityEventType when syncing session activity"));
		break;
	}
}

void FConcertClientWorkspace::HandleWorkspaceSyncLockEvent(const FConcertSessionContext& Context, const FConcertWorkspaceSyncLockEvent& Event)
{
	// Initial sync of the locked resources
	if (LockManager)
	{
		LockManager->SetLockedResources(Event.LockedResources);
	}
}

void FConcertClientWorkspace::HandleWorkspaceSyncCompletedEvent(const FConcertSessionContext& Context, const FConcertWorkspaceSyncCompletedEvent& Event)
{
	// Request the sync to finalize at the end of the next frame
	bFinalizeWorkspaceSyncRequested = true;
	OnWorkspaceSyncedDelegate.Broadcast();
}

bool FConcertClientWorkspace::IsAssetModifiedByOtherClients(const FName& AssetName, int32* OutOtherClientsWithModifNum, TArray<FConcertClientInfo>* OutOtherClientsWithModifInfo, int32 OtherClientsWithModifMaxFetchNum) const
{
	return LiveTransactionAuthors && LiveTransactionAuthors->IsPackageAuthoredByOtherClients(AssetName, OutOtherClientsWithModifNum, OutOtherClientsWithModifInfo, OtherClientsWithModifMaxFetchNum);
}

void FConcertClientWorkspace::SetEndpoint(const FGuid& InEndpointId, const FConcertSyncEndpointData& InEndpointData)
{
	// Update this endpoint
	if (!LiveSession->GetSessionDatabase().SetEndpoint(InEndpointId, InEndpointData))
	{
		UE_LOGF(LogConcert, Error, "Failed to set endpoint '%ls' on live session '%ls': %ls", *InEndpointId.ToString(), *LiveSession->GetSession().GetName(), *LiveSession->GetSessionDatabase().GetLastError());
	}
}

void FConcertClientWorkspace::SetConnectionActivity(const FConcertSyncConnectionActivity& InConnectionActivity)
{
	// Update this activity
	if (LiveSession->GetSessionDatabase().SetConnectionActivity(InConnectionActivity))
	{
		PostActivityUpdated(InConnectionActivity);
	}
	else
	{
		UE_LOGF(LogConcert, Error, "Failed to set connection activity '%ls' on live session '%ls': %ls", *LexToString(InConnectionActivity.ActivityId), *LiveSession->GetSession().GetName(), *LiveSession->GetSessionDatabase().GetLastError());
	}
}

void FConcertClientWorkspace::SetLockActivity(const FConcertSyncLockActivity& InLockActivity)
{
	// Update this activity
	if (LiveSession->GetSessionDatabase().SetLockActivity(InLockActivity))
	{
		PostActivityUpdated(InLockActivity);
	}
	else
	{
		UE_LOGF(LogConcert, Error, "Failed to set lock activity '%ls' on live session '%ls': %ls", *LexToString(InLockActivity.ActivityId), *LiveSession->GetSession().GetName(), *LiveSession->GetSessionDatabase().GetLastError());
	}
}

void FConcertClientWorkspace::SetTransactionActivity(const FConcertSyncTransactionActivity& InTransactionActivity)
{
	// Update this activity
	if (LiveSession->GetSessionDatabase().SetTransactionActivity(InTransactionActivity))
	{
		PostActivityUpdated(InTransactionActivity);
		if (TransactionManager)
		{
			TransactionManager->HandleRemoteTransaction(InTransactionActivity.EndpointId, InTransactionActivity.EventId, bHasSyncedWorkspace);
		}
		if (LiveTransactionAuthors)
		{
			LiveTransactionAuthors->AddLiveTransactionActivity(InTransactionActivity.EndpointId, InTransactionActivity.EventData.Transaction.ModifiedPackages);
		}
	}
	else
	{
		UE_LOGF(LogConcert, Error, "Failed to set transaction activity '%ls' on live session '%ls': %ls", *LexToString(InTransactionActivity.ActivityId), *LiveSession->GetSession().GetName(), *LiveSession->GetSessionDatabase().GetLastError());
	}
}

void FConcertClientWorkspace::SetPackageActivity(const FConcertSyncPackageActivity& InPackageActivity)
{
	const FConcertSyncActivity& PackageActivityBasePart = InPackageActivity; // The base activity part correspond the the base class.
	FConcertSyncPackageEventData PackageActivityEventPart; // The event part is going to be streamed in the database (because it can be huge)
	PackageActivityEventPart.MetaData.PackageRevision = InPackageActivity.EventData.PackageRevision;
	PackageActivityEventPart.MetaData.PackageInfo = InPackageActivity.EventData.Package.Info;

	auto SetPackageActivityFn = [&](const FConcertSyncActivity& BaseActivityPart, FConcertSyncPackageEventData& EventDataPart)
	{
		// Update this activity
		if (LiveSession->GetSessionDatabase().SetPackageActivity(BaseActivityPart, EventDataPart, /*bMetaDataOnly*/false))
		{
			PostActivityUpdated(BaseActivityPart);
			if (PackageManager)
			{
				PackageManager->HandleRemotePackage(InPackageActivity.EndpointId, InPackageActivity.EventId, bHasSyncedWorkspace);
			}
			if (LiveTransactionAuthors)
			{
				LiveTransactionAuthors->ResolveLiveTransactionAuthorsForPackage(InPackageActivity.EventData.Package.Info.PackageName);
			}
		}
		else
		{
			UE_LOGF(LogConcert, Error, "Failed to set package activity '%ls' on live session '%ls': %ls", *LexToString(InPackageActivity.ActivityId), *LiveSession->GetSession().GetName(), *LiveSession->GetSessionDatabase().GetLastError());
		}
	};

	// Decide how the package data (part of the package event) is going to be streamed in the database.
	if (!InPackageActivity.EventData.Package.FileId.IsEmpty())
	{
		TSharedPtr<FArchive> PackageDataAr = FileSharingService->CreateReader(InPackageActivity.EventData.Package.FileId); // Package data is available as a shared file.
		PackageActivityEventPart.PackageDataStream.DataAr = PackageDataAr.Get();
		PackageActivityEventPart.PackageDataStream.DataSize = PackageDataAr ? PackageDataAr->TotalSize() : 0;
		SetPackageActivityFn(PackageActivityBasePart, PackageActivityEventPart);
	}
	else
	{
		FMemoryReader Ar(InPackageActivity.EventData.Package.PackageData.Bytes); // Package data is embedded in the activity itself.
		PackageActivityEventPart.PackageDataStream.DataAr = &Ar;
		PackageActivityEventPart.PackageDataStream.DataSize = Ar.TotalSize();
		PackageActivityEventPart.PackageDataStream.DataBlob = &InPackageActivity.EventData.Package.PackageData.Bytes;
		SetPackageActivityFn(PackageActivityBasePart, PackageActivityEventPart);
	}
}

void FConcertClientWorkspace::SetReplicationActivity(const FConcertSyncReplicationActivity& InReplicationActivity)
{
	// Update this activity
	if (LiveSession->GetSessionDatabase().SetReplicationActivity(InReplicationActivity))
	{
		PostActivityUpdated(InReplicationActivity);
	}
	else
	{
		UE_LOGF(LogConcert, Error, "Failed to set replication activity '%ls' on live session '%ls': %ls", *LexToString(InReplicationActivity.ActivityId), *LiveSession->GetSession().GetName(), *LiveSession->GetSessionDatabase().GetLastError());
	}
}

void FConcertClientWorkspace::PostActivityUpdated(const FConcertSyncActivity& InActivity)
{
	FConcertSyncActivity Activity;
	if (LiveSession->GetSessionDatabase().GetActivity(InActivity.ActivityId, Activity))
	{
		FConcertSyncEndpointData EndpointData;
		if (LiveSession->GetSessionDatabase().GetEndpoint(InActivity.EndpointId, EndpointData))
		{
			FStructOnScope ActivitySummary;
			if (Activity.EventSummary.GetPayload(ActivitySummary))
			{
				check(ActivitySummary.GetStruct()->IsChildOf(FConcertSyncActivitySummary::StaticStruct()));
				const FConcertSyncActivitySummary* ActivitySummaryPtr = (FConcertSyncActivitySummary*)ActivitySummary.GetStructMemory();
				UE_LOGF(LogConcert, Display, "Synced activity '%ls' produced by endpoint '%ls': %ls", *LexToString(InActivity.ActivityId), *InActivity.EndpointId.ToString(), *ActivitySummaryPtr->ToDisplayText(FText::AsCultureInvariant(EndpointData.ClientInfo.DisplayName)).ToString());
				OnActivityAddedOrUpdatedDelegate.Broadcast(EndpointData.ClientInfo, Activity, ActivitySummary);
			}
		}
	}
}

void FConcertClientWorkspace::SetIgnoreOnRestoreFlagForEmittedActivities(bool bIgnore)
{
	if (bIgnore) // Start ignoring further activities immediately.
	{
		FConcertIgnoreActivityStateChangedEvent StateChangeEvent{LiveSession->GetSession().GetSessionClientEndpointId(), bIgnore};
		LiveSession->GetSession().SendCustomEvent(StateChangeEvent, LiveSession->GetSession().GetSessionServerEndpointId(), EConcertMessageFlags::ReliableOrdered);
		bPendingStopIgnoringActivityOnRestore = false; // In case the 'ignore' state was toggle twice in the same frame.
	}
	else
	{
		// Stop 'ignoring' at the end of the frame, so that all pending transactions are sent and marked by the server properly. If a transaction started while
		// the activities were set as 'ignored' but ended after the flat was cleared, the corresponding transaction activities will NOT be marked 'ignored' and
		// will be seen as 'should restore' by the system.
		bPendingStopIgnoringActivityOnRestore = true;
	}
}

bool FConcertClientWorkspace::CanFinalize() const
{
	return bFinalizeWorkspaceSyncRequested
		&& LiveSession->GetSessionDatabase().HasWritePackageTasksCompleted()
		&& Algo::AllOf(CanFinalizeDelegates, [](const TTuple<FName,FCanFinalizeWorkspaceDelegate>& Pair)
		{
			if (Pair.Get<1>().IsBound())
			{
				return Pair.Get<1>().Execute();
			}
			return true;
		});
}

void FConcertClientWorkspace::AddWorkspaceFinalizeDelegate(FName InDelegateName, FCanFinalizeWorkspaceDelegate InDelegate)
{
	CanFinalizeDelegates.FindOrAdd(InDelegateName, MoveTemp(InDelegate));
}

void FConcertClientWorkspace::RemoveWorkspaceFinalizeDelegate(FName InDelegateName)
{
	CanFinalizeDelegates.Remove(InDelegateName);
}

void FConcertClientWorkspace::AddWorkspaceCanProcessPackagesDelegate(FName InDelegateName, FCanProcessPendingPackages Delegate)
{
	CanProcessPendingDelegates.FindOrAdd(InDelegateName, MoveTemp(Delegate));
}

void FConcertClientWorkspace::RemoveWorkspaceCanProcessPackagesDelegate(FName InDelegateName)
{
	CanProcessPendingDelegates.Remove(InDelegateName);
}

bool FConcertClientWorkspace::IsReloadingPackage(FName PackageName) const
{
	if (PackageManager)
	{
		return PackageManager->IsReloadingPackage(MoveTemp(PackageName));
	}
	return false;
}

bool FConcertClientWorkspace::CanProcessPendingPackages() const
{
	return LiveSession->GetSessionDatabase().HasWritePackageTasksCompleted() &&
		Algo::AllOf(CanProcessPendingDelegates, [](const TTuple<FName,FCanProcessPendingPackages>& Pair)
		{
			if (Pair.Get<1>().IsBound())
			{
				return Pair.Get<1>().Execute();
			}
			return true;
		});
}


#undef LOCTEXT_NAMESPACE
