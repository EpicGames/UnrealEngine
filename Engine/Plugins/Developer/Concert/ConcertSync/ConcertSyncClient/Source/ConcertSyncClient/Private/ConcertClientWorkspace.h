// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConcertBridges.h"
#include "ConcertSyncSessionFlags.h"
#include "ConcertWorkspaceMessages.h"
#include "ConcertClientWorkspaceData.h"
#include "ConcertSyncClientLiveSession.h"
#include "Delegates/Delegate.h"
#include "IConcertSyncClient.h"
#include "IConcertClientWorkspace.h"
#include "IConcertSessionHandler.h"

class IConcertClientSession;
class IConcertClientPackageBridge;
class IConcertFileSharingService;
class FConcertClientPackageManager;
class IConcertClientTransactionBridge;
class FConcertClientTransactionManager;
class FConcertClientLockManager;
class FConcertClientLiveTransactionAuthors;
class FConcertSyncClientLiveSession;
class ISourceControlProvider;
class FConcertClientDataStore;

namespace UE::ConcertSyncClient
{
	struct FSessionBindArgs
	{
		TSharedPtr<FConcertSyncClientLiveSession> LiveSession;
		FConcertBridges Bridges;
		
		bool IsValid() const { return LiveSession->IsValidSession() && Bridges.IsValid(); }
	};
}

struct FConcertSyncReplicationActivity;
struct FScopedSlowTask;

DECLARE_MULTICAST_DELEGATE(FOnWorkspaceEndFrameCompleted);

class FConcertClientWorkspace : public IConcertClientWorkspace
{
public:
	FConcertClientWorkspace(const UE::ConcertSyncClient::FSessionBindArgs& SessionBindArgs, TSharedPtr<IConcertFileSharingService> InFileSharingService, IConcertSyncClient* InOwnerSyncClient);
	virtual ~FConcertClientWorkspace();

	// IConcertClientWorkspace interface
	virtual IConcertClientSession& GetSession() const override;
	virtual FGuid GetWorkspaceLockId() const override;
	virtual FGuid GetResourceLockId(const FName InResourceName) const override;
	virtual bool AreResourcesLockedBy(TArrayView<const FName> ResourceNames, const FGuid& ClientId) override;
	virtual TFuture<FConcertResourceLockResponse> LockResources(TArray<FName> InResourceNames) override;
	virtual TFuture<FConcertResourceLockResponse> UnlockResources(TArray<FName> InResourceNames) override;
	virtual bool HasSessionChanges() const override;
	virtual TArray<FName> GatherSessionChanges(bool IgnorePersisted = true) override;
	virtual TOptional<FString> GetValidPackageSessionPath(FName PackageName) const override;
	virtual FPersistResult PersistSessionChanges(FPersistParameters InParam) override;
	virtual bool HasLiveTransactionSupport(UPackage* InPackage) const override;
	virtual bool ShouldIgnorePackageDirtyEvent(class UPackage* InPackage) const override;
	virtual bool FindTransactionEvent(const int64 TransactionEventId, FConcertSyncTransactionEvent& OutTransactionEvent, const bool bMetaDataOnly) const override;
	virtual TFuture<TOptional<FConcertSyncTransactionEvent>> FindOrRequestTransactionEvent(const int64 TransactionEventId, const bool bMetaDataOnly) override;
	virtual bool FindPackageEvent(const int64 PackageEventId, FConcertSyncPackageEventMetaData& OutPackageEvent) const override;
	virtual bool FindReplicationEvent(const int64 ReplicationEventId, FConcertSyncReplicationEvent& OutReplicationEvent) const override;
	virtual void GetActivities(const int64 FirstActivityIdToFetch, const int64 MaxNumActivities, TMap<FGuid, FConcertClientInfo>& OutEndpointClientInfoMap, TArray<FConcertSessionActivity>& OutActivities) const override;
	virtual int64 GetLastActivityId() const override;
	virtual FOnActivityAddedOrUpdated& OnActivityAddedOrUpdated() override;
	virtual FOnWorkspaceSynchronized& OnWorkspaceSynchronized() override;
	virtual FOnFinalizeWorkspaceSyncCompleted& OnFinalizeWorkspaceSyncCompleted() override;

	virtual IConcertClientDataStore& GetDataStore() override;
	virtual bool IsAssetModifiedByOtherClients(const FName& AssetName, int32* OutOtherClientsWithModifNum, TArray<FConcertClientInfo>* OutOtherClientsWithModifInfo, int32 OtherClientsWithModifMaxFetchNum) const override;
	virtual void SetIgnoreOnRestoreFlagForEmittedActivities(bool bIgnore) override;

	virtual void AddWorkspaceFinalizeDelegate(FName InDelegateName, FCanFinalizeWorkspaceDelegate InDelegate) override;
	virtual void RemoveWorkspaceFinalizeDelegate(FName InDelegateName) override;

	virtual void AddWorkspaceCanProcessPackagesDelegate(FName InDelegateName, FCanProcessPendingPackages Delegate) override;
	virtual void RemoveWorkspaceCanProcessPackagesDelegate(FName InDelegateName) override;

	virtual bool IsReloadingPackage(FName PackageName) const override;

	/** Indicates if we can process any pending package updates. */
	bool CanProcessPendingPackages() const;

	/** Indicates if we can finalize activity sync. */
	bool CanFinalize() const;

	/** Gets the delegate called after the workspace has completed its handling at the end of an engine frame. */
	FOnWorkspaceEndFrameCompleted& OnWorkspaceEndFrameCompleted();

private:

	/** Bind the workspace to this session. */
	void BindSession(const UE::ConcertSyncClient::FSessionBindArgs& SessionBindArgs);

	/** Unbind the workspace to its bound session. */
	void UnbindSession();

	/**
	 * Load client side info associated with this session if any. (i.e already persisted files)
	 */
	void LoadSessionData();

	/**
	 * Save client side info associated with this session if any. (i.e already persisted files)
	 */
	void SaveSessionData();

	/** */
	void HandleConnectionChanged(IConcertClientSession& InSession, EConcertConnectionStatus Status);

#if WITH_EDITOR
	/**
	 * Save all live transactions to packages.
	 */
	void SaveLiveTransactionsToPackages();

	/**
	 * Save live transactions, if any, to the specified packages, adding a client side dummy package event to the db
	 */
	void SaveLiveTransactionsToPackage(const FName PackageName);

	/** */
	void HandleAssetLoaded(UObject* InAsset);

	/** */
	void HandlePackageDiscarded(UPackage* InPackage);

	/** */
	void HandlePostPIEStarted(const bool InIsSimulating);

	/** */
	void HandleSwitchBeginPIEAndSIE(const bool InIsSimulating);

	/** */
	void HandleEndPIE(const bool InIsSimulating);
#endif	// WITH_EDITOR

	/** */
	void OnEndFrame();

	/** */
	void HandleWorkspaceSyncEndpointEvent(const FConcertSessionContext& Context, const FConcertWorkspaceSyncEndpointEvent& Event);

	/** */
	void HandleWorkspaceSyncActivityEvent(const FConcertSessionContext& Context, const FConcertWorkspaceSyncActivityEvent& Event);

	/** */
	void HandleWorkspaceSyncLockEvent(const FConcertSessionContext& Context, const FConcertWorkspaceSyncLockEvent& Event);

	/** */
	void HandleWorkspaceSyncCompletedEvent(const FConcertSessionContext& Context, const FConcertWorkspaceSyncCompletedEvent& Event);

	/**
	 * Set an endpoint in the session database, creating or replacing it.
	 *
	 * @param InEndpointId				The ID of the endpoint to set.
	 * @param InEndpointData			The endpoint data to set.
	 */
	void SetEndpoint(const FGuid& InEndpointId, const FConcertSyncEndpointData& InEndpointData);

	/**
	 * Set a connection activity in the session database, creating or replacing it.
	 * @note The endpoint ID referenced by the activity must exist in the database (@see SetEndpoint).
	 *
	 * @param InConnectionActivity		The connection activity to set.
	 */
	void SetConnectionActivity(const FConcertSyncConnectionActivity& InConnectionActivity);

	/**
	 * Set a lock activity in the session database, creating or replacing it.
	 * @note The endpoint ID referenced by the activity must exist in the database (@see SetEndpoint).
	 *
	 * @param InLockActivity			The lock activity to set.
	 */
	void SetLockActivity(const FConcertSyncLockActivity& InLockActivity);

	/**
	 * Set a transaction activity in the session database, creating or replacing it.
	 * @note The endpoint ID referenced by the activity must exist in the database (@see SetEndpoint).
	 *
	 * @param InTransactionActivity		The transaction activity to set.
	 */
	void SetTransactionActivity(const FConcertSyncTransactionActivity& InTransactionActivity);

	/**
	 * Set a package activity in the session database, creating or replacing it.
	 * @note The endpoint ID referenced by the activity must exist in the database (@see SetEndpoint).
	 *
	 * @param InPackageActivity		The package activity to set.
	 */
	void SetPackageActivity(const FConcertSyncPackageActivity& InPackageActivity);

	/**
	 * Set a replication activity in the session database, creating or replacing it.
	 * @note The endpoint ID referenced by the activity must exist in the database (@see SetEndpoint).
	 *
	 * @param InReplicationActivity		The replication activity to set.
	 */
	void SetReplicationActivity(const FConcertSyncReplicationActivity& InReplicationActivity);

	/**
	 * Called after any updated in the session database.
	 *
	 * @param InActivity			The activity that was updated.
	 */
	void PostActivityUpdated(const FConcertSyncActivity& InActivity);

	/**
	 * Check whether a transaction activity is partially synced, i.e. that only the meta data
	 * was synced because the activity event was superseded by another one and the transaction data
	 * wasn't required to reconstruct the state of a level.
	 */
	bool IsTransactionEventPartiallySynced(const FConcertSyncTransactionEvent& TransactionEvent) const;

	/** A hash map of delegates that provide true/false value for applying pendings during the current end of frame. */
	TMap<FName, FCanProcessPendingPackages> CanProcessPendingDelegates;

	/** A hash map of delegates that provide true/false value for finalizing an activity sync during the current end of frame. */
	TMap<FName, FCanFinalizeWorkspaceDelegate> CanFinalizeDelegates;

	/** */
	TUniquePtr<FConcertClientTransactionManager> TransactionManager;

	/** */
	TUniquePtr<FConcertClientPackageManager> PackageManager;

	/** */
	TUniquePtr<FConcertClientLockManager> LockManager;

	/**
	 * Tracks the clients that have live transactions on any given packages.
	 */
	TUniquePtr<FConcertClientLiveTransactionAuthors> LiveTransactionAuthors;

	/** */
	IConcertClientPackageBridge* PackageBridge;

	/** The sync client that owns the workspace. */
	IConcertSyncClient *OwnerSyncClient;

	/** */
	TSharedPtr<FConcertSyncClientLiveSession> LiveSession;

	/** Persistent client workspace data associated with this workspaces session. */
	FConcertClientWorkspaceData SessionData;

	/** True if this client has performed its initial sync with the server session */
	bool bHasSyncedWorkspace;

	/** True if a request to finalize a workspace sync has been requested */
	bool bFinalizeWorkspaceSyncRequested;

	/** Slow task used during the initial sync of this workspace */
	TUniquePtr<FScopedSlowTask> InitialSyncSlowTask;

	/** The delegate called every time activity is added to or updated in this session. */
	FOnActivityAddedOrUpdated OnActivityAddedOrUpdatedDelegate;

	/** The delegate called every time the workspace is synced. */
	FOnWorkspaceSynchronized OnWorkspaceSyncedDelegate;

	/** The delegate called after the workspace has been synced and finalized. */
	FOnFinalizeWorkspaceSyncCompleted OnFinalizeWorkspaceSyncCompletedDelegate;

	/** The delegate called after the workspace has completed its handling at the end of an engine frame. */
	FOnWorkspaceEndFrameCompleted OnWorkspaceEndFrameCompletedDelegate;

	/** The session key/value store proxy. The real store is held by the server and shared across all clients. */
	TUniquePtr<FConcertClientDataStore> DataStore;

	/** True if the client has marked the further transaction as 'non-ignored'. This is sent at the end of the frame. */
	bool bPendingStopIgnoringActivityOnRestore = false;

	/** Optional side channel to exchange large blobs (package data) with the server in a scalable way (ex. the request/response transport layer is not designed and doesn't support exchanging 3GB packages). */
	TSharedPtr<IConcertFileSharingService> FileSharingService;
};
