// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConcertClientWorkspaceData.generated.h"

/**
 * Client workspace data associated with a specific
 * session that persist between connections.
 */
USTRUCT()
struct FConcertClientWorkspaceData
{
	GENERATED_BODY()

	/** The session identifier that this client data is associated with. */
	UPROPERTY()
	FGuid SessionIdentifier;
};
