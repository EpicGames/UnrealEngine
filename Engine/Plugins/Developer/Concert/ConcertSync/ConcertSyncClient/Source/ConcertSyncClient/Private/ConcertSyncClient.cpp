// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertSyncClient.h"

#include "IConcertModule.h"
#include "IConcertClient.h"
#include "IConcertSession.h"
#include "ConcertClientSettings.h"
#include "ConcertSyncClientLiveSession.h"
#include "ConcertClientWorkspace.h"

#include "ConcertClientSequencerManager.h"
#include "ConcertClientPresenceManager.h"
#include "Replication/Manager/ReplicationManager.h"

#include "Misc/Paths.h"

LLM_DECLARE_TAG(Concert_ConcertSyncClient);
#define LOCTEXT_NAMESPACE "ConcertSyncClient"

FConcertSyncClient::FConcertSyncClient(
	const FString& InRole,
	const UE::ConcertSyncClient::FConcertBridges& InBridges
	)
	: ConcertClient(IConcertClientModule::Get().CreateClient(InRole))
	, SessionFlags(EConcertSyncSessionFlags::None)
	, Bridges(InBridges)
#if WITH_EDITOR
	, SequencerManager(MakeUnique<FConcertClientSequencerManager>(this))
#endif
{
	check(Bridges.IsValid());

	ConcertClient->OnSessionStartup().AddRaw(this, &FConcertSyncClient::RegisterConcertSyncHandlers);
	ConcertClient->OnSessionShutdown().AddRaw(this, &FConcertSyncClient::UnregisterConcertSyncHandlers);
}

FConcertSyncClient::~FConcertSyncClient()
{
	ConcertClient->OnSessionStartup().RemoveAll(this);
	ConcertClient->OnSessionShutdown().RemoveAll(this);
}

void FConcertSyncClient::Startup(const UConcertClientConfig* InClientConfig, const EConcertSyncSessionFlags InSessionFlags)
{
	LLM_SCOPE_BYTAG(Concert_ConcertSyncClient);
	SessionFlags = InSessionFlags;
	
	ConcertClient->Configure(InClientConfig);
	ConcertClient->Startup();
	Bridges.TransactionBridge->StartBridge();

	// if auto connection, start auto-connection routine
	if (InClientConfig->bAutoConnect && ConcertClient->CanAutoConnect())
	{
		ConcertClient->StartAutoConnect();
	}
}

void FConcertSyncClient::Shutdown()
{
	Bridges.TransactionBridge->StopBridge();
	ConcertClient->Shutdown();
}

IConcertClientRef FConcertSyncClient::GetConcertClient() const
{
	return ConcertClient;
}

TSharedPtr<IConcertClientWorkspace> FConcertSyncClient::GetWorkspace() const
{
	return Workspace;
}

IConcertClientPresenceManager* FConcertSyncClient::GetPresenceManager() const
{
	IConcertClientPresenceManager* Manager = nullptr;
#if WITH_EDITOR
	Manager = PresenceManager.Get();
#endif
	return Manager;
}

IConcertClientSequencerManager* FConcertSyncClient::GetSequencerManager() const
{
	IConcertClientSequencerManager* Manager = nullptr;
#if WITH_EDITOR
	Manager = SequencerManager.Get();
#endif
	return Manager;
}

IConcertClientReplicationManager* FConcertSyncClient::GetReplicationManager() const
{
	return ReplicationManager.Get();
}

FOnConcertClientWorkspaceStartupOrShutdown& FConcertSyncClient::OnWorkspaceStartup()
{
	return OnWorkspaceStartupDelegate;
}

FOnConcertClientWorkspaceStartupOrShutdown& FConcertSyncClient::OnWorkspaceShutdown()
{
	return OnWorkspaceShutdownDelegate;
}

FOnConcertClientSyncSessionStartupOrShutdown& FConcertSyncClient::OnSyncSessionStartup()
{
	return OnSyncSessionStartupDelegate;
}

FOnConcertClientSyncSessionStartupOrShutdown& FConcertSyncClient::OnSyncSessionShutdown()
{
	return OnSyncSessionShutdownDelegate;
}

void FConcertSyncClient::PersistSpecificChanges(TArrayView<const FName> InPackages)
{
#if WITH_EDITOR
	if (Workspace)
	{
		Workspace->PersistSessionChanges({ InPackages });
	}
#endif
}

void FConcertSyncClient::PersistAllSessionChanges()
{
#if WITH_EDITOR
	if (Workspace)
	{
		TArray<FName> SessionChanges = Workspace->GatherSessionChanges();
		Workspace->PersistSessionChanges({ SessionChanges });
	}
#endif
}

void FConcertSyncClient::GetSessionClientActions(const FConcertSessionClientInfo& InClientInfo, TArray<FConcertActionDefinition>& OutActions) const
{
#if WITH_EDITOR
	if (PresenceManager)
	{
		PresenceManager->GetPresenceClientActions(InClientInfo, OutActions);
	}
#endif
}

void FConcertSyncClient::SetFileSharingService(TSharedPtr<IConcertFileSharingService> InFileSharingService)
{
	ConcertClient->SetFileSharingService(MoveTemp(InFileSharingService));
}

void FConcertSyncClient::CreateWorkspace(const TSharedRef<FConcertSyncClientLiveSession>& InLiveSession)
{
	DestroyWorkspace();
	Workspace = MakeShared<FConcertClientWorkspace>(UE::ConcertSyncClient::FSessionBindArgs{ InLiveSession, Bridges } , ConcertClient->GetFileSharingService(), this);
	OnWorkspaceStartupDelegate.Broadcast(Workspace);
#if WITH_EDITOR
	SequencerManager->SetActiveWorkspace(Workspace);
#endif
}

void FConcertSyncClient::DestroyWorkspace()
{
	OnWorkspaceShutdownDelegate.Broadcast(Workspace);
	Workspace.Reset();
}

void FConcertSyncClient::RegisterConcertSyncHandlers(TSharedRef<IConcertClientSession> InSession)
{
	LiveSession = MakeShared<FConcertSyncClientLiveSession>(InSession, SessionFlags);
	if (LiveSession->IsValidSession())
	{
		OnSyncSessionStartupDelegate.Broadcast(this);
		CreateWorkspace(LiveSession.ToSharedRef());
		
		// Create Replication Manager
		if (EnumHasAnyFlags(LiveSession->GetSessionFlags(), EConcertSyncSessionFlags::EnableReplication) && ensure(Bridges.ReplicationBridge))
		{
			ReplicationManager = MakeUnique<UE::ConcertSyncClient::Replication::FReplicationManager>(InSession, *Bridges.ReplicationBridge, LiveSession->GetSessionFlags());
			ReplicationManager->StartAcceptingJoinRequests();
		}

#if WITH_EDITOR
		PresenceManager.Reset();
		if (EnumHasAnyFlags(SessionFlags, EConcertSyncSessionFlags::EnablePresence))
		{
			PresenceManager = MakeShared<FConcertClientPresenceManager>(InSession); // TODO: Use LiveSession?
		}
		if (EnumHasAnyFlags(SessionFlags, EConcertSyncSessionFlags::EnableSequencer))
		{
			SequencerManager->Register(InSession);  // TODO: Use LiveSession?
		}
#endif
	}
	else
	{
		LiveSession.Reset();
	}
}

IConcertClientTransactionBridge* FConcertSyncClient::GetTransactionBridge() const
{
	return Bridges.TransactionBridge;
}

IConcertClientPackageBridge* FConcertSyncClient::GetPackageBridge() const
{
	return Bridges.PackageBridge;
}

IConcertClientReplicationBridge* FConcertSyncClient::GetReplicationBridge() const
{
	return Bridges.ReplicationBridge;
}

void FConcertSyncClient::UnregisterConcertSyncHandlers(TSharedRef<IConcertClientSession> InSession)
{
#if WITH_EDITOR
	SequencerManager->Unregister(InSession);
	PresenceManager.Reset();
#endif
	ReplicationManager.Reset();
	DestroyWorkspace();
	OnSyncSessionShutdownDelegate.Broadcast(this);
	LiveSession.Reset();
}

#undef LOCTEXT_NAMESPACE
