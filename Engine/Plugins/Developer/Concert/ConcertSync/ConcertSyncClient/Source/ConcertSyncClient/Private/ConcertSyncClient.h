// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertBridges.h"
#include "IConcertSyncClient.h"

namespace UE::ConcertSyncClient::Replication
{
	class FReplicationManager;
}

class IConcertClientSession;
class IConcertClientPackageBridge;
class IConcertClientReplicationBridge;
class IConcertClientTransactionBridge;
class FConcertSyncClientLiveSession;
class FConcertClientWorkspace;
class FConcertClientPresenceManager;
class FConcertClientSequencerManager;
class FConcertSourceControlProxy;

/**
 * Implementation for a Concert Sync Client.
 */
class FConcertSyncClient : public IConcertSyncClient
{
public:
	FConcertSyncClient(const FString& InRole, const UE::ConcertSyncClient::FConcertBridges& InBridges);
	virtual ~FConcertSyncClient();

	//~ IConcertSyncClient interface
	virtual void Startup(const UConcertClientConfig* InClientConfig, const EConcertSyncSessionFlags InSessionFlags) override;
	virtual void Shutdown() override;
	virtual IConcertClientRef GetConcertClient() const override;
	virtual TSharedPtr<IConcertClientWorkspace> GetWorkspace() const override;
	virtual IConcertClientPresenceManager* GetPresenceManager() const override;
	virtual IConcertClientSequencerManager* GetSequencerManager() const override;
	virtual IConcertClientReplicationManager* GetReplicationManager() const override;

	virtual FOnConcertClientWorkspaceStartupOrShutdown& OnWorkspaceStartup() override;
	virtual FOnConcertClientWorkspaceStartupOrShutdown& OnWorkspaceShutdown() override;

	virtual FOnConcertClientSyncSessionStartupOrShutdown& OnSyncSessionStartup() override;
	virtual FOnConcertClientSyncSessionStartupOrShutdown& OnSyncSessionShutdown() override;

	virtual void PersistSpecificChanges(TArrayView<const FName> InPackages) override;
	virtual void PersistAllSessionChanges() override;
	virtual void GetSessionClientActions(const FConcertSessionClientInfo& InClientInfo, TArray<FConcertActionDefinition>& OutActions) const override;

	virtual void SetFileSharingService(TSharedPtr<IConcertFileSharingService> InFileSharingService) override;
	virtual IConcertClientTransactionBridge* GetTransactionBridge() const override;
	virtual IConcertClientPackageBridge* GetPackageBridge() const override;
	virtual IConcertClientReplicationBridge* GetReplicationBridge() const override;

private:
	void CreateWorkspace(const TSharedRef<FConcertSyncClientLiveSession>& InLiveSession);
	void DestroyWorkspace();

	void RegisterConcertSyncHandlers(TSharedRef<IConcertClientSession> InSession);
	void UnregisterConcertSyncHandlers(TSharedRef<IConcertClientSession> InSession);

	/** Client for Concert */
	IConcertClientRef ConcertClient;

	/** Flags controlling what features are enabled for sessions within this client */
	EConcertSyncSessionFlags SessionFlags;

	/** Pointers to Concert Bridges used by this client. */
	const UE::ConcertSyncClient::FConcertBridges Bridges;

	/** Live session data for the current session. */
	TSharedPtr<FConcertSyncClientLiveSession> LiveSession;

	/** Client workspace for the current session. */
	TSharedPtr<FConcertClientWorkspace> Workspace;

	/** Delegate called on every workspace startup. */
	FOnConcertClientWorkspaceStartupOrShutdown OnWorkspaceStartupDelegate;

	/** Delegate called on every workspace shutdown. */
	FOnConcertClientWorkspaceStartupOrShutdown OnWorkspaceShutdownDelegate;

	/** Delegate called just before the workspace get created. */
	FOnConcertClientSyncSessionStartupOrShutdown OnSyncSessionStartupDelegate;
	
	/** Delegate called just after the workspace was deleted. */
	FOnConcertClientSyncSessionStartupOrShutdown OnSyncSessionShutdownDelegate;
	
	/** Manages the client's replication system. Can be used to join & leave replication sessions. */
	TUniquePtr<UE::ConcertSyncClient::Replication::FReplicationManager> ReplicationManager;

#if WITH_EDITOR
	/** Presence manager for the current session. */
	TSharedPtr<FConcertClientPresenceManager> PresenceManager;

	/** Sequencer event manager for Concert session. */
	TUniquePtr<FConcertClientSequencerManager> SequencerManager;
#endif
};
