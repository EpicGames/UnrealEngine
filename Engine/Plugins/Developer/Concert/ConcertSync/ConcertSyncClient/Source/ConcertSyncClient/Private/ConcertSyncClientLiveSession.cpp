// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertSyncClientLiveSession.h"
#include "ConcertSyncSessionDatabase.h"
#include "IConcertSession.h"

FConcertSyncClientLiveSession::FConcertSyncClientLiveSession(const TSharedRef<IConcertClientSession>& InSession, const EConcertSyncSessionFlags InSessionFlags)
	: Session(InSession)
	, SessionFlags(InSessionFlags)
	, SessionDatabase(MakeUnique<FConcertSyncSessionDatabase>())
{
	SessionDatabase->Open(Session->GetSessionWorkingDirectory());
}

FConcertSyncClientLiveSession::~FConcertSyncClientLiveSession()
{
	SessionDatabase->Close();
	SessionDatabase.Reset();
}

bool FConcertSyncClientLiveSession::IsValidSession() const
{
	return SessionDatabase->IsValid();
}
