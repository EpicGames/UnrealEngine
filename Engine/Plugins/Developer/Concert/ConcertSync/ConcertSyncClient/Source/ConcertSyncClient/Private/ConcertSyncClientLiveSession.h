// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConcertMessageData.h"
#include "ConcertSyncSessionFlags.h"

class IConcertClientSession;
class FConcertSyncSessionDatabase;

class FConcertSyncClientLiveSession
{
public:
	FConcertSyncClientLiveSession(const TSharedRef<IConcertClientSession>& InSession, const EConcertSyncSessionFlags InSessionFlags);
	~FConcertSyncClientLiveSession();

	FConcertSyncClientLiveSession(const FConcertSyncClientLiveSession&) = delete;
	FConcertSyncClientLiveSession& operator=(const FConcertSyncClientLiveSession&) = delete;

	FConcertSyncClientLiveSession(FConcertSyncClientLiveSession&&) = delete;
	FConcertSyncClientLiveSession& operator=(FConcertSyncClientLiveSession&&) = delete;

	/** Is this a valid session? (ie, has been successfully opened) */
	bool IsValidSession() const;

	/** Get the underlying live session (if IsValidSession) */
	IConcertClientSession& GetSession()
	{
		return *Session;
	}

	/** Get the underlying live session (if IsValidSession) */
	const IConcertClientSession& GetSession() const
	{
		return *Session;
	}

	/** Get the flags controlling what features are enabled for this live session */
	EConcertSyncSessionFlags GetSessionFlags() const
	{
		return SessionFlags;
	}

	/** Get the database for this live session (if IsValidSession) */
	FConcertSyncSessionDatabase& GetSessionDatabase()
	{
		return *SessionDatabase;
	}

	/** Get the database for this live session (if IsValidSession) */
	const FConcertSyncSessionDatabase& GetSessionDatabase() const
	{
		return *SessionDatabase;
	}

private:
	/** The underlying live session */
	TSharedRef<IConcertClientSession> Session;

	/** Flags controlling what features are enabled for this live session */
	EConcertSyncSessionFlags SessionFlags;

	/** Database for this live session */
	TUniquePtr<FConcertSyncSessionDatabase> SessionDatabase;
};
