// Copyright Epic Games, Inc. All Rights Reserved.

#include "IConcertSyncClientModule.h"
#include "IConcertClient.h"
#include "ConcertSyncClient.h"
#include "ConcertSettings.h"
#include "ConcertClientSettings.h"
#include "ConcertLogGlobal.h"

#include "ConcertCloudFileSharingService.h"

#include "ConcertClientPackageBridge.h"
#include "ConcertClientTransactionBridge.h"
#include "Replication/ConcertClientReplicationBridge.h"

#include "Framework/Notifications/NotificationManager.h"

LLM_DEFINE_TAG(Concert_ConcertSyncClient);

/**
 * Implements the Concert Sync module for Event synchronization
 */
class FConcertSyncClientModule : public IConcertSyncClientModule
{
public:
	FConcertSyncClientModule() {}
	virtual ~FConcertSyncClientModule() {}

	virtual void StartupModule() override
	{
		LLM_SCOPE_BYTAG(Concert_ConcertSyncClient);
		PackageBridge = MakeUnique<FConcertClientPackageBridge>();
		TransactionBridge = MakeUnique<FConcertClientTransactionBridge>();
		ReplicationBridge = MakeUnique<UE::ConcertSyncClient::Replication::FConcertClientReplicationBridge>();
	}

	virtual void ShutdownModule() override
	{
		LLM_SCOPE_BYTAG(Concert_ConcertSyncClient);
		PackageBridge.Reset();
		TransactionBridge.Reset();
		ReplicationBridge.Reset();
	}

	virtual UConcertClientConfig* ParseClientSettings(const TCHAR* CommandLine) override
	{
		LLM_SCOPE_BYTAG(Concert_ConcertSyncClient);

		UConcertClientConfig* ClientConfig = NewObject<UConcertClientConfig>();

		if (ClientConfig)
		{
			// Validates the user input before overwriting the default config value.
			auto OverwriteConfigParamIfValid = [CommandLine](FText (*ValidateFunc)(const FString&), const TCHAR* Param, FString& OutParamValue)
			{
				FString ParsedParamValue;
				if (FParse::Value(CommandLine, Param, ParsedParamValue)) // Parameter was supplied on the command line?
				{
					if (!ParsedParamValue.IsEmpty() && ValidateFunc) // An empty value is always valid -> it clears the default value, no need to validate.
					{
						FText ValidateError = ValidateFunc(ParsedParamValue);
						if (!ValidateError.IsEmpty())
						{
							UE_LOGF(LogConcert, Warning, "Invalid value for '%ls' parameter. (Reason: %ls). Parameter ignored.", Param, *ValidateError.ToString());
							return;
						}
					}

					OutParamValue = ParsedParamValue; // Overwrite the default config value.
				}
			};

			OverwriteConfigParamIfValid(nullptr, TEXT("-CONCERTSERVER="), ClientConfig->DefaultServerURL);
			OverwriteConfigParamIfValid(&ConcertSettingsUtils::ValidateSessionName, TEXT("-CONCERTSESSION="), ClientConfig->DefaultSessionName);
			OverwriteConfigParamIfValid(&ConcertSettingsUtils::ValidateSessionName, TEXT("-CONCERTSESSIONTORESTORE="), ClientConfig->DefaultSessionToRestore);
			OverwriteConfigParamIfValid(&ConcertSettingsUtils::ValidateSessionName, TEXT("-CONCERTSAVESESSIONAS="), ClientConfig->DefaultSaveSessionAs);
			OverwriteConfigParamIfValid(&ConcertSettingsUtils::ValidateDisplayName, TEXT("-CONCERTDISPLAYNAME="), ClientConfig->ClientSettings.DisplayName);
			OverwriteConfigParamIfValid(&ConcertSettingsUtils::ValidateFileSharingPath, TEXT("-CONCERTFILESHAREPATH="), ClientConfig->ClientSettings.CloudFileSharingPath.Path);
				
			ClientConfig->bAutoConnect |= FParse::Param(CommandLine, TEXT("CONCERTAUTOCONNECT"));
			FParse::Bool(CommandLine, TEXT("-CONCERTAUTOCONNECT="), ClientConfig->bAutoConnect);

			ClientConfig->bShouldPromptForHotReloadOnLevel |= FParse::Param(CommandLine, TEXT("CONCERTSHOULDPROMPTFORHOTRELOAD"));
			FParse::Bool(CommandLine, TEXT("-CONCERTSHOULDPROMPTFORHOTRELOAD="), ClientConfig->bShouldPromptForHotReloadOnLevel);

			ClientConfig->bRetryAutoConnectOnError |= FParse::Param(CommandLine, TEXT("CONCERTRETRYAUTOCONNECTONERROR"));
			FParse::Bool(CommandLine, TEXT("-CONCERTRETRYAUTOCONNECTONERROR="), ClientConfig->bRetryAutoConnectOnError);

			ClientConfig->bIsHeadless |= FParse::Param(CommandLine, TEXT("CONCERTISHEADLESS"));
			FParse::Bool(CommandLine, TEXT("-CONCERTISHEADLESS="), ClientConfig->bIsHeadless);
			ClientConfig->bIsHeadless |= !FSlateNotificationManager::Get().AreNotificationsAllowed();

			ClientConfig->EndpointSettings.bEnableLogging |= FParse::Param(CommandLine, TEXT("CONCERTLOGGING"));
			FParse::Bool(CommandLine, TEXT("-CONCERTLOGGING="), ClientConfig->EndpointSettings.bEnableLogging);

			ClientConfig->ClientSettings.bEnableFileSharing |= FParse::Param(CommandLine, TEXT("CONCERTFILESHARINGENABLED"));
			FParse::Bool(CommandLine, TEXT("-CONCERTFILESHARINGENABLED="), ClientConfig->ClientSettings.bEnableFileSharing);

			if (ClientConfig->ClientSettings.CloudFileSharingPath.Path.IsEmpty())
			{
				ClientConfig->ClientSettings.bEnableFileSharing = false;
			}

			// CONCERTTAGS
			{
				FString CmdTags;
				if (FParse::Value(CommandLine, TEXT("-CONCERTTAGS="), CmdTags))
				{
					ClientConfig->ClientSettings.Tags.Reset();

					TArray<FString> Tags;
					CmdTags.ParseIntoArray(Tags, TEXT("|"), true);
					for (const FString& Tag : Tags)
					{
						ClientConfig->ClientSettings.Tags.Add(FName(*Tag));
					}
				}
			}
		}
		return ClientConfig;
	}

	virtual TSharedRef<IConcertSyncClient> CreateClient(const FString& InRole) override
	{
		LLM_SCOPE_BYTAG(Concert_ConcertSyncClient);

		// Remove dead clients.
		Clients.RemoveAll([](TWeakPtr<IConcertSyncClient> WeakClient) { return !WeakClient.IsValid(); });

		const TSharedRef<IConcertSyncClient> NewClient = MakeShared<FConcertSyncClient>(
			InRole,
			UE::ConcertSyncClient::FConcertBridges{ PackageBridge.Get(),
				TransactionBridge.Get(),
				ReplicationBridge.Get() });

		NewClient->SetFileSharingService(MakeShared<FConcertCloudFileSharingService>(InRole));
		Clients.Add(NewClient);
		OnClientCreated().Broadcast(NewClient);

		return NewClient;
	}

	virtual IConcertClientPackageBridge& GetPackageBridge() override
	{
		return *PackageBridge;
	}

	virtual IConcertClientTransactionBridge& GetTransactionBridge() override
	{
		return *TransactionBridge;
	}

	virtual IConcertClientReplicationBridge& GetReplicationBridge() override
	{
		return *ReplicationBridge;
	}

	virtual TArray<TSharedRef<IConcertSyncClient>> GetClients() const override
	{
		TArray<TSharedRef<IConcertSyncClient>> ActiveClients;
		for (TWeakPtr<IConcertSyncClient> WeakClient : Clients)
		{
			if (TSharedPtr<IConcertSyncClient> Client = WeakClient.Pin())
			{
				ActiveClients.Add(Client.ToSharedRef());
			}
		}

		return ActiveClients;
	}

	virtual TSharedPtr<IConcertSyncClient> GetClient(const FString& InRole) const override
	{
		TSharedPtr<IConcertSyncClient> Client;

		for (const TWeakPtr<IConcertSyncClient>& WeakClient : Clients)
		{
			if (TSharedPtr<IConcertSyncClient> ClientPtr = WeakClient.Pin())
			{
				if (ClientPtr->GetConcertClient()->GetRole() == InRole)
				{
					Client = ClientPtr;
				}
			}
		}

		return Client;
	}

	virtual FOnConcertClientCreated& OnClientCreated() override
	{
		return OnClientCreatedDelegate;
	}

private:
	TUniquePtr<FConcertClientPackageBridge> PackageBridge;
	TUniquePtr<FConcertClientTransactionBridge> TransactionBridge;
	TUniquePtr<UE::ConcertSyncClient::Replication::FConcertClientReplicationBridge> ReplicationBridge;
	TArray<TWeakPtr<IConcertSyncClient>> Clients;
	FOnConcertClientCreated OnClientCreatedDelegate;
};

IMPLEMENT_MODULE(FConcertSyncClientModule, ConcertSyncClient);
