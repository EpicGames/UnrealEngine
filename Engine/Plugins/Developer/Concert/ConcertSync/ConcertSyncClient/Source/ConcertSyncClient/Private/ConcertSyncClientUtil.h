// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/EnumClassFlags.h"
#include "Containers/ArrayView.h"

#if WITH_EDITOR
	#include "DirectoryWatcherModule.h"
	#include "IDirectoryWatcher.h"
#endif

class UObject;
class UPackage;
class UStruct;
class ULevel;
class AActor;
class UActorComponent;

class FConcertSyncWorldRemapper;
class FConcertLocalIdentifierTable;
class UConcertClientObjectFactory;

struct FConcertObjectId;
struct FConcertWorldNodeId;
struct FConcertSessionVersionInfo;
struct FConcertSerializedPropertyData;
struct FConcertPackageInfo;

enum class EConcertPackageUpdateType : uint8;

namespace ConcertSyncClientUtil
{
	enum class EGetObjectResultFlags : uint8
	{
		None = 0,
		NeedsGC = 1<<0,
		NewlyCreated = 1<<1,
	};
	ENUM_CLASS_FLAGS(EGetObjectResultFlags);

	struct FGetObjectResult
	{
		FGetObjectResult() = default;

		explicit FGetObjectResult(UObject* InObj, const EGetObjectResultFlags InFlags = EGetObjectResultFlags::None)
			: Obj(InObj)
			, Flags(InFlags)
		{
		}

		explicit FGetObjectResult(UObject* InObj, const UConcertClientObjectFactory* InFactory, const EGetObjectResultFlags InFlags)
			: Obj(InObj)
			, Factory(InFactory)
			, Flags(InFlags)
		{
		}

		bool NeedsGC() const
		{
			return EnumHasAnyFlags(Flags, EGetObjectResultFlags::NeedsGC);
		}

		bool NewlyCreated() const
		{
			return EnumHasAnyFlags(Flags, EGetObjectResultFlags::NewlyCreated);
		}

		UObject* Obj = nullptr;
		const UConcertClientObjectFactory* Factory = nullptr;
		EGetObjectResultFlags Flags = EGetObjectResultFlags::None;
	};

	bool IsUserEditing();

	bool ShouldDelayTransaction();

	bool CanPerformBlockingAction(const bool bBlockDuringInteraction = true);

	void UpdatePendingKillState(UObject* InObj, const bool bIsPendingKill);

	void AddActorToOwnerLevel(AActor* InActor);

	bool ObjectIdsMatch(const FConcertObjectId& One, const FConcertObjectId& Two);

	int32 GetObjectPathDepth(UObject* InObjToTest);

	FGetObjectResult GetObject(const FConcertObjectId& InObjectId, const FName InNewName, const FName InNewOuterPath, const FName InNewPackageName, const FSoftObjectPath& InSourceObject, const bool bAllowCreate);
	
	TArray<const FProperty*> GetExportedProperties(const UStruct* InStruct, const TArray<FName>& InPropertyNames, const bool InIncludeEditorOnlyData);

	const FProperty* GetExportedProperty(const UStruct* InStruct, const FName InPropertyName, const bool InIncludeEditorOnlyData);

	void SerializeProperties(FConcertLocalIdentifierTable* InLocalIdentifierTable, const UObject* InObject, const TArray<const FProperty*>& InProperties, const bool InIncludeEditorOnlyData, TArray<FConcertSerializedPropertyData>& OutPropertyDatas);

	void SerializeProperty(FConcertLocalIdentifierTable* InLocalIdentifierTable, const UObject* InObject, const FProperty* InProperty, const bool InIncludeEditorOnlyData, TArray<uint8>& OutSerializedData);

	void SerializeObject(FConcertLocalIdentifierTable* InLocalIdentifierTable, const UObject* InObject, const TArray<const FProperty*>* InProperties, const bool InIncludeEditorOnlyData, TArray<uint8>& OutSerializedData);

	void FlushPackageLoading(const FName InPackageName);

	void FlushPackageLoading(const FString& InPackageName, bool bForceBulkDataLoad = true);

#if WITH_EDITOR
	/** Gets the DirectoryWatcher module.
	 *
	 *  The module will be loaded if it is not currently loaded.
	 * 
	 *  @return: the DirectoryWatcher module
	 */
	FDirectoryWatcherModule& GetDirectoryWatcherModule();

	/** Gets the DirectoryWatcher module if it is currently loaded.
	 *
	 *  @return: a pointer to the DirectoryWatcher module if is currently loaded, or nullptr otherwise
	 */
	FDirectoryWatcherModule* GetDirectoryWatcherModuleIfLoaded();

	/** Gets the directory watcher.
	 *
	 *  The DirectoryWatcher module will be loaded if it is not currently loaded.
	 *
	 *  @return: the directory watcher if the platform supports directory watching, or nullptr otherwise
	 */
	IDirectoryWatcher* GetDirectoryWatcher();

	/** Gets the directory watcher if the DirectoryWatcher module is currently loaded.
	 *
	 *  @return: the directory watcher if the DirectoryWatcher module is currently loaded and the platform supports directory watching, or nullptr otherwise
	 */
	IDirectoryWatcher* GetDirectoryWatcherIfLoaded();
#endif // WITH_EDITOR

	/** Synchronizes the Asset Registry.
	 * 
	 *  This ensures that any pending file changes are completed and that the
	 *  Asset Registry is updated to correctly reflect those changes.
	 */
	void SynchronizeAssetRegistry();

	void HotReloadPackages(TArrayView<const FName> InPackageNames);

	void PurgePackages(TArrayView<const FName> InPackageNames);

	UWorld* GetCurrentWorld();

	/** Return the Persistent Level if it is currently using external objects, i.e. one file per actor for world partitioning. */
	ULevel* GetExternalPersistentWorld();

	/** Return true if we are in a world partition world. */
	bool IsWorldPartitionWorld();

	FConcertPackageInfo FillPackageInfo(UPackage* InPackage, UObject* InAsset, const EConcertPackageUpdateType InPackageUpdateType);
}
