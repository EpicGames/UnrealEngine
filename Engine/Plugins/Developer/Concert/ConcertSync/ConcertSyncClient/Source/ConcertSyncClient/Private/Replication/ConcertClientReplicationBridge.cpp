// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertClientReplicationBridge.h"

#include "ConcertLogGlobal.h"

#include "Engine/Engine.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "Misc/CoreDelegates.h"

namespace UE::ConcertSyncClient::Replication
{
	FConcertClientReplicationBridge::FConcertClientReplicationBridge()
	{
		// This constructor is called by StartupModule: at that time GEngine is not inited yet.
		// The constructor is not supposed "know" when it is called though so handle it here.
		if (!GEngine)
		{
			FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FConcertClientReplicationBridge::SetupEngineDelegates);
		}
		else
		{
			SetupEngineDelegates();
		}

		OnObjectDiscoveredDelegate.AddLambda([](UObject& Object)
		{
			UE_LOGF(LogConcert, Verbose, "Discovered object %ls for replication", *Object.GetPathName());
		});
		OnObjectRemovedDelegate.AddLambda([](const FSoftObjectPath& ObjectPath)
		{
			UE_LOGF(LogConcert, Verbose, "Removed object %ls from replication", *ObjectPath.ToString());
		});
	}

	FConcertClientReplicationBridge::~FConcertClientReplicationBridge()
	{
		if (GEngine)
		{
			GEngine->OnWorldAdded().RemoveAll(this);
			GEngine->OnWorldDestroyed().RemoveAll(this);
			GEngine->OnLevelActorAdded().RemoveAll(this);
			GEngine->OnLevelActorDeleted().RemoveAll(this);
		}
	}

	void FConcertClientReplicationBridge::PushTrackedObjects(TArrayView<const FSoftObjectPath> InTrackedObjects)
	{
		TSet<UObject*> ResolvedObjects;
		
		for (const FSoftObjectPath& TrackedObjectPath : InTrackedObjects)
		{
			FTrackedObjectInfo& TrackedObject = TrackedObjects.FindOrAdd(TrackedObjectPath);
			++TrackedObject.TrackCounter;
			if (!TrackedObject.ResolvedObject.IsValid())
			{
				UObject* ResolvedObject = FindObjectIfAvailable(TrackedObjectPath);
				TrackedObject.ResolvedObject = ResolvedObject;
				if (ResolvedObject)
				{
					ResolvedObjects.Add(ResolvedObject);
				}
			}
		}
		
		// Broadcast after updating TrackedObjects in the unlikely case that the broadcast calls PushTrackedObjects or PopTrackedObjects.
		for (UObject* ResolvedObject : ResolvedObjects)
		{
			OnObjectDiscoveredDelegate.Broadcast(*ResolvedObject);
		}
	}

	void FConcertClientReplicationBridge::PopTrackedObjects(TArrayView<const FSoftObjectPath> InTrackedObjects)
	{
		for (const FSoftObjectPath& TrackedObjectPath : InTrackedObjects)
		{
			FTrackedObjectInfo* TrackedObject = TrackedObjects.Find(TrackedObjectPath);
			if (!TrackedObject)
			{
				UE_LOGF(LogConcert, Warning, "Object %ls was requested to not be tracked but was it already wasn't.", *TrackedObjectPath.ToString());
				continue;
			}

			if (TrackedObject->TrackCounter == 1)
			{
				TrackedObjects.Remove(TrackedObjectPath);
			}
			else
			{
				--TrackedObject->TrackCounter;
			}
		}
	}

	bool FConcertClientReplicationBridge::IsObjectAvailable(const FSoftObjectPath& Path)
	{
		// TODO: Possibly iterate through worlds and see whether they're the parent of this object. Profile performance.
		return FindObjectIfAvailable(Path) != nullptr;
	}

	UObject* FConcertClientReplicationBridge::FindObjectIfAvailable(const FSoftObjectPath& Path)
	{
		// Replicated UObject's may be renamed, e.g. by using UObject::Rename. If that happens, we need to re-resolve the Path.
		// A legitimate case that could cause this: We're replicating a component that has EComponentCreationMethod::SimpleConstructionScript.
		// Upon editing the object, the construction script is re-run thus replacing the component instance with a totally new object!
		FTrackedObjectInfo* ObjectInfo = TrackedObjects.Find(Path);
		const bool bObjectWasInCache = ObjectInfo && ObjectInfo->ResolvedObject.IsValid();
		if (bObjectWasInCache)
		{
			if (Path == ObjectInfo->ResolvedObject.Get())
			{
				return ObjectInfo->ResolvedObject.Get();
			}
			UE_LOGF(LogConcert, Verbose, "Replicated object %ls was renamed to %ls. Resolving new target object...", *Path.ToString(), *ObjectInfo->ResolvedObject->GetPathName());
		}
		
		// This should be an object in a UWorld. This will resolve if the world is opened.
		UObject* Object = Path.ResolveObject();
		UE_CLOGF(Object && !Object->IsInA(UWorld::StaticClass()), LogConcert, Warning, "Object %ls is not in any UWorld. The replication system was designed to work with objects in UWorlds.", *Path.ToString());

		// Update the cache for faster look ups if we found anything
		if (Object)
		{
			ObjectInfo = ObjectInfo ? ObjectInfo : &TrackedObjects.Add(Path);
			ObjectInfo->ResolvedObject = Object;
		}
		else
		{
			// If the object was renamed but not replaced, issue a warning.
			UE_CLOGF(bObjectWasInCache, LogConcert, Warning, "Replicated object %ls was renamed to %ls. The target object was not replaced by any new object so it will stop being replicated.", *Path.ToString(), *ObjectInfo->ResolvedObject.Get()->GetPathName());
		}
		
		return Object;
	}

	void FConcertClientReplicationBridge::SetupEngineDelegates()
	{
		if (ensure(GEngine))
		{
			GEngine->OnWorldAdded().AddRaw(this, &FConcertClientReplicationBridge::SetupWorld);
			GEngine->OnWorldDestroyed().AddRaw(this, &FConcertClientReplicationBridge::CleanupWorld);
			// Important: In a Multi-User session, actors might get added only after the world is loaded. Transactions are replayed at the end of tick.
			// Replaying calls SpawnActor so the below events will trigger.
			GEngine->OnLevelActorAdded().AddRaw(this, &FConcertClientReplicationBridge::DiscoverTrackedObjectsInActor);
			GEngine->OnLevelActorDeleted().AddRaw(this, &FConcertClientReplicationBridge::RemoveDiscoveredObjectsInActor);
		}
	}

	void FConcertClientReplicationBridge::SetupWorld(UWorld* World)
	{
		// We'll allow PIE even though it is currently not explicitly supported; the primary use case is editor or game worlds (if someone tries it in packaged).
		if (World->IsPreviewWorld())
		{
			return;
		}
		
		LoadedWorlds.Add(World);
		DiscoverTrackedObjectsInWorld(*World);
	}

	void FConcertClientReplicationBridge::CleanupWorld(UWorld* World)
	{
		const int32 NumRemoved = LoadedWorlds.Remove(World);
		bool bRemovedAnything = NumRemoved > 0;
		if (bRemovedAnything)
		{
			RemoveDiscoveredObjectsInWorld(*World);
		}
	}

	void FConcertClientReplicationBridge::DiscoverTrackedObjectsInWorld(UWorld& World)
	{
		UE_LOGF(LogConcert, Verbose, "Discovering objects in world %ls for replication", *World.GetPathName());
		// Defer calling the delegate in case it causes a world unload or changes the tracking state.
		TArray<TWeakObjectPtr<UObject>> DeferredDiscoveredObjects;
		
		for (TPair<FSoftObjectPath, FTrackedObjectInfo>& Pair : TrackedObjects)
		{
			FTrackedObjectInfo& ObjectInfo = Pair.Value;
			if (ObjectInfo.ResolvedObject.IsValid())
			{
				continue;
			}

			const FSoftObjectPath& ObjectPath = Pair.Key;
			UObject* Object = ObjectPath.ResolveObject();
			// Only objects from the loaded world should be considered. ResolveObject will still resolve to objects from the old world.
			if (Object && Object->IsIn(&World))
			{
				ObjectInfo.ResolvedObject = Object;
				DeferredDiscoveredObjects.Emplace(Object);
			}
		}

		MarkAsDiscovered(DeferredDiscoveredObjects);
	}

	void FConcertClientReplicationBridge::RemoveDiscoveredObjectsInWorld(UWorld& World)
	{
		UE_LOGF(LogConcert, Verbose, "Removing objects from world %ls for replication", *World.GetPathName());
		// Defer calling the delegate in case it causes a world load or changes the tracking state.
		TArray<FSoftObjectPath> DeferredHiddenObjects;
		
		for (TPair<FSoftObjectPath, FTrackedObjectInfo>& Pair : TrackedObjects)
		{
			TWeakObjectPtr<UObject>& WeakObject = Pair.Value.ResolvedObject;
			if (WeakObject.IsStale())
			{
				WeakObject.Reset();
				DeferredHiddenObjects.Add(Pair.Key);
			}
			// Not sure whether the weak ptr would return IsStale in a OnPostWorldCleanup callback... this makes sure.
			else if (UObject* Object = Pair.Key.ResolveObject()
				; Object && Object->IsIn(&World))
			{
				WeakObject.Reset();
				DeferredHiddenObjects.Add(Pair.Key);
			}
		}

		MarkAsUndiscovered(DeferredHiddenObjects);
	}

	void FConcertClientReplicationBridge::DiscoverTrackedObjectsIn(UObject* AnalysedObject)
	{
		TArray<TWeakObjectPtr<UObject>> FoundObjects;
		auto AnalyseObject = [this, &FoundObjects](UObject* Object)
		{
			FTrackedObjectInfo* ObjectInfo = TrackedObjects.Find(Object);
			if (!ObjectInfo)
			{
				// Object is not being tracked
				return;
			}

			if (!ObjectInfo->ResolvedObject.IsValid())
			{
				ObjectInfo->ResolvedObject = Object;
				FoundObjects.Emplace(Object);
			}
		};
		
		AnalyseObject(AnalysedObject);
		ForEachObjectWithOuter(AnalysedObject, [&AnalyseObject](UObject* Subobject)
		{
			AnalyseObject(Subobject);
		}, EGetObjectsFlags::IncludeNestedObjects);

		MarkAsDiscovered(FoundObjects);
	}

	void FConcertClientReplicationBridge::RemoveDiscoveredObjectsIn(UObject* Object)
	{
		TArray<FSoftObjectPath> ObjectsToRemove;
		auto AnalyseObject = [this, &ObjectsToRemove](UObject* Object)
		{
			const FSoftObjectPath ObjectPath = Object;
			FTrackedObjectInfo* ObjectInfo = TrackedObjects.Find(ObjectPath);
			if (!ObjectInfo)
			{
				// Object is not being tracked
				return;
			}

			if (ObjectInfo->ResolvedObject == Object)
			{
				ObjectInfo->ResolvedObject.Reset();
				ObjectsToRemove.Emplace(ObjectPath);
			}
		};
		
		AnalyseObject(Object);
		ForEachObjectWithOuter(Object, [&AnalyseObject](UObject* Subobject)
		{
			AnalyseObject(Subobject);
		}, EGetObjectsFlags::IncludeNestedObjects);

		MarkAsUndiscovered(ObjectsToRemove);
	}

	void FConcertClientReplicationBridge::DiscoverTrackedObjectsInActor(AActor* Actor)
	{
		DiscoverTrackedObjectsIn(Actor);
	}

	void FConcertClientReplicationBridge::RemoveDiscoveredObjectsInActor(AActor* Actor)
	{
		RemoveDiscoveredObjectsIn(Actor);
	}

	void FConcertClientReplicationBridge::MarkAsDiscovered(TConstArrayView<TWeakObjectPtr<UObject>> DeferredDiscoveredObjects)
	{
		for (const TWeakObjectPtr<UObject>& WeakObject : DeferredDiscoveredObjects)
		{
			// The point of checking this again is that a previous OnObjectDiscoveredDelegate call may have caused a world unload or stopped tracking something.
			// If a world unload was triggered, then OnPostWorldCleanup will have cleaned FTrackedObjectInfo::ResolvedObject (in call recursive to this one).
			if (WeakObject.IsValid() && TrackedObjects.Contains(WeakObject.Get()))
			{
				OnObjectDiscoveredDelegate.Broadcast(*WeakObject);
			}
		}
	}
	void FConcertClientReplicationBridge::MarkAsUndiscovered(TConstArrayView<FSoftObjectPath> DeferredHiddenObjects)
	{
		for (const FSoftObjectPath& HiddenObjectPath : DeferredHiddenObjects)
		{
			const UObject* Resolved = HiddenObjectPath.ResolveObject();
			
			// The point of checking this again is that a previous OnObjectRemovedDelegate call may have caused a world load or stopped tracking something.
			// If a world load was triggered, then OnPostWorldInitialization will have set FTrackedObjectInfo::ResolvedObject (in call recursive to this one).
			if (TrackedObjects.Contains(HiddenObjectPath))
			{
				OnObjectRemovedDelegate.Broadcast(HiddenObjectPath);
			}
		}
	}
}
