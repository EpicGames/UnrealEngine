// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/IConcertClientReplicationBridge.h"

#include "Containers/Map.h"
#include "Containers/Set.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/WeakObjectPtr.h"
#include "Templates/SharedPointer.h"

class AActor;
class UObject;
class UWorld;
struct FWorldInitializationValues;

namespace UE::ConcertSyncClient::Replication
{
	class FConcertClientReplicationBridge : public IConcertClientReplicationBridge
	{
	public:
		
		FConcertClientReplicationBridge();
		virtual ~FConcertClientReplicationBridge() override;

		//~ Begin IConcertClientReplicationBridge Interface
		virtual void PushTrackedObjects(TArrayView<const FSoftObjectPath> InTrackedObjects) override;
		virtual void PopTrackedObjects(TArrayView<const FSoftObjectPath> InTrackedObjects) override;
		virtual bool IsObjectAvailable(const FSoftObjectPath& Path) override;
		virtual UObject* FindObjectIfAvailable(const FSoftObjectPath& Path) override;
		virtual FConcertClientReplicationBridgeObjectEvent& OnObjectDiscovered() override { return OnObjectDiscoveredDelegate; }
		virtual FConcertClientReplicationBridgeObjectPathEvent& OnObjectHidden() override { return OnObjectRemovedDelegate; }
		//~ End IConcertClientReplicationBridge Interface

	private:

		struct FTrackedObjectInfo
		{
			/** Tracks the number of PushTrackedObjects calls. This entry is removed upon reaching 0. */
			int32 TrackCounter = 0;
			/** Cached object pointer to avoid constant resolving of the soft object ptr. */
			TWeakObjectPtr<UObject> ResolvedObject;
		};

		/** The objects that wish to be tracked. */
		TMap<FSoftObjectPath, FTrackedObjectInfo> TrackedObjects;

		/** The set of worlds currently open on this client. */
		TSet<TWeakObjectPtr<UWorld>> LoadedWorlds;
	
		FConcertClientReplicationBridgeObjectEvent OnObjectDiscoveredDelegate;
		FConcertClientReplicationBridgeObjectPathEvent OnObjectRemovedDelegate;

		// Setting up delegates for worlds
		void SetupEngineDelegates();
		void SetupWorld(UWorld* World);
		void CleanupWorld(UWorld* World);

		// Searching everything in world
		void DiscoverTrackedObjectsInWorld(UWorld& World);
		void RemoveDiscoveredObjectsInWorld(UWorld& World);

		// Searching a specific object
		void DiscoverTrackedObjectsIn(UObject* AnalysedObject);
		void RemoveDiscoveredObjectsIn(UObject* Object);
		void DiscoverTrackedObjectsInActor(AActor* Actor);
		void RemoveDiscoveredObjectsInActor(AActor* Actor);

		// Util for formally marking objects as discovered
		void MarkAsDiscovered(TConstArrayView<TWeakObjectPtr<UObject>> DeferredDiscoveredObjects);
		void MarkAsUndiscovered(TConstArrayView<FSoftObjectPath> DeferredHiddenObjects);
	};
}

