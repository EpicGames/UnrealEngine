// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/IConcertClientReplicationManager.h"

#include "ConcertLogGlobal.h"

#include "Async/Future.h"

namespace UE::ConcertSyncClient::Replication::Private
{
	template<typename Callback>
	void ForEachStreamContainingObject(TArrayView<const FSoftObjectPath> Objects, const IConcertClientReplicationManager& Manager, Callback InCallback)
	{
		for (const FSoftObjectPath& ObjectPath : Objects)
		{
			bool bWasFound = false;
			Manager.ForEachRegisteredStream([&InCallback, &ObjectPath, &bWasFound](const FConcertReplicationStream& StreamDescription)
			{
				const bool bStreamContainsObject = StreamDescription.BaseDescription.ReplicationMap.ReplicatedObjects.Contains(ObjectPath);
				if (bStreamContainsObject)
				{
					bWasFound = true;
					InCallback(ObjectPath, StreamDescription.BaseDescription.Identifier);
				}
				return EBreakBehavior::Continue;
			});
			UE_CLOGF(!bWasFound, LogConcert, Warning, "Object %ls is not a valid argument because it is not contained in any stream.", *ObjectPath.ToString());
		}
	}
}

bool IConcertClientReplicationManager::HasRegisteredStreams() const
{
	return ForEachRegisteredStream([](const auto&){ return EBreakBehavior::Break; }) == EStreamEnumerationResult::Iterated;
}

TArray<FConcertReplicationStream> IConcertClientReplicationManager::GetRegisteredStreams() const
{
	TArray<FConcertReplicationStream> Result;
	ForEachRegisteredStream([&Result](const FConcertReplicationStream& Description){ Result.Add(Description); return EBreakBehavior::Continue; });
	return Result;
}

TFuture<FConcertReplication_ChangeAuthority_Response> IConcertClientReplicationManager::TakeAuthorityOver(TConstArrayView<FSoftObjectPath> Objects)
{
	using namespace UE::ConcertSyncClient::Replication;
	
	if (!HasRegisteredStreams())
	{
		UE_LOGF(LogConcert, Error, "Attempted to take authority while not connected!");
		TMap<FSoftObjectPath, FConcertStreamArray> Result;
		Algo::Transform(Objects, Result, [](const FSoftObjectPath& Path){ return Path; });
		return MakeFulfilledPromise<FConcertReplication_ChangeAuthority_Response>(FConcertReplication_ChangeAuthority_Response{EReplicationResponseErrorCode::Handled ,MoveTemp(Result) }).GetFuture();
	}

	FConcertReplication_ChangeAuthority_Request Request;
	Private::ForEachStreamContainingObject(Objects, *this,
		[&Request](const FSoftObjectPath& ObjectPath, const FGuid& StreamId)
		{
			Request.TakeAuthority.FindOrAdd(ObjectPath).StreamIds.Add(StreamId);
		});

	// Do not send pointless, empty requests to the server
	if (Request.TakeAuthority.IsEmpty())
	{
		// Not only does this warn about incorrect API use at runtime, this also helps debug (incorrectly written) unit tests
		const FString ObjectsAsString = FString::JoinBy(Objects, TEXT(","), [](const FSoftObjectPath& Path){ return Path.ToString(); });
		UE_LOGF(LogConcert, Warning, "Local client did not register any stream for the given objects. This take authority request will not be sent. Objects: %ls", *ObjectsAsString);
		return MakeFulfilledPromise<FConcertReplication_ChangeAuthority_Response>(FConcertReplication_ChangeAuthority_Response{}).GetFuture();
	}
	
	return RequestAuthorityChange(MoveTemp(Request));
}

TFuture<FConcertReplication_ChangeAuthority_Response> IConcertClientReplicationManager::ReleaseAuthorityOf(TConstArrayView<FSoftObjectPath> Objects)
{
	using namespace UE::ConcertSyncClient::Replication;
	
	if (!HasRegisteredStreams())
	{
		UE_LOGF(LogConcert, Error, "Attempted to take authority while not connected!");
		TMap<FSoftObjectPath, FConcertStreamArray> Result;
		Algo::Transform(Objects, Result, [](const FSoftObjectPath& Path){ return Path; });
		return MakeFulfilledPromise<FConcertReplication_ChangeAuthority_Response>(FConcertReplication_ChangeAuthority_Response{ EReplicationResponseErrorCode::Handled, MoveTemp(Result) }).GetFuture();
	}
	
	FConcertReplication_ChangeAuthority_Request Request;
	Private::ForEachStreamContainingObject(Objects, *this,
		[&Request](const FSoftObjectPath& ObjectPath, const FGuid& StreamId)
		{
			Request.ReleaseAuthority.FindOrAdd(ObjectPath).StreamIds.Add(StreamId);
		});

	// Do not send pointless, empty requests to the server
	if (Request.ReleaseAuthority.IsEmpty())
	{
		// Not only does this warn about incorrect API use at runtime, this also helps debug (incorrectly written) unit tests
		const FString ObjectsAsString = FString::JoinBy(Objects, TEXT(","), [](const FSoftObjectPath& Path){ return Path.ToString(); });
		UE_LOGF(LogConcert, Warning, "Local client did not register any stream for the given objects. This release authority request will not be sent. Objects: %ls", *ObjectsAsString);
		return MakeFulfilledPromise<FConcertReplication_ChangeAuthority_Response>(FConcertReplication_ChangeAuthority_Response{}).GetFuture();
	}
	
	return RequestAuthorityChange(MoveTemp(Request));
}

TMap<FSoftObjectPath, TSet<FGuid>> IConcertClientReplicationManager::GetClientOwnedObjects() const
{
	TMap<FSoftObjectPath, TSet<FGuid>> Result;
	ForEachClientOwnedObject([&Result](const FSoftObjectPath& Object, TSet<FGuid>&& OwningStreams)
	{
		Result.Emplace(Object, MoveTemp(OwningStreams));
		return EBreakBehavior::Continue;
	});
	return Result;
}

TSet<FConcertObjectInStreamID> IConcertClientReplicationManager::GetSyncControlledObjects() const
{
	TSet<FConcertObjectInStreamID> Result;
	const uint32 NumObjects = NumSyncControlledObjects();
	if (NumObjects == 0)
	{
		return Result;
	}
	
	Result.Reserve(NumObjects);
	ForEachSyncControlledObject([&](const FConcertObjectInStreamID& Object)
	{
		Result.Add(Object);
		return EBreakBehavior::Continue;
	});
	return Result;
}

TFuture<FConcertReplication_ChangeMuteState_Response> IConcertClientReplicationManager::MuteObjects(TConstArrayView<FSoftObjectPath> Objects, EConcertReplicationMuteOption Flags)
{
	FConcertReplication_ChangeMuteState_Request Request;
	for (const FSoftObjectPath& Object : Objects)
	{
		Request.ObjectsToMute.Add(Object, { Flags });
	}
	return ChangeMuteState(Request);
}

TFuture<FConcertReplication_ChangeMuteState_Response> IConcertClientReplicationManager::UnmuteObjects(TSet<FSoftObjectPath> Objects, EConcertReplicationMuteOption Flags)
{
	FConcertReplication_ChangeMuteState_Request Request;
	for (const FSoftObjectPath& Object : Objects)
	{
		Request.ObjectsToUnmute.Add(Object, { Flags });
	}
	return ChangeMuteState(Request);
}

TFuture<FConcertReplication_QueryMuteState_Response> IConcertClientReplicationManager::QueryMuteState(TSet<FSoftObjectPath> Objects)
{
	return QueryMuteState(FConcertReplication_QueryMuteState_Request{ MoveTemp(Objects) });
}
