// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationManager.h"

#include "IConcertSession.h"
#include "ReplicationManagerState_Disconnected.h"
#include "Replication/Manager/Utils/ReplicationManagerUtils.h"

#include "Replication/Messages/Handshake.h"

namespace UE::ConcertSyncClient::Replication
{
	FReplicationManager::FReplicationManager(
		TSharedRef<IConcertClientSession> InLiveSession,
		IConcertClientReplicationBridge& InBridge,
		EConcertSyncSessionFlags SessionFlags
		)
		: Session(MoveTemp(InLiveSession))
		, Bridge(InBridge)
		, SessionFlags(SessionFlags)
	{}

	FReplicationManager::~FReplicationManager()
	{}

	void FReplicationManager::StartAcceptingJoinRequests()
	{
		checkSlow(!CurrentState.IsValid());
		CurrentState = MakeShared<FReplicationManagerState_Disconnected>(Session, Bridge, *this, SessionFlags);
	}

	TFuture<FJoinReplicatedSessionResult> FReplicationManager::JoinReplicationSession(FJoinReplicatedSessionArgs Args)
	{
		return ensureMsgf(CurrentState, TEXT("StartAcceptingJoinRequests should have been called at this point."))
			? CurrentState->JoinReplicationSession(MoveTemp(Args))
			: MakeFulfilledPromise<FJoinReplicatedSessionResult>(EJoinReplicationErrorCode::Cancelled).GetFuture();
	}

	void FReplicationManager::LeaveReplicationSession()
	{
		if (ensureMsgf(CurrentState, TEXT("StartAcceptingJoinRequests should have been called at this point.")))
		{
			CurrentState->LeaveReplicationSession();
		}
	}

	bool FReplicationManager::CanJoin()
	{
		return ensureMsgf(CurrentState, TEXT("StartAcceptingJoinRequests should have been called at this point."))
			&& CurrentState->CanJoin(); 
	}

	bool FReplicationManager::IsConnectedToReplicationSession()
	{
		return ensureMsgf(CurrentState, TEXT("StartAcceptingJoinRequests should have been called at this point."))
			&& CurrentState->IsConnectedToReplicationSession(); 
	}

	IConcertClientReplicationManager::EStreamEnumerationResult FReplicationManager::ForEachRegisteredStream(
		TFunctionRef<EBreakBehavior(const FConcertReplicationStream& Stream)> Callback
		) const
	{
		return ensureMsgf(CurrentState, TEXT("StartAcceptingJoinRequests should have been called at this point."))
			? CurrentState->ForEachRegisteredStream(Callback)
			: EStreamEnumerationResult::NoRegisteredStreams;
	}

	TFuture<FConcertReplication_ChangeAuthority_Response> FReplicationManager::RequestAuthorityChange(FConcertReplication_ChangeAuthority_Request Args)
	{
		if (ensureMsgf(CurrentState, TEXT("StartAcceptingJoinRequests should have been called at this point.")))
		{
			return CurrentState->RequestAuthorityChange(Args);
		}
		
		return RejectAll(MoveTemp(Args));
	}

	TFuture<FConcertReplication_QueryReplicationInfo_Response> FReplicationManager::QueryClientInfo(FConcertReplication_QueryReplicationInfo_Request Args)
	{
		return ensureMsgf(CurrentState, TEXT("StartAcceptingJoinRequests should have been called at this point."))
			? CurrentState->QueryClientInfo(MoveTemp(Args))
			: MakeFulfilledPromise<FConcertReplication_QueryReplicationInfo_Response>().GetFuture();
	}

	TFuture<FConcertReplication_ChangeStream_Response> FReplicationManager::ChangeStream(FConcertReplication_ChangeStream_Request Args)
	{
		return ensureMsgf(CurrentState, TEXT("StartAcceptingJoinRequests should have been called at this point."))
			? CurrentState->ChangeStream(MoveTemp(Args))
			: MakeFulfilledPromise<FConcertReplication_ChangeStream_Response>().GetFuture(); 
	}

	IConcertClientReplicationManager::EAuthorityEnumerationResult FReplicationManager::ForEachClientOwnedObject(
		TFunctionRef<EBreakBehavior(const FSoftObjectPath& Object, TSet<FGuid>&& OwningStreams)> Callback) const
	{
		return ensureMsgf(CurrentState, TEXT("StartAcceptingJoinRequests should have been called at this point."))
			? CurrentState->ForEachClientOwnedObject(Callback)
			: EAuthorityEnumerationResult::NoAuthorityAvailable;
	}

	TSet<FGuid> FReplicationManager::GetClientOwnedStreamsForObject(const FSoftObjectPath& ObjectPath) const
	{
		return ensureMsgf(CurrentState, TEXT("StartAcceptingJoinRequests should have been called at this point."))
			? CurrentState->GetClientOwnedStreamsForObject(ObjectPath)
			: TSet<FGuid>{}; 
	}

	bool FReplicationManager::HasAuthorityOver(const FSoftObjectPath& ObjectPath) const
	{
		return ensureMsgf(CurrentState, TEXT("StartAcceptingJoinRequests should have been called at this point."))
			&& CurrentState->HasAuthorityOver(ObjectPath);
	}

	IConcertClientReplicationManager::ESyncControlEnumerationResult FReplicationManager::ForEachSyncControlledObject(TFunctionRef<EBreakBehavior(const FConcertObjectInStreamID& Object)> Callback) const
	{
		// Check() to avoid returning some dummy static variable
		ensureMsgf(CurrentState, TEXT("StartAcceptingJoinRequests should have been called at this point."));
		return CurrentState
			? CurrentState->ForEachSyncControlledObject(Callback)
			: ESyncControlEnumerationResult::NoneAvailable;
	}

	uint32 FReplicationManager::NumSyncControlledObjects() const
	{
		return ensureMsgf(CurrentState, TEXT("StartAcceptingJoinRequests should have been called at this point."))
			? CurrentState->NumSyncControlledObjects()
			: 0;
	}

	bool FReplicationManager::HasSyncControl(const FConcertObjectInStreamID& Object) const
	{
		return ensureMsgf(CurrentState, TEXT("StartAcceptingJoinRequests should have been called at this point."))
			&& CurrentState->HasSyncControl(Object);
	}

	TFuture<FConcertReplication_ChangeMuteState_Response> FReplicationManager::ChangeMuteState(FConcertReplication_ChangeMuteState_Request Request)
	{
		return ensureMsgf(CurrentState, TEXT("StartAcceptingJoinRequests should have been called at this point."))
			? CurrentState->ChangeMuteState(MoveTemp(Request))
			: MakeFulfilledPromise<FConcertReplication_ChangeMuteState_Response>().GetFuture(); 
	}

	TFuture<FConcertReplication_QueryMuteState_Response> FReplicationManager::QueryMuteState(FConcertReplication_QueryMuteState_Request Request)
	{
		return ensureMsgf(CurrentState, TEXT("StartAcceptingJoinRequests should have been called at this point."))
			? CurrentState->QueryMuteState(MoveTemp(Request))
			: MakeFulfilledPromise<FConcertReplication_QueryMuteState_Response>().GetFuture(); 
	}

	TFuture<FConcertReplication_RestoreContent_Response> FReplicationManager::RestoreContent(FConcertReplication_RestoreContent_Request Request)
	{
		return ensureMsgf(CurrentState, TEXT("StartAcceptingJoinRequests should have been called at this point."))
			? CurrentState->RestoreContent(MoveTemp(Request))
			: MakeFulfilledPromise<FConcertReplication_RestoreContent_Response>().GetFuture(); 
	}

	TFuture<FConcertReplication_PutState_Response> FReplicationManager::PutClientState(FConcertReplication_PutState_Request Request)
	{
		return ensureMsgf(CurrentState, TEXT("StartAcceptingJoinRequests should have been called at this point."))
			? CurrentState->PutClientState(MoveTemp(Request))
			: MakeFulfilledPromise<FConcertReplication_PutState_Response>().GetFuture(); 
	}

	IConcertClientReplicationManager::FOnPreStreamsChanged& FReplicationManager::OnPreStreamsChanged()
	{
		// checkf() to avoid returning some dummy static variable
		checkf(CurrentState, TEXT("StartAcceptingJoinRequests should have been called at this point."));
		return CurrentState->OnPreStreamsChanged();
	}

	IConcertClientReplicationManager::FOnPostStreamsChanged& FReplicationManager::OnPostStreamsChanged()
	{
		// checkf() to avoid returning some dummy static variable
		checkf(CurrentState, TEXT("StartAcceptingJoinRequests should have been called at this point."));
		return CurrentState->OnPostStreamsChanged();
	}

	IConcertClientReplicationManager::FOnPreAuthorityChanged& FReplicationManager::OnPreAuthorityChanged()
	{
		// checkf() to avoid returning some dummy static variable
		checkf(CurrentState, TEXT("StartAcceptingJoinRequests should have been called at this point."));
		return CurrentState->OnPreAuthorityChanged();
	}

	IConcertClientReplicationManager::FOnPostAuthorityChanged& FReplicationManager::OnPostAuthorityChanged()
	{
		// checkf() to avoid returning some dummy static variable
		checkf(CurrentState, TEXT("StartAcceptingJoinRequests should have been called at this point."));
		return CurrentState->OnPostAuthorityChanged();
	}
	
	IConcertClientReplicationManager::FSyncControlChanged& FReplicationManager::OnPreSyncControlChanged()
	{
		// checkf() to avoid returning some dummy static variable
		checkf(CurrentState, TEXT("StartAcceptingJoinRequests should have been called at this point."));
		return CurrentState->OnPreSyncControlChanged();
	}

	IConcertClientReplicationManager::FSyncControlChanged& FReplicationManager::OnPostSyncControlChanged()
	{
		// checkf() to avoid returning some dummy static variable
		checkf(CurrentState, TEXT("StartAcceptingJoinRequests should have been called at this point."));
		return CurrentState->OnPostSyncControlChanged();
	}

	IConcertClientReplicationManager::FOnRemoteEditApplied& FReplicationManager::OnPreRemoteEditApplied()
	{
		// checkf() to avoid returning some dummy static variable
		checkf(CurrentState, TEXT("StartAcceptingJoinRequests should have been called at this point."));
		return CurrentState->OnPreRemoteEditApplied();
	}

	IConcertClientReplicationManager::FOnRemoteEditApplied& FReplicationManager::OnPostRemoteEditApplied()
	{
		// checkf() to avoid returning some dummy static variable
		checkf(CurrentState, TEXT("StartAcceptingJoinRequests should have been called at this point."));
		return CurrentState->OnPostRemoteEditApplied();
	}

	void FReplicationManager::OnChangeState(TSharedRef<FReplicationManagerState> NewState)
	{
		CurrentState = MoveTemp(NewState);
	}
}

