// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertSyncSessionFlags.h"
#include "Replication/IConcertClientReplicationManager.h"
#include "Replication/Messages/Handshake.h"
#include "Templates/SharedPointer.h"

class IConcertClientReplicationBridge;
class IConcertClientSession;

namespace UE::ConcertSyncClient::Replication
{
	class FReplicationManagerState;
	
	class FReplicationManager : public IConcertClientReplicationManager
	{
		friend FReplicationManagerState; 
	public:
		
		FReplicationManager(
			TSharedRef<IConcertClientSession> InLiveSession,
			IConcertClientReplicationBridge& InBridge UE_LIFETIMEBOUND,
			EConcertSyncSessionFlags SessionFlags
			);
		virtual ~FReplicationManager() override;

		/** Starts accepting join requests. Must be called separately from constructor because of TSharedFromThis asserting if SharedThis is called in constructor. */
		void StartAcceptingJoinRequests();

		//~ Begin IConcertClientReplicationManager Interface
		virtual TFuture<FJoinReplicatedSessionResult> JoinReplicationSession(FJoinReplicatedSessionArgs Args) override;
		virtual void LeaveReplicationSession() override;
		virtual bool CanJoin() override;
		virtual bool IsConnectedToReplicationSession() override;
		virtual EStreamEnumerationResult ForEachRegisteredStream(TFunctionRef<EBreakBehavior(const FConcertReplicationStream& Stream)> Callback) const override;
		virtual TFuture<FConcertReplication_ChangeAuthority_Response> RequestAuthorityChange(FConcertReplication_ChangeAuthority_Request Args) override;
		virtual TFuture<FConcertReplication_QueryReplicationInfo_Response> QueryClientInfo(FConcertReplication_QueryReplicationInfo_Request Args) override;
		virtual TFuture<FConcertReplication_ChangeStream_Response> ChangeStream(FConcertReplication_ChangeStream_Request Args) override;
		virtual EAuthorityEnumerationResult ForEachClientOwnedObject(TFunctionRef<EBreakBehavior(const FSoftObjectPath& Object, TSet<FGuid>&& OwningStreams)>) const override;
		virtual TSet<FGuid> GetClientOwnedStreamsForObject(const FSoftObjectPath& ObjectPath) const override;
		virtual bool HasAuthorityOver(const FSoftObjectPath& ObjectPath) const override;
		virtual ESyncControlEnumerationResult ForEachSyncControlledObject(TFunctionRef<EBreakBehavior(const FConcertObjectInStreamID& Object)> Callback) const override;
		virtual uint32 NumSyncControlledObjects() const override;
		virtual bool HasSyncControl(const FConcertObjectInStreamID& Object) const override;
		virtual TFuture<FConcertReplication_ChangeMuteState_Response> ChangeMuteState(FConcertReplication_ChangeMuteState_Request Request) override;
		virtual TFuture<FConcertReplication_QueryMuteState_Response> QueryMuteState(FConcertReplication_QueryMuteState_Request Request) override;
		virtual TFuture<FConcertReplication_RestoreContent_Response> RestoreContent(FConcertReplication_RestoreContent_Request Request) override;
		virtual TFuture<FConcertReplication_PutState_Response> PutClientState(FConcertReplication_PutState_Request Request) override;
		virtual FOnPreStreamsChanged& OnPreStreamsChanged() override;
		virtual FOnPostStreamsChanged& OnPostStreamsChanged() override;
		virtual FOnPreAuthorityChanged& OnPreAuthorityChanged() override;
		virtual FOnPostAuthorityChanged& OnPostAuthorityChanged() override;
		virtual FSyncControlChanged& OnPreSyncControlChanged() override;
		virtual FSyncControlChanged& OnPostSyncControlChanged() override;
		virtual FOnRemoteEditApplied& OnPreRemoteEditApplied() override;
		virtual FOnRemoteEditApplied& OnPostRemoteEditApplied() override;
		//~ End IConcertClientReplicationManager Interface

	private:
		
		/** Session instance this manager was created for. */
		TSharedRef<IConcertClientSession> Session;
		/** The replication bridge is responsible for applying received data and generating data to send. */
		IConcertClientReplicationBridge& Bridge;
		/** These flags are passed along to all the states. */
		const EConcertSyncSessionFlags SessionFlags;

		/** The current state this manager is in, e.g. waiting for connection request, connecting, connected, etc. */
		TSharedPtr<FReplicationManagerState> CurrentState;
		
		/** Called by FReplicationManagerState to change the state. */
		void OnChangeState(TSharedRef<FReplicationManagerState> NewState);
	};
}
