// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationManagerState.h"

#include "ReplicationManager.h"

namespace UE::ConcertSyncClient::Replication
{
	FReplicationManagerState::FReplicationManagerState(FReplicationManager& Owner)
		: Owner(Owner)
	{}

	void FReplicationManagerState::ChangeState(TSharedRef<FReplicationManagerState> NewState)
	{
		// OnChangeState destroys us
		TSharedPtr<FReplicationManagerState> Guard = SharedThis(this);
		Owner.OnChangeState(MoveTemp(NewState));
		Owner.CurrentState->OnEnterState();
	}
}
