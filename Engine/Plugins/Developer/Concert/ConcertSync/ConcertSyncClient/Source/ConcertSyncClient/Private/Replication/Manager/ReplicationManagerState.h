// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/IConcertClientReplicationManager.h"
#include "Replication/Manager/Utils/ReplicationManagerUtils.h"
#include "Replication/Messages/Muting.h"

class UConcertSyncReplicationConfig;

namespace UE::ConcertSyncClient::Replication
{
	class FReplicationManager;
	
	/**
	 * Implements the State design pattern (see GOF) for FConcertClientReplicationManager.
	 * Depending on the handshake state, the replication manager will react differently to the implemented IConcertClientReplicationManager functions.
	 */
	class FReplicationManagerState : public IConcertClientReplicationManager, public TSharedFromThis<FReplicationManagerState>
	{
	public:

		FReplicationManagerState(FReplicationManager& Owner);

		// Default implementations for subclasses in which the operation is not valid
		
		//~ Begin IConcertClientReplicationManager Interface
		virtual EStreamEnumerationResult ForEachRegisteredStream(TFunctionRef<EBreakBehavior(const FConcertReplicationStream& Stream)> Callback) const override { return EStreamEnumerationResult::NoRegisteredStreams; }
		virtual TFuture<FConcertReplication_ChangeAuthority_Response> RequestAuthorityChange(FConcertReplication_ChangeAuthority_Request Args) override { return RejectAll(MoveTemp(Args)); }
		virtual TFuture<FConcertReplication_QueryReplicationInfo_Response> QueryClientInfo(FConcertReplication_QueryReplicationInfo_Request Args) override { return MakeFulfilledPromise<FConcertReplication_QueryReplicationInfo_Response>().GetFuture(); }
		virtual TFuture<FConcertReplication_ChangeStream_Response> ChangeStream(FConcertReplication_ChangeStream_Request Args) override { return MakeFulfilledPromise<FConcertReplication_ChangeStream_Response>().GetFuture(); }
		virtual EAuthorityEnumerationResult ForEachClientOwnedObject(TFunctionRef<EBreakBehavior(const FSoftObjectPath& Object, TSet<FGuid>&& OwningStreams)> Callback) const override { return EAuthorityEnumerationResult::NoAuthorityAvailable; }
		virtual TSet<FGuid> GetClientOwnedStreamsForObject(const FSoftObjectPath& ObjectPath) const override { return {}; }
		virtual bool HasAuthorityOver(const FSoftObjectPath& ObjectPath) const override { return false; }
		virtual ESyncControlEnumerationResult ForEachSyncControlledObject(TFunctionRef<EBreakBehavior(const FConcertObjectInStreamID& Object)> Callback) const override { return ESyncControlEnumerationResult::NoneAvailable; }
		virtual uint32 NumSyncControlledObjects() const override { return 0; }
		virtual bool HasSyncControl(const FConcertObjectInStreamID& Object) const override { return false; }
		virtual TFuture<FConcertReplication_ChangeMuteState_Response> ChangeMuteState(FConcertReplication_ChangeMuteState_Request) override { return MakeFulfilledPromise<FConcertReplication_ChangeMuteState_Response>(FConcertReplication_ChangeMuteState_Response{ EConcertReplicationMuteErrorCode::Rejected }).GetFuture(); };
		virtual TFuture<FConcertReplication_QueryMuteState_Response> QueryMuteState(FConcertReplication_QueryMuteState_Request Request) override { return MakeFulfilledPromise<FConcertReplication_QueryMuteState_Response>().GetFuture(); }
		virtual TFuture<FConcertReplication_RestoreContent_Response> RestoreContent(FConcertReplication_RestoreContent_Request Request) override { return MakeFulfilledPromise<FConcertReplication_RestoreContent_Response>().GetFuture(); }
		virtual TFuture<FConcertReplication_PutState_Response> PutClientState(FConcertReplication_PutState_Request Request) override { return MakeFulfilledPromise<FConcertReplication_PutState_Response>().GetFuture(); }
		virtual FOnPreStreamsChanged& OnPreStreamsChanged() override { return OnPreStreamsChangedDelegate; } 
		virtual FOnPostStreamsChanged& OnPostStreamsChanged() override { return OnPostStreamsChangedDelegate; }
		virtual FOnPreAuthorityChanged& OnPreAuthorityChanged() override { return OnPreAuthorityChangedDelegate; }
		virtual FOnPostAuthorityChanged& OnPostAuthorityChanged() override { return OnPostAuthorityChangedDelegate; }
		virtual FSyncControlChanged& OnPreSyncControlChanged() override { return OnPreSyncControlChangedDelegate; }
		virtual FSyncControlChanged& OnPostSyncControlChanged() override { return OnPostSyncControlChangedDelegate; }
		virtual FOnRemoteEditApplied& OnPreRemoteEditApplied() override { return OnPreRemoteEditAppliedDelegate; }
		virtual FOnRemoteEditApplied& OnPostRemoteEditApplied() override { return OnPostRemoteEditAppliedDelegate; }
		//~ End IConcertClientReplicationManager Interface

	protected:
		
		FOnPreStreamsChanged OnPreStreamsChangedDelegate;
		FOnPostStreamsChanged OnPostStreamsChangedDelegate;
		FOnPreAuthorityChanged OnPreAuthorityChangedDelegate;
		FOnPostAuthorityChanged OnPostAuthorityChangedDelegate;
		FSyncControlChanged OnPreSyncControlChangedDelegate;
		FSyncControlChanged OnPostSyncControlChangedDelegate;
		FOnRemoteEditApplied OnPreRemoteEditAppliedDelegate;
		FOnRemoteEditApplied OnPostRemoteEditAppliedDelegate;

		/**
		 * Subclasses can change the state with this function.
		 * Important: This decrements the reference count so it may destroy your state unless you keep it alive manually!
		 */
		void ChangeState(TSharedRef<FReplicationManagerState> NewState);
		
		const FReplicationManager& GetOwner() const { return Owner; }
		FReplicationManager& GetOwner() { return Owner; }
		
	private:

		/** Used to change the state on the owning manager. */
		FReplicationManager& Owner;
		
		/**
		 * Do any logic for entering state here instead of constructor.
		 * 
		 * Important to handle "recursive" calls to ChangState and also constructor does not have access to SharedThis.
		 */
		virtual void OnEnterState() {}
	};

}
