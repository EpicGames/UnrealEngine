// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationManagerState_Connected.h"

#include "ConcertLogGlobal.h"
#include "ConcertTransportMessages.h"
#include "IConcertSession.h"
#include "ReplicationManagerState_Disconnected.h"
#include "Replication/ChangeStreamSharedUtils.h"
#include "Replication/Formats/FullObjectFormat.h"
#include "Replication/Messages/ChangeClientEvent.h"
#include "Replication/Messages/Handshake.h"
#include "Replication/Processing/ClientReplicationDataCollector.h"
#include "Replication/Processing/ObjectReplicationApplierProcessor.h"
#include "Replication/Processing/ObjectReplicationReceiver.h"
#include "Utils/LocalSyncControl.h"
#include "Utils/NetworkMessageLogging.h"
#include "Utils/ReplicationManagerUtils.h"

#include "Algo/RemoveIf.h"
#include "Misc/ScopeExit.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "Utils/ContentComparisionUtils.h"

namespace UE::ConcertSyncClient::Replication
{
	FReplicationManagerState_Connected::FReplicationManagerState_Connected(
		TSharedRef<IConcertClientSession> InLiveSession,
		IConcertClientReplicationBridge& ReplicationBridge,
		FReplicationManager& Owner,
		EConcertSyncSessionFlags SessionFlags,
		TArray<FConcertReplicationStream> InitialStreams,
		const FConcertReplication_ChangeSyncControl& InitialSyncControl
		)
		: FReplicationManagerState(Owner)
		, LiveSession(InLiveSession)
		, ReplicationBridge(ReplicationBridge)
		, SessionFlags(SessionFlags)
		, RegisteredStreams(MoveTemp(InitialStreams))
		// TODO DP: Use config to determine which replication format to use
		, ReplicationFormat(MakeUnique<ConcertSyncCore::FFullObjectFormat>())
		, SyncControl(*LiveSession)
		, ReplicationDataSource(
			ReplicationBridge,
			*ReplicationFormat,
			SyncControl,
			FClientReplicationDataCollector::FGetClientStreams::CreateLambda([this]()
			{
				return &RegisteredStreams;
			}),
			LiveSession->GetSessionClientEndpointId()
			)
		, Sender(
			ConcertSyncCore::FGetObjectFrequencySettings::CreateRaw(this, &FReplicationManagerState_Connected::GetObjectFrequencySettings),
			InLiveSession->GetSessionServerEndpointId(), *LiveSession, ReplicationDataSource
			)
		, ReceivedDataCache(MakeShared<ConcertSyncCore::FObjectReplicationCache>(*ReplicationFormat))
		, Receiver(*InLiveSession, *ReceivedDataCache)
		, ReceivedReplicationQueuer(FClientReplicationDataQueuer::Make(ReplicationBridge, *ReceivedDataCache))
		, ReplicationApplier(ReplicationBridge, *ReplicationFormat, *ReceivedReplicationQueuer)
	{
		SyncControl.ProcessSyncControlChange(InitialSyncControl);
		SyncControl.OnPreSyncControlChanged().AddLambda([this](){ OnPreSyncControlChangedDelegate.Broadcast(); });
		SyncControl.OnPostSyncControlChanged().AddLambda([this](){ OnPostSyncControlChangedDelegate.Broadcast(); });
	}

	FReplicationManagerState_Connected::~FReplicationManagerState_Connected()
	{
		// Technically not needed due to AddSP but let's be nice and clean up after ourselves
		LiveSession->OnTick().RemoveAll(this);
		LiveSession->UnregisterCustomEventHandler<FConcertReplication_ChangeClientEvent>(this);
	}

	TFuture<FJoinReplicatedSessionResult> FReplicationManagerState_Connected::JoinReplicationSession(FJoinReplicatedSessionArgs Args)
	{
		UE_LOGF(LogConcert, Warning, "JoinReplicationSession requested while already in a session");
		return MakeFulfilledPromise<FJoinReplicatedSessionResult>(FJoinReplicatedSessionResult{ EJoinReplicationErrorCode::AlreadyInSession }).GetFuture();
	}

	void FReplicationManagerState_Connected::LeaveReplicationSession()
	{
		LiveSession->SendCustomEvent(FConcertReplication_LeaveEvent{}, LiveSession->GetSessionServerEndpointId(), EConcertMessageFlags::ReliableOrdered);
		ChangeState(MakeShared<FReplicationManagerState_Disconnected>(LiveSession, ReplicationBridge, GetOwner(), SessionFlags));
	}

	IConcertClientReplicationManager::EStreamEnumerationResult FReplicationManagerState_Connected::ForEachRegisteredStream(
		TFunctionRef<EBreakBehavior(const FConcertReplicationStream& Stream)> Callback
		) const
	{
		for (const FConcertReplicationStream& Stream : RegisteredStreams)
		{
			if (Callback(Stream) == EBreakBehavior::Break)
			{
				break;
			}
		}
		return EStreamEnumerationResult::Iterated;
	}

	TFuture<FConcertReplication_ChangeAuthority_Response> FReplicationManagerState_Connected::RequestAuthorityChange(FConcertReplication_ChangeAuthority_Request Args)
	{
		if (CVarSimulateAuthorityTimeouts.GetValueOnGameThread())
		{
			return MakeFulfilledPromise<FConcertReplication_ChangeAuthority_Response>(FConcertReplication_ChangeAuthority_Response{ EReplicationResponseErrorCode::Timeout }).GetFuture();
		}
		if (CVarSimulateAuthorityRejection.GetValueOnGameThread())
		{
			return RejectAll(MoveTemp(Args));
		}
		
		// Stop replicating removed objects right now: the server will remove authority after processing this request.
		// At that point, it will log errors for receiving replication data from a client without authority.
		TMap<FSoftObjectPath, TArray<FGuid>> PredictedChange = ApplyAuthorityChangeRemovedObjects(Args);
		// We don't need worry about updating sync control until it is processed below - the local client will not attempt to replicate the object
		// because we just locally updated the authority cache.

		LogNetworkMessage(CVarLogAuthorityRequestsAndResponsesOnClient, Args);
		return LiveSession->SendCustomRequest<FConcertReplication_ChangeAuthority_Request, FConcertReplication_ChangeAuthority_Response>(Args, LiveSession->GetSessionServerEndpointId())
			.Next(
				[WeakThis = TWeakPtr<FReplicationManagerState_Connected>(SharedThis(this)), Args, PredictedChange = MoveTemp(PredictedChange)]
				(FConcertReplication_ChangeAuthority_Response&& Response) mutable
				{
					LogNetworkMessage(CVarLogAuthorityRequestsAndResponsesOnClient, Response);
					
					if (const TSharedPtr<FReplicationManagerState_Connected> ThisPin = WeakThis.Pin()
						; ThisPin && Response.ErrorCode == EReplicationResponseErrorCode::Handled)
					{
						ThisPin->FinalizePredictedAuthorityChange(Args, Response.RejectedObjects, Response.SyncControl);
					}
					else if (ThisPin && Response.ErrorCode == EReplicationResponseErrorCode::Timeout)
					{
						// ApplyAuthorityChangeRemovedObjects caused PredictedChange to stop being replicated. Revert.
						ThisPin->RevertAuthorityChangeReleasedObjects(PredictedChange);
					}

					return FConcertReplication_ChangeAuthority_Response { MoveTemp(Response) };
				});
	}

	TFuture<FConcertReplication_QueryReplicationInfo_Response> FReplicationManagerState_Connected::QueryClientInfo(FConcertReplication_QueryReplicationInfo_Request Args)
	{
		if (CVarSimulateQueryTimeouts.GetValueOnGameThread())
		{
			return MakeFulfilledPromise<FConcertReplication_QueryReplicationInfo_Response>(FConcertReplication_QueryReplicationInfo_Response{ EReplicationResponseErrorCode::Timeout }).GetFuture();
		}
		
		if (EnumHasAllFlags(Args.QueryFlags, EConcertQueryClientStreamFlags::SkipAuthority | EConcertQueryClientStreamFlags::SkipStreamInfo | EConcertQueryClientStreamFlags::SkipFrequency ))
		{
			UE_LOGF(LogConcert, Warning, "Request QueryClientInfo is pointless because SkipAuthority and SkipStreamInfo are both set. Returning immediately...");
			return MakeFulfilledPromise<FConcertReplication_QueryReplicationInfo_Response>().GetFuture();
		}
		
		return LiveSession->SendCustomRequest<FConcertReplication_QueryReplicationInfo_Request, FConcertReplication_QueryReplicationInfo_Response>(Args, LiveSession->GetSessionServerEndpointId())
			.Next([](FConcertReplication_QueryReplicationInfo_Response&& Response)
			{
				return FConcertReplication_QueryReplicationInfo_Response { MoveTemp(Response) };
			});
	}

	TFuture<FConcertReplication_ChangeStream_Response> FReplicationManagerState_Connected::ChangeStream(FConcertReplication_ChangeStream_Request Args)
	{
		if (CVarSimulateStreamChangeTimeouts.GetValueOnGameThread())
		{
			return MakeFulfilledPromise<FConcertReplication_ChangeStream_Response>(FConcertReplication_ChangeStream_Response{ EReplicationResponseErrorCode::Timeout }).GetFuture();
		}
		
		// Stop replicating removed objects right now: the server will remove authority after processing this request.
		// At that point, it will log errors for receiving replication data from a client without authority.
		FChangeStreamPredictedChange PredictedChange = PredictAndApplyStreamChangeRemovedObjects(Args);
		// We don't need worry about updating sync control until it is processed below - the local client will not attempt to replicate the object
		// because we just locally updated the replication cache.
		
		LogNetworkMessage(CVarLogStreamRequestsAndResponsesOnClient, Args);
		return LiveSession->SendCustomRequest<FConcertReplication_ChangeStream_Request, FConcertReplication_ChangeStream_Response>(Args, LiveSession->GetSessionServerEndpointId())
			.Next([WeakThis = TWeakPtr<FReplicationManagerState_Connected>(SharedThis(this)), Args, PredictedChange = MoveTemp(PredictedChange)](FConcertReplication_ChangeStream_Response&& Response)
			{
				LogNetworkMessage(CVarLogStreamRequestsAndResponsesOnClient, Response);

				const TSharedPtr<FReplicationManagerState_Connected> ThisPin = WeakThis.Pin();
				if (ThisPin && Response.IsSuccess())
				{
					ThisPin->FinalizePredictedStreamChange(Args);
				}
				else if (ThisPin && Response.ErrorCode == EReplicationResponseErrorCode::Timeout)
				{
					// HandleRemovingReplicatedObjects caused Request.ObjectsToRemove to stop being replicated. Revert.
					ThisPin->RevertPredictedStreamChangeRemovedObjects(PredictedChange);
				}
				
				return FConcertReplication_ChangeStream_Response { MoveTemp(Response) };
			});
	}

	IConcertClientReplicationManager::EAuthorityEnumerationResult FReplicationManagerState_Connected::ForEachClientOwnedObject(
		TFunctionRef<EBreakBehavior(const FSoftObjectPath& Object, TSet<FGuid>&& OwningStreams)> Callback
		) const
	{
		TSet<FGuid> Result;
		int32 ExpectedNumStreams = 0;
		ReplicationDataSource.ForEachOwnedObject([this, &Callback, &Result, &ExpectedNumStreams](const FSoftObjectPath& ObjectPath)
		{
			// Reuse TSet (if possible) for a slightly better memory footprint
			ExpectedNumStreams = FMath::Max(ExpectedNumStreams, Result.Num());
			Result.Empty(Result.Num());
			
			ReplicationDataSource.AppendOwningStreamsForObject(ObjectPath, Result);
			return Callback(ObjectPath, MoveTemp(Result));
		});
		return EAuthorityEnumerationResult::Iterated;
	}

	TSet<FGuid> FReplicationManagerState_Connected::GetClientOwnedStreamsForObject(
		const FSoftObjectPath& ObjectPath
		) const
	{
		TSet<FGuid> Result;
		ReplicationDataSource.AppendOwningStreamsForObject(ObjectPath, Result);
		return Result;
	}

	bool FReplicationManagerState_Connected::HasAuthorityOver(const FSoftObjectPath& ObjectPath) const
	{
		return ReplicationDataSource.OwnsObjectInAnyStream(ObjectPath);
	}

	IConcertClientReplicationManager::ESyncControlEnumerationResult FReplicationManagerState_Connected::ForEachSyncControlledObject(TFunctionRef<EBreakBehavior(const FConcertObjectInStreamID& Object)> Callback) const
	{
		return SyncControl.EnumerateAllowedObjects(Callback)
			? ESyncControlEnumerationResult::Iterated
			: ESyncControlEnumerationResult::NoneAvailable;
	}

	TFuture<FConcertReplication_ChangeMuteState_Response> FReplicationManagerState_Connected::ChangeMuteState(FConcertReplication_ChangeMuteState_Request Request)
	{
		if (!EnumHasAnyFlags(SessionFlags, EConcertSyncSessionFlags::ShouldAllowGlobalMuting) || CVarSimulateMuteRequestRejection.GetValueOnAnyThread())
		{
			return MakeFulfilledPromise<FConcertReplication_ChangeMuteState_Response>(FConcertReplication_ChangeMuteState_Response{ EConcertReplicationMuteErrorCode::Rejected }).GetFuture();
		}

		FLocalSyncControl::FPredictedObjectRemoval PredictedChanges = SyncControl.PredictAndApplyMuteChanges(Request);
		
		LogNetworkMessage(CVarLogMuteRequestsAndResponsesOnClient, Request);
		return LiveSession->SendCustomRequest<FConcertReplication_ChangeMuteState_Request, FConcertReplication_ChangeMuteState_Response>(Request, LiveSession->GetSessionServerEndpointId())
			.Next([WeakThis = TWeakPtr<FReplicationManagerState_Connected>(SharedThis(this)), PredictedChanges = MoveTemp(PredictedChanges)](FConcertReplication_ChangeMuteState_Response&& Response)
			{
				LogNetworkMessage(CVarLogMuteRequestsAndResponsesOnClient, Response);
				if (const TSharedPtr<FReplicationManagerState_Connected> This = WeakThis.Pin())
				{
					This->SyncControl.ApplyOrRevertMuteResponse(PredictedChanges, Response);
				}
				return Response;
			});
	}

	TFuture<FConcertReplication_QueryMuteState_Response> FReplicationManagerState_Connected::QueryMuteState(FConcertReplication_QueryMuteState_Request Request)
	{
		if (!EnumHasAnyFlags(SessionFlags, EConcertSyncSessionFlags::ShouldAllowGlobalMuting))
		{
			return MakeFulfilledPromise<FConcertReplication_QueryMuteState_Response>().GetFuture();
		}
		
		return LiveSession->SendCustomRequest<FConcertReplication_QueryMuteState_Request, FConcertReplication_QueryMuteState_Response>(
			Request,
			LiveSession->GetSessionServerEndpointId()
			);
	}

	TFuture<FConcertReplication_RestoreContent_Response> FReplicationManagerState_Connected::RestoreContent(FConcertReplication_RestoreContent_Request Request)
	{
		if (!EnumHasAnyFlags(SessionFlags, EConcertSyncSessionFlags::ShouldEnableReplicationActivities))
		{
			return MakeFulfilledPromise<FConcertReplication_RestoreContent_Response>(FConcertReplication_RestoreContent_Response{ EConcertReplicationRestoreErrorCode::NotSupported }).GetFuture();
		}
		
		FLocalSyncControl::FPredictedObjectRemoval PredictedChanges = SyncControl.PredictAndApplyRestoreContentChanges(Request);

		// We want the response to contain ClientInfo to update our internal state - so set the SendNewState flag
		const bool bWantedNewState = EnumHasAnyFlags(Request.Flags, EConcertReplicationRestoreContentFlags::SendNewState);
		Request.Flags |= EConcertReplicationRestoreContentFlags::SendNewState;
		LogNetworkMessage(CVarLogRestoreContentRequestsAndResponsesOnClient, Request);
		
		return LiveSession->SendCustomRequest<FConcertReplication_RestoreContent_Request, FConcertReplication_RestoreContent_Response>(Request, LiveSession->GetSessionServerEndpointId())
			.Next([WeakThis = TWeakPtr<FReplicationManagerState_Connected>(SharedThis(this)), PredictedChanges = MoveTemp(PredictedChanges)](FConcertReplication_RestoreContent_Response&& Response)
			{
				LogNetworkMessage(CVarLogRestoreContentRequestsAndResponsesOnClient, Response);
				
				const TSharedPtr<FReplicationManagerState_Connected> This = WeakThis.Pin();
				if (!This)
				{
					return Response;
				}

				This->SyncControl.ApplyOrRevertRestoreContentResponse(PredictedChanges, Response);
				if (Response.IsSuccess())
				{
					// Update the list of objects we'll be replicating. This is why we added the SendNewState flag above.
					This->UpdateReplicatedObjectAfterServerSideChange(Response.ClientInfo);
				}
				
				return Response;
			})
			.Next([bWantedNewState](FConcertReplication_RestoreContent_Response&& Response)
			{
				// Since we added EConcertReplicationRestoreContentFlags::SendNewState above, remove the returned data if the flag was not originally in the request.
				if (!bWantedNewState)
				{
					Response.ClientInfo = {};
				}
				return Response;
			});
	}

	void FReplicationManagerState_Connected::OnEnterState()
	{
		LiveSession->OnTick().AddSP(this, &FReplicationManagerState_Connected::Tick);
		LiveSession->RegisterCustomEventHandler<FConcertReplication_ChangeClientEvent>(this, &FReplicationManagerState_Connected::HandleChangeClientEvent);
	}

	void FReplicationManagerState_Connected::Tick(IConcertClientSession& Session, float DeltaTime)
	{
		// TODO UE-190714: We should set a time budget for the client so ticking does not cause frame spikes
		const ConcertSyncCore::FProcessObjectsParams Params { DeltaTime };
		Sender.ProcessObjects(Params);
		ReplicationApplier.ProcessObjects(Params);
	}

	FChangeStreamPredictedChange FReplicationManagerState_Connected::PredictAndApplyStreamChangeRemovedObjects(const FConcertReplication_ChangeStream_Request& Request)
	{
		FChangeStreamPredictedChange Change;
		Change.ObjectsRemovedFromStream = ComputeRemovedObjects(RegisteredStreams, Request);
		Change.AuthorityRemovedFromStreams = RemoveObjectsFromAuthority(Change.ObjectsRemovedFromStream);
		return Change;
	}

	void FReplicationManagerState_Connected::RevertPredictedStreamChangeRemovedObjects(const FChangeStreamPredictedChange& PredictedChange)
	{
		if (PredictedChange.AuthorityRemovedFromStreams.IsEmpty())
		{
			return;
		}

		OnPreAuthorityChangedDelegate.Broadcast();
		for (const TPair<FSoftObjectPath, TArray<FGuid>>& RemovedObjectInfo : PredictedChange.AuthorityRemovedFromStreams)
		{
			ReplicationDataSource.AddReplicatedObjectStreams(RemovedObjectInfo.Key, RemovedObjectInfo.Value);
		}
		OnPostAuthorityChangedDelegate.Broadcast();
	}
	
	void FReplicationManagerState_Connected::FinalizePredictedStreamChange(const FConcertReplication_ChangeStream_Request& StreamChange)
	{
		SyncControl.ProcessStreamChange(StreamChange);
		UpdateReplicatedObjectsAfterStreamChange(StreamChange);
	}
	
	void FReplicationManagerState_Connected::UpdateReplicatedObjectsAfterStreamChange(const FConcertReplication_ChangeStream_Request& Request)
	{
		OnPreStreamsChangedDelegate.Broadcast();
		ON_SCOPE_EXIT{ OnPostStreamsChangedDelegate.Broadcast(); };
		
		// Build RegisteredStreams while RegisteredStreams has the old, unupdated state
		TMap<FSoftObjectPath, TArray<FGuid>> BundledModifiedObjects;
		for (const TPair<FConcertObjectInStreamID, FConcertReplication_ChangeStream_PutObject>& PutObject : Request.ObjectsToPut)
		{
			const FConcertObjectInStreamID ObjectInfo = PutObject.Key;
			const FSoftObjectPath Object = ObjectInfo.Object;
			const FGuid StreamId = ObjectInfo.StreamId;
			
			const FConcertReplicationStream* StreamDescription = RegisteredStreams.FindByPredicate([&StreamId](const FConcertReplicationStream& Stream)
			{
				return Stream.BaseDescription.Identifier == StreamId;
			});
			// ReplicationDataSource only cares about inflight objects.
			// Newly added objects inflight because the client must first request authority.
			const bool bWasAddedByRequest = !ensure(StreamDescription) || !StreamDescription->BaseDescription.ReplicationMap.ReplicatedObjects.Contains(Object);
			if (!bWasAddedByRequest)
			{
				BundledModifiedObjects.FindOrAdd(Object).Add(StreamId);
			}
		}

		// The local cache must be updated before calling OnObjectStreamModified
		ConcertSyncCore::Replication::ChangeStreamUtils::ApplyValidatedRequest(Request, RegisteredStreams);
		
		for (const TPair<FSoftObjectPath, TArray<FGuid>>& ModifiedObject : BundledModifiedObjects)
		{
			ReplicationDataSource.OnObjectStreamModified(ModifiedObject.Key, ModifiedObject.Value);
		}
	}

	TMap<FSoftObjectPath, TArray<FGuid>> FReplicationManagerState_Connected::ApplyAuthorityChangeRemovedObjects(
		const FConcertReplication_ChangeAuthority_Request& Request
		)
	{
		return RemoveObjectsFromAuthority(Request.ReleaseAuthority);
	}

	void FReplicationManagerState_Connected::RevertAuthorityChangeReleasedObjects(const TMap<FSoftObjectPath, TArray<FGuid>>& PredictedChange)
	{
		OnPreAuthorityChangedDelegate.Broadcast();
		for (const TPair<FSoftObjectPath, TArray<FGuid>>& ReleaseAuthority : PredictedChange)
		{
			ReplicationDataSource.AddReplicatedObjectStreams(ReleaseAuthority.Key, ReleaseAuthority.Value);
		}
		OnPostAuthorityChangedDelegate.Broadcast();
	}

	void FReplicationManagerState_Connected::FinalizePredictedAuthorityChange(
		const FConcertReplication_ChangeAuthority_Request& AuthorityChange,
		const TMap<FSoftObjectPath, FConcertStreamArray>& RejectedObjects,
		const FConcertReplication_ChangeSyncControl& SyncControlChange
		)
	{
		SyncControl.ProcessAuthorityChange(AuthorityChange, SyncControlChange);
		UpdateReplicatedObjectsAfterAuthorityChange(AuthorityChange, RejectedObjects);
	}
	
	void FReplicationManagerState_Connected::UpdateReplicatedObjectsAfterAuthorityChange(
		const FConcertReplication_ChangeAuthority_Request& Request,
		const TMap<FSoftObjectPath, FConcertStreamArray>& RejectedObjects
		)
	{
		OnPreAuthorityChangedDelegate.Broadcast();
		ON_SCOPE_EXIT{ OnPostAuthorityChangedDelegate.Broadcast(); };
		
		for (const TPair<FSoftObjectPath, FConcertStreamArray>& TakeAuthority : Request.TakeAuthority)
		{
			const FSoftObjectPath& ReplicatedObject = TakeAuthority.Key;
			FConcertStreamArray ReplicatedStreams = TakeAuthority.Value;
			
			UE_CLOGF(ReplicatedStreams.StreamIds.IsEmpty(), LogConcert, Warning, "Your FConcertReplication_ChangeAuthority_Request::TakeAuthority request contained empty stream ID array for object %ls", *ReplicatedObject.ToString());
			const FConcertStreamArray* RejectedStreams = RejectedObjects.Find(ReplicatedObject);
			if (RejectedStreams)
			{
				ReplicatedStreams.StreamIds.SetNum(Algo::RemoveIf(ReplicatedStreams.StreamIds, [RejectedStreams](const FGuid& Stream)
				{
					return RejectedStreams->StreamIds.Contains(Stream);
				}));
			}

			const bool bWasFullyRejected = ReplicatedStreams.StreamIds.IsEmpty(); 
			if (!bWasFullyRejected)
			{
				ReplicationDataSource.AddReplicatedObjectStreams(TakeAuthority.Key, ReplicatedStreams.StreamIds);
			}
		}
	}

	void FReplicationManagerState_Connected::UpdateReplicatedObjectAfterServerSideChange(const FConcertQueriedClientInfo& NewState)
	{
		if (!AreStreamsEquivalent(NewState.Streams, RegisteredStreams))
		{
			OnPreStreamsChangedDelegate.Broadcast();
			
			RegisteredStreams.Empty(NewState.Streams.Num());
			Algo::Transform(NewState.Streams, RegisteredStreams, [](const FConcertBaseStreamInfo& Info)
			{
				return FConcertReplicationStream{ Info };
			});
			
			OnPostStreamsChangedDelegate.Broadcast();
		}

		if (!IsAuthorityEquivalent(NewState.Authority, ReplicationDataSource))
		{
			OnPreAuthorityChangedDelegate.Broadcast();
			
			ReplicationDataSource.ClearReplicatedObjects();
			for (const FConcertAuthorityClientInfo& AuthorityState : NewState.Authority)
			{
				for (const FSoftObjectPath& ObjectPath : AuthorityState.AuthoredObjects)
				{
					ReplicationDataSource.AddReplicatedObjectStreams(ObjectPath, { AuthorityState.StreamId });
				}
			}
			
			OnPostAuthorityChangedDelegate.Broadcast();
		}
	}

	FConcertObjectReplicationSettings FReplicationManagerState_Connected::GetObjectFrequencySettings(const FConcertReplicatedObjectId& Object) const
	{
		const FConcertReplicationStream* Stream = RegisteredStreams.FindByPredicate([&Object](const FConcertReplicationStream& Description)
		{
			return Description.BaseDescription.Identifier == Object.StreamId;
		});
		
		if (!ensureMsgf(Stream, TEXT("Caller of GetObjectFrequencySettings is trying to send an object that is not registered with the client")))
		{
			UE_LOGF(LogConcert, Warning, "Requested frequency settings for unknown stream %ls and object %ls", *Object.StreamId.ToString(), *Object.Object.ToString());
			return {};
		}
		
		return Stream->BaseDescription.FrequencySettings.GetSettingsFor(Object.Object);
	}
}

