// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertSyncSessionFlags.h"
#include "ReplicationManagerState.h"
#include "Replication/Formats/IObjectReplicationFormat.h"
#include "Replication/Messages/ChangeClientEvent.h"
#include "Replication/Processing/ClientReplicationDataCollector.h"
#include "Replication/Processing/ClientReplicationDataQueuer.h"
#include "Replication/Processing/ObjectReplicationApplierProcessor.h"
#include "Replication/Processing/ObjectReplicationReceiver.h"
#include "Replication/Processing/ObjectReplicationSender.h"
#include "Replication/Processing/Proxy/ObjectProcessorProxy_Frequency.h"
#include "Utils/AuthorityRemovalPrediction.h"
#include "Utils/LocalSyncControl.h"

#include <type_traits>

class IConcertClientSession;

namespace UE::ConcertSyncCore { class FObjectReplicationReceiver; }

namespace UE::ConcertSyncClient::Replication
{
	struct FChangeStreamPredictedChange
	{
		TMap<FSoftObjectPath, TArray<FGuid>> ObjectsRemovedFromStream;
		TMap<FSoftObjectPath, TArray<FGuid>> AuthorityRemovedFromStreams;
	};
	
	/**
	 * State for when the client has successfully completed a replication handshake.
	 *
	 * Every tick this state tries to
	 *	- collect data and sends it to the server
	 *	- process received data and applies it
	 */
	class FReplicationManagerState_Connected : public FReplicationManagerState
	{
	public:
		
		FReplicationManagerState_Connected(
			TSharedRef<IConcertClientSession> InLiveSession,
			IConcertClientReplicationBridge& ReplicationBridge UE_LIFETIMEBOUND,
			FReplicationManager& Owner UE_LIFETIMEBOUND,
			EConcertSyncSessionFlags SessionFlags,
			TArray<FConcertReplicationStream> InitialStreams,
			const FConcertReplication_ChangeSyncControl& InitialSyncControl
			);
		virtual ~FReplicationManagerState_Connected() override;

		//~ Begin IConcertClientReplicationManager Interface
		virtual TFuture<FJoinReplicatedSessionResult> JoinReplicationSession(FJoinReplicatedSessionArgs Args) override;
		virtual void LeaveReplicationSession() override;
		virtual bool CanJoin() override { return false; }
		virtual bool IsConnectedToReplicationSession() override { return true; }
		virtual EStreamEnumerationResult ForEachRegisteredStream(TFunctionRef<EBreakBehavior(const FConcertReplicationStream& Stream)> Callback) const override;
		virtual TFuture<FConcertReplication_ChangeAuthority_Response> RequestAuthorityChange(FConcertReplication_ChangeAuthority_Request Args) override;
		virtual TFuture<FConcertReplication_QueryReplicationInfo_Response> QueryClientInfo(FConcertReplication_QueryReplicationInfo_Request Args) override;
		virtual TFuture<FConcertReplication_ChangeStream_Response> ChangeStream(FConcertReplication_ChangeStream_Request Args) override;
		virtual EAuthorityEnumerationResult ForEachClientOwnedObject(TFunctionRef<EBreakBehavior(const FSoftObjectPath& Object, TSet<FGuid>&& OwningStreams)> Callback) const override;
		virtual TSet<FGuid> GetClientOwnedStreamsForObject(const FSoftObjectPath& ObjectPath) const override;
		virtual bool HasAuthorityOver(const FSoftObjectPath& ObjectPath) const override;
		virtual ESyncControlEnumerationResult ForEachSyncControlledObject(TFunctionRef<EBreakBehavior(const FConcertObjectInStreamID& Object)> Callback) const override;
		virtual uint32 NumSyncControlledObjects() const override { return SyncControl.Num(); }
		virtual bool HasSyncControl(const FConcertObjectInStreamID& Object) const override { return SyncControl.IsObjectAllowed(Object); }
		virtual TFuture<FConcertReplication_ChangeMuteState_Response> ChangeMuteState(FConcertReplication_ChangeMuteState_Request) override;
		virtual TFuture<FConcertReplication_QueryMuteState_Response> QueryMuteState(FConcertReplication_QueryMuteState_Request Request) override;
		virtual TFuture<FConcertReplication_RestoreContent_Response> RestoreContent(FConcertReplication_RestoreContent_Request Request) override;
		virtual TFuture<FConcertReplication_PutState_Response> PutClientState(FConcertReplication_PutState_Request Request) override;
		//~ End IConcertClientReplicationManager Interface

	private:

		/** Passed to FReplicationManagerState_Disconnected */
		const TSharedRef<IConcertClientSession> LiveSession;
		/** Passed to FReplicationManagerState_Disconnected */
		IConcertClientReplicationBridge& ReplicationBridge;
		/** Passed to FReplicationManagerState_Disconnected and used to determine whether certain operations are supported by the server. */
		const EConcertSyncSessionFlags SessionFlags;
		/** The streams this client has registered with the server. */
		TArray<FConcertReplicationStream> RegisteredStreams;
		
		/** The format this client will use for sending & receiving data. */
		const TUniquePtr<ConcertSyncCore::IObjectReplicationFormat> ReplicationFormat;

		// Sending
		/** Decides whether an object should be replicated. */
		FLocalSyncControl SyncControl;
		/** Used as source of replication data. */
		FClientReplicationDataCollector ReplicationDataSource;
		
		/** Sends to remote endpoint and makes sure the objects are replicated at the specified frequency settings. */
		using FDataRelayThrottledByFrequency = ConcertSyncCore::TObjectProcessorProxy_Frequency<ConcertSyncCore::FObjectReplicationSender>;
		/** Sends data collected by ReplicationDataSource to the server. */
		FDataRelayThrottledByFrequency Sender;

		// Receiving
		/** Stores data received by Receiver until it is consumed by ReceivedReplicationQueuer. */
		const TSharedRef<ConcertSyncCore::FObjectReplicationCache> ReceivedDataCache;
		/** Receives data from remote endpoints via message bus.  */
		ConcertSyncCore::FObjectReplicationReceiver Receiver;
		/** Queues data until is can be processed. Shared because FObjectReplicationCache API expects it. */
		const TSharedRef<FClientReplicationDataQueuer> ReceivedReplicationQueuer;
		/** Processes data from ReceivedReplicationQueuer once we tick. */
		FObjectReplicationApplierProcessor ReplicationApplier;

		//~ Begin FReplicationManagerState Interface
		virtual void OnEnterState() override;
		//~ End FReplicationManagerState Interface

		/**
		 * Ticks this client.
		 * 
		 * This processes:
		 *  - data that is to be sent
		 *  - data that was received
		 *
		 * The tasks have a time budget so that the frame rate remains stable.
		 * It is configured in the project settings TODO: Add config
		 */
		void Tick(IConcertClientSession& Session, float DeltaTime);

		/** Handle the server telling us that our state has changed. */
		void HandleChangeClientEvent(const FConcertSessionContext& Context, const FConcertReplication_ChangeClientEvent& Event);

		/** Changes the local state assuming that Request will succeed. */
		FChangeStreamPredictedChange PredictAndApplyStreamChangeRemovedObjects(const FConcertReplication_ChangeStream_Request& Request);
		/** Reverts changes previously made by PredictStreamChangeRemovedObjects. */
		void RevertPredictedStreamChangeRemovedObjects(const FChangeStreamPredictedChange& PredictedChange);
		/** Applies stream changes that we previously predicted using PredictStreamChangeRemovedObjects. */
		void FinalizePredictedStreamChange(const FConcertReplication_ChangeStream_Request& StreamChange);
		/** Updates replicated objects affected by the change request. */
		void UpdateReplicatedObjectsAfterStreamChange(const FConcertReplication_ChangeStream_Request& Request);
		
		/** Changes the local state assuming that Request will succeed. */
		TMap<FSoftObjectPath, TArray<FGuid>> ApplyAuthorityChangeRemovedObjects(const FConcertReplication_ChangeAuthority_Request& Request);
		/** Reverts changes previously made by PredictAuthorityChangeReleasedObjects. */
		void RevertAuthorityChangeReleasedObjects(const TMap<FSoftObjectPath, TArray<FGuid>>& PredictedChange);
		/** Applies authority changes that we previously predicted using PredictAuthorityChangeReleasedObjects. */
		void FinalizePredictedAuthorityChange(const FConcertReplication_ChangeAuthority_Request& AuthorityChange, const TMap<FSoftObjectPath, FConcertStreamArray>& RejectedObjects, const FConcertReplication_ChangeSyncControl& SyncControlChange);
		/** Updates the objects which should be replicated after changing authority. */
		void UpdateReplicatedObjectsAfterAuthorityChange(const FConcertReplication_ChangeAuthority_Request& Request, const TMap<FSoftObjectPath, FConcertStreamArray>& RejectedObjects);

		/**
		 * Shared logic for removing objects from authority.
		 * @return The objects that were actually removed (so it can be reverted).
		 */
		template<typename TStreamIdArray> requires std::is_same_v<TStreamIdArray, TArray<FGuid>> || std::is_same_v<TStreamIdArray, FConcertStreamArray>
		TMap<FSoftObjectPath, TArray<FGuid>> RemoveObjectsFromAuthority(const TMap<FSoftObjectPath, TStreamIdArray>& PredicatedObjects); 

		/** Updates the objects which should be replicated after they have been reset to a completely new state (e.g. when restoring session content manually). */
		void UpdateReplicatedObjectAfterServerSideChange(const FConcertQueriedClientInfo& NewState);
		
		/** Callback to Sender for obtaining an object's frequency settings. */
		FConcertObjectReplicationSettings GetObjectFrequencySettings(const FConcertReplicatedObjectId& Object) const;
	};
}

namespace UE::ConcertSyncClient::Replication
{
	template <typename TStreamIdArray> requires std::is_same_v<TStreamIdArray, TArray<FGuid>> || std::is_same_v<TStreamIdArray, FConcertStreamArray>
	TMap<FSoftObjectPath, TArray<FGuid>> FReplicationManagerState_Connected::RemoveObjectsFromAuthority(
		const TMap<FSoftObjectPath, TStreamIdArray>& PredicatedObjects
	)
	{
		if (!HasAuthorityChanges(ReplicationDataSource, PredicatedObjects))
		{
			return {};
		}
		
		OnPreAuthorityChangedDelegate.Broadcast();
		ON_SCOPE_EXIT{ OnPostAuthorityChangedDelegate.Broadcast(); };
		return RemoveObjectsAndTrackRemovedAuthority(ReplicationDataSource, PredicatedObjects);
	}
}
