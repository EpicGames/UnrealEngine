// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationManagerState_Connected.h"

#include "ConcertLogGlobal.h"
#include "IConcertSession.h"
#include "ReplicationManagerState_Disconnected.h"
#include "Replication/ChangeStreamSharedUtils.h"
#include "Replication/Messages/ChangeClientEvent.h"
#include "Replication/Processing/ClientReplicationDataCollector.h"
#include "Utils/LocalSyncControl.h"
#include "Utils/NetworkMessageLogging.h"
#include "Utils/ReplicationManagerUtils.h"

#include "Misc/ScopeExit.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

namespace UE::ConcertSyncClient::Replication
{
	namespace Private
	{
		static TOptional<FConcertReplication_ChangeStream_Request> BuildDeltaStreamChange(
			const FGuid& EndpointId,
			const TConstArrayView<FConcertReplicationStream>& Streams,
			const FConcertReplication_PutState_Request& Request
			)
		{
			const FConcertReplicationStreamArray* LocallyNewStreams = Request.NewStreams.Find(EndpointId);
			if (!LocallyNewStreams)
			{
				return {};
			}
			
			FConcertReplication_ChangeStream_Request StreamRequest;
			Algo::Transform(Streams, StreamRequest.StreamsToRemove, [](const FConcertReplicationStream& Stream){ return Stream.BaseDescription.Identifier; });
			StreamRequest.StreamsToAdd = LocallyNewStreams->Streams;
			return StreamRequest;
		}

		static TOptional<FConcertReplication_ChangeAuthority_Request> BuildDeltaAuthorityChange(const FGuid& EndpointId, const FConcertReplication_PutState_Request& Request)
		{
			const FConcertObjectInStreamArray* LocallyNewAuthority = Request.NewAuthorityState.Find(EndpointId);
			if (!LocallyNewAuthority)
			{
				return {};
			}
			
			FConcertReplication_ChangeAuthority_Request AuthorityRequest;
			for (const FConcertObjectInStreamID& ObjectId : LocallyNewAuthority->Objects)
			{
				AuthorityRequest.TakeAuthority.FindOrAdd(ObjectId.Object).StreamIds.AddUnique(ObjectId.StreamId);
			}
			return AuthorityRequest;
		}

		static FConcertReplication_ChangeSyncControl BuildRetainedSyncControl(
			const FGuid& EndpointId,
			const FLocalSyncControl& SyncControl,
			const FConcertReplication_PutState_Request& Request
			)
		{
			FConcertReplication_ChangeSyncControl RetainedSyncControl;
			
			const FConcertObjectInStreamArray* LocallyNewAuthority = Request.NewAuthorityState.Find(EndpointId);
			if (!LocallyNewAuthority)
			{
				// The client has not specified authority changes, so the sync control it has right now should be retained
				SyncControl.EnumerateAllowedObjects([&RetainedSyncControl](const FConcertObjectInStreamID& ObjectId)
				{
					RetainedSyncControl.NewControlStates.Add(ObjectId, true);
					return EBreakBehavior::Continue;
				});
				return RetainedSyncControl;
			}
			
			for (const FConcertObjectInStreamID& ObjectId : LocallyNewAuthority->Objects)
			{
				if (SyncControl.IsObjectAllowed(ObjectId))
				{
					RetainedSyncControl.NewControlStates.Add(ObjectId, true);
				}
			}
			return RetainedSyncControl;
		}

		static void RemoveEntriesThatShouldNotBeRetained(
			const FConcertReplication_ChangeSyncControl& ResponseSyncControl,
			const TConstArrayView<FConcertReplicationStream>& RegisteredStreams,
			FConcertReplication_ChangeSyncControl& OutRetainedSyncControl
			)
		{
			for (auto It = OutRetainedSyncControl.NewControlStates.CreateIterator(); It; ++It)
			{
				const FConcertObjectInStreamID& ObjectId = It->Key;

				const bool bIsOverridenByResponse = ResponseSyncControl.NewControlStates.Contains(ObjectId);
				const bool bIsRegistered = RegisteredStreams.ContainsByPredicate([&ObjectId](const FConcertReplicationStream& Stream)
				{
					const bool bIsSameStream = Stream.BaseDescription.Identifier == ObjectId.StreamId;
					const bool bIsContainedInStream = Stream.BaseDescription.ReplicationMap.ReplicatedObjects.Contains(ObjectId.Object);
					return bIsSameStream && bIsContainedInStream;
				});

				// If the object is contained in the response, the sync control has changed so we should not reapply it.
				// If the request removed the object, it should also not be applied.
				if (bIsOverridenByResponse || !bIsRegistered)
				{
					It.RemoveCurrent();
				}
			}
		}

		struct FPutStateChange
		{
			const TOptional<FConcertReplication_ChangeStream_Request> StreamChange;
			const TOptional<FConcertReplication_ChangeAuthority_Request> AuthorityChange;
			const TMap<FSoftObjectPath, TArray<FGuid>> PredictedStreamChange;
			const FLocalSyncControl::FPredictedObjectRemoval PredictedMuteChange;
			
			FConcertReplication_ChangeSyncControl RetainedSyncControl;
			FChangeStreamPredictedChange StreamChangeToRevert;
			TMap<FSoftObjectPath, TArray<FGuid>> AuthorityChangeToRevert;

			FPutStateChange(
				const TConstArrayView<FConcertReplicationStream>& RegisteredStreams,
				FLocalSyncControl::FPredictedObjectRemoval PredictedMuteChange,
				const FLocalSyncControl& SyncControl,
				const FGuid& ClientId,
				const FConcertReplication_PutState_Request& Request
				)
				: StreamChange(BuildDeltaStreamChange(ClientId, RegisteredStreams, Request))
				, AuthorityChange(BuildDeltaAuthorityChange(ClientId, Request))
				, PredictedMuteChange(MoveTemp(PredictedMuteChange))
				, RetainedSyncControl(BuildRetainedSyncControl(ClientId, SyncControl, Request))
			{}
		};
	}

	TFuture<FConcertReplication_PutState_Response> FReplicationManagerState_Connected::PutClientState(FConcertReplication_PutState_Request Request)
	{
		using namespace Private;
		if (!EnumHasAnyFlags(SessionFlags, EConcertSyncSessionFlags::ShouldEnableRemoteEditing))
		{
			return MakeFulfilledPromise<FConcertReplication_PutState_Response>(FConcertReplication_PutState_Response{ EConcertReplicationPutStateResponseCode::FeatureDisabled }).GetFuture();
		}

		// If the request changes the local state, we follow the approaches from like ChangeStreams, ChangeAuthority, and ChangeMuteState by predicatively
		// updating our local state if changing the local client as well... [1]
		const FGuid& ClientId = LiveSession->GetSessionClientEndpointId();
		FLocalSyncControl::FPredictedObjectRemoval PredictedMuteChange = SyncControl.PredictAndApplyMuteChanges(Request.MuteChange);
		FPutStateChange Change(RegisteredStreams, MoveTemp(PredictedMuteChange), SyncControl, ClientId, Request);
		if (Change.StreamChange)
		{
			Change.StreamChangeToRevert = PredictAndApplyStreamChangeRemovedObjects(*Change.StreamChange);
		}
		if (Change.AuthorityChange)
		{
			Change.AuthorityChangeToRevert = ApplyAuthorityChangeRemovedObjects(*Change.AuthorityChange);
		}
		
		LogNetworkMessage(CVarLogChangeClientsRequestsAndResponsesOnClient, Request);
		return LiveSession->SendCustomRequest<FConcertReplication_PutState_Request, FConcertReplication_PutState_Response>(Request, LiveSession->GetSessionServerEndpointId())
			.Next([WeakThis = TWeakPtr<FReplicationManagerState_Connected>(SharedThis(this)), Change = MoveTemp(Change)](FConcertReplication_PutState_Response&& Response) mutable
			{
				LogNetworkMessage(CVarLogChangeClientsRequestsAndResponsesOnClient, Response);
				const TSharedPtr<FReplicationManagerState_Connected> ThisPin = WeakThis.Pin();
				if (!ThisPin)
				{
					return Response;
				}

				const bool bIsSuccess = Response.IsSuccess();
				if (bIsSuccess)
				{
					// We treat PutState as a separate stream operation followed by an authority operation.
					// Applying the stream bit removes sync control... [2]
					if (Change.StreamChange)
					{
						ThisPin->FinalizePredictedStreamChange(*Change.StreamChange);
					}

					// [2]... but we need to add back sync control for those objects that we retained sync control over.
					// The server tells us which objects have changed sync control; those unchanged we must retain, so add them back now.
					RemoveEntriesThatShouldNotBeRetained(Response.SyncControl, ThisPin->RegisteredStreams, Change.RetainedSyncControl);
					ThisPin->SyncControl.ProcessSyncControlChange(Change.RetainedSyncControl);
					
					if (Change.AuthorityChange)
					{
						ThisPin->FinalizePredictedAuthorityChange(*Change.AuthorityChange, {}, Response.SyncControl);
					}
				}
				else
				{
					// [1]... and we may have to revert predictive changes
					if (Change.StreamChange)
					{
						ThisPin->RevertPredictedStreamChangeRemovedObjects(Change.StreamChangeToRevert);
					}
					if (Change.AuthorityChange)
					{
						ThisPin->RevertAuthorityChangeReleasedObjects(Change.AuthorityChangeToRevert);
					}
					
					ThisPin->SyncControl.ApplyOrRevertMuteResponse(Change.PredictedMuteChange, { .ErrorCode = EConcertReplicationMuteErrorCode::Rejected });
				}
				
				const EConcertReplicationMuteErrorCode DummyMuteResponseCode = bIsSuccess ? EConcertReplicationMuteErrorCode::Accepted : EConcertReplicationMuteErrorCode::Rejected;
				ThisPin->SyncControl.ApplyOrRevertMuteResponse(Change.PredictedMuteChange, { .ErrorCode = DummyMuteResponseCode });
				
				return Response;
			});
	}

	void FReplicationManagerState_Connected::HandleChangeClientEvent(const FConcertSessionContext& Context, const FConcertReplication_ChangeClientEvent& Event)
	{
		LogNetworkMessage(CVarLogChangeClientEventsOnClient, Event);

		// If we receive this from a different endpoint, it's probably from malicious user or someone scripting around with the API, either way warn about it.
		if (!ensure(Context.SourceEndpointId == LiveSession->GetSessionServerEndpointId()))
		{
			UE_LOGF(LogConcert, Warning, "The FConcertReplication_ChangeClientEvent is only supposed to be sent by the server.")
			return;
		}

		const FRemoteEditEvent EditEvent { Event.Reason, Event.ChangeData };
		OnPreRemoteEditAppliedDelegate.Broadcast(EditEvent);
		{
			const FConcertReplication_ChangeStream_Request& StreamChange = Event.ChangeData.StreamChange;
			PredictAndApplyStreamChangeRemovedObjects(StreamChange);
			FinalizePredictedStreamChange(StreamChange);

			const FConcertReplication_ChangeAuthority_Request& AuthorityChange = Event.ChangeData.AuthorityChange;
			ApplyAuthorityChangeRemovedObjects(AuthorityChange);
			FinalizePredictedAuthorityChange(AuthorityChange, {}, Event.ChangeData.SyncControlChange);
		}
		OnPostRemoteEditAppliedDelegate.Broadcast(EditEvent);
	}
}

