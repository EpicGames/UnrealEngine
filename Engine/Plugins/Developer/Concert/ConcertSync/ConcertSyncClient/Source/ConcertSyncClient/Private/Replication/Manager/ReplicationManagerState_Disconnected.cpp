// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationManagerState_Disconnected.h"

#include "ConcertLogGlobal.h"
#include "ReplicationManagerState_Handshaking.h"

#include "HAL/IConsoleManager.h"

namespace UE::ConcertSyncClient::Replication
{
	TAutoConsoleVariable<bool> CVarSimulateJoinTimeouts(
		TEXT("Concert.Replication.SimulateJoinTimeouts"),
		false,
		TEXT("Whether the client should pretend that join requests timed out instead of sending to the server.")
		);
	
	FReplicationManagerState_Disconnected::FReplicationManagerState_Disconnected(
		TSharedRef<IConcertClientSession> LiveSession,
		IConcertClientReplicationBridge& ReplicationBridge,
		FReplicationManager& Owner,
		EConcertSyncSessionFlags SessionFlags
		)
		: FReplicationManagerState(Owner)
		, LiveSession(MoveTemp(LiveSession))
		, ReplicationBridge(ReplicationBridge)
		, SessionFlags(SessionFlags)
	{}

	TFuture<FJoinReplicatedSessionResult> FReplicationManagerState_Disconnected::JoinReplicationSession(FJoinReplicatedSessionArgs Args)
	{
		if (CVarSimulateJoinTimeouts.GetValueOnGameThread())
		{
			return MakeFulfilledPromise<FJoinReplicatedSessionResult>(FJoinReplicatedSessionResult{ EJoinReplicationErrorCode::NetworkError }).GetFuture();
		}
		
		TPromise<FJoinReplicatedSessionResult> JoinPromise;
		TFuture<FJoinReplicatedSessionResult> JoinFuture = JoinPromise.GetFuture();
		ChangeState(MakeShared<FReplicationManagerState_Handshaking>(MoveTemp(Args), MoveTemp(JoinPromise), LiveSession, ReplicationBridge, GetOwner(), SessionFlags));
		return JoinFuture;
	}

	void FReplicationManagerState_Disconnected::LeaveReplicationSession()
	{
		UE_LOGF(LogConcert, Warning, "LeaveReplicationSession: Already disconnected.");
	}
}
