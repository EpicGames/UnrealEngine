// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertSyncSessionFlags.h"
#include "ReplicationManagerState.h"
#include "Replication/Processing/ClientReplicationDataCollector.h"

class IConcertClientSession;

namespace UE::ConcertSyncClient::Replication
{
	/** Initial state: waiting for a call to JoinReplicationSession to join replication session. */
	class FReplicationManagerState_Disconnected : public FReplicationManagerState
	{
	public:

		FReplicationManagerState_Disconnected(
			TSharedRef<IConcertClientSession> LiveSession,
			IConcertClientReplicationBridge& ReplicationBridge UE_LIFETIMEBOUND,
			FReplicationManager& Owner UE_LIFETIMEBOUND,
			EConcertSyncSessionFlags SessionFlags
			);

		//~ Begin IConcertClientReplicationManager Interface
		virtual TFuture<FJoinReplicatedSessionResult> JoinReplicationSession(FJoinReplicatedSessionArgs Args) override;
		virtual void LeaveReplicationSession() override;
		virtual bool CanJoin() override { return true; }
		virtual bool IsConnectedToReplicationSession() override { return false; }
		//~ End IConcertClientReplicationManager Interface

	private:
		
		/** Passed to FReplicationManagerState_Handshaking */
		TSharedRef<IConcertClientSession> LiveSession;
		/** Passed to FReplicationManagerState_Handshaking */
		IConcertClientReplicationBridge& ReplicationBridge;
		/** Passed to FReplicationManagerState_Handshaking */
		const EConcertSyncSessionFlags SessionFlags;
	};
}