// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationManagerState_Handshaking.h"

#include "IConcertSession.h"
#include "ReplicationManagerState_Connected.h"
#include "ReplicationManagerState_Disconnected.h"
#include "Replication/Messages/Handshake.h"

#include "Algo/Transform.h"

namespace UE::ConcertSyncClient::Replication
{
	FReplicationManagerState_Handshaking::FReplicationManagerState_Handshaking(
		FJoinReplicatedSessionArgs RequestArgs,
		TPromise<FJoinReplicatedSessionResult> JoinSessionPromise,
		TSharedRef<IConcertClientSession> LiveSession,
		IConcertClientReplicationBridge& ReplicationBridge,
		FReplicationManager& Owner,
		EConcertSyncSessionFlags SessionFlags
		)
		: FReplicationManagerState(Owner)
		, RequestArgs(MoveTemp(RequestArgs))
		, JoinSessionPromise(MoveTemp(JoinSessionPromise))
		, LiveSession(MoveTemp(LiveSession))
		, ReplicationBridge(ReplicationBridge)
		, SessionFlags(SessionFlags)
	{}

	FReplicationManagerState_Handshaking::~FReplicationManagerState_Handshaking()
	{
		if (!bFulfilledPromise)
		{
			JoinSessionPromise.EmplaceValue(FJoinReplicatedSessionResult{ EJoinReplicationErrorCode::Cancelled });
		}
	}

	TFuture<FJoinReplicatedSessionResult> FReplicationManagerState_Handshaking::JoinReplicationSession(FJoinReplicatedSessionArgs Args)
	{
		return MakeFulfilledPromise<FJoinReplicatedSessionResult>(FJoinReplicatedSessionResult{ EJoinReplicationErrorCode::AlreadyInProgress }).GetFuture();
	}

	void FReplicationManagerState_Handshaking::LeaveReplicationSession()
	{
		// If the server acknowledges us (later or the response is on route but not yet processed), immediately leave the session.
		LiveSession->SendCustomEvent(FConcertReplication_LeaveEvent{}, LiveSession->GetSessionServerEndpointId(), EConcertMessageFlags::ReliableOrdered);
		ReturnToDisconnectedState();
	}

	void FReplicationManagerState_Handshaking::OnEnterState()
	{
		const FConcertReplication_Join_Request Request{ RequestArgs.Streams };
		LiveSession->SendCustomRequest<FConcertReplication_Join_Request, FConcertReplication_Join_Response>(Request, LiveSession->GetSessionServerEndpointId())
			.Next([WeakThis = TWeakPtr<FReplicationManagerState_Handshaking>(SharedThis(this))](const FConcertReplication_Join_Response& Response)
			{
				// If we're still in the same state, this is valid. If we've left the state, this is invalid.
				const TSharedPtr<FReplicationManagerState_Handshaking> This = WeakThis.Pin();
				const bool bHasLeftState = !This.IsValid();
				if (bHasLeftState)
				{
					return;
				}

				if (Response.JoinErrorCode == EJoinReplicationErrorCode::Success)
				{
					This->ChangeState(
						MakeShared<FReplicationManagerState_Connected>(
							This->LiveSession, This->ReplicationBridge, This->GetOwner(), This->SessionFlags, MoveTemp(This->RequestArgs.Streams), Response.SyncControl
							)
						);
				}
				else
				{
					This->ReturnToDisconnectedState();
				}

				This->bFulfilledPromise = true;
				This->JoinSessionPromise.EmplaceValue(Response.JoinErrorCode, Response.DetailedErrorMessage);
			});
	}

	void FReplicationManagerState_Handshaking::ReturnToDisconnectedState()
	{
		ChangeState(
			MakeShared<FReplicationManagerState_Disconnected>(LiveSession, ReplicationBridge, GetOwner(), SessionFlags)
			);
	}
}


