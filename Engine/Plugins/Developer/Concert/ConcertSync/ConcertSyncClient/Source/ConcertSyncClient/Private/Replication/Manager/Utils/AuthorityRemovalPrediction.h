// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Messages/ChangeAuthority.h"
#include "Replication/Processing/ClientReplicationDataCollector.h"

#include "Algo/AnyOf.h"
#include "Algo/RemoveIf.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Misc/Guid.h"
#include "UObject/SoftObjectPath.h"

#include <type_traits>

namespace UE::ConcertSyncClient::Replication
{
	class FClientReplicationDataCollector;
	
	/** Removes ObjectsToRemove and returns the authority that was actually removed, possibly to be reverted later. */
	template<typename TStreamArrayType, typename TProjectStreamArray> requires std::is_invocable_r_v<const TArray<FGuid>&, TProjectStreamArray, const TStreamArrayType&>
	TMap<FSoftObjectPath, TArray<FGuid>> RemoveObjectsAndTrackRemovedAuthorityGeneric(
		FClientReplicationDataCollector& ReplicationDataSource,
		const TMap<FSoftObjectPath, TStreamArrayType>& ObjectsToRemove,
		TProjectStreamArray&& Project
		);

	/** @return Whether RemoveObjectsAndTrackRemovedAuthority would make any changes. This is useful for avoiding broadcasting change events. */
	template<typename TStreamArrayType, typename TProjectStreamArray> requires std::is_invocable_r_v<const TArray<FGuid>&, TProjectStreamArray, const TStreamArrayType&>
	bool HasAuthorityChangesGeneric(
		FClientReplicationDataCollector& ReplicationDataSource,
		const TMap<FSoftObjectPath, TStreamArrayType>& ObjectsToRemove,
		TProjectStreamArray&& Project
	);
	
	// Inline version for FConcertStreamArray & TArray<FGuid>
	TMap<FSoftObjectPath, TArray<FGuid>> RemoveObjectsAndTrackRemovedAuthority(
		FClientReplicationDataCollector& ReplicationDataSource, const TMap<FSoftObjectPath, FConcertStreamArray>& ObjectsToRemove
		);
	TMap<FSoftObjectPath, TArray<FGuid>> RemoveObjectsAndTrackRemovedAuthority(
		FClientReplicationDataCollector& ReplicationDataSource, const TMap<FSoftObjectPath, TArray<FGuid>>& ObjectsToRemove
		);
	bool HasAuthorityChanges(FClientReplicationDataCollector& ReplicationDataSource, const TMap<FSoftObjectPath, TArray<FGuid>>& ObjectsToRemove);
	bool HasAuthorityChanges(FClientReplicationDataCollector& ReplicationDataSource, const TMap<FSoftObjectPath, FConcertStreamArray>& ObjectsToRemove);
}

namespace UE::ConcertSyncClient::Replication
{
	/** Removes ObjectsToRemove and returns the authority that was actually removed, possibly to be reverted later. */
	template<typename TStreamArrayType, typename TProjectStreamArray> requires std::is_invocable_r_v<const TArray<FGuid>&, TProjectStreamArray, const TStreamArrayType&>
	TMap<FSoftObjectPath, TArray<FGuid>> RemoveObjectsAndTrackRemovedAuthorityGeneric(
		FClientReplicationDataCollector& ReplicationDataSource,
		const TMap<FSoftObjectPath, TStreamArrayType>& ObjectsToRemove,
		TProjectStreamArray&& Project
		)
	{
		TMap<FSoftObjectPath, TArray<FGuid>> RevertedContent;
		for (const TPair<FSoftObjectPath, TStreamArrayType>& ReleaseAuthority : ObjectsToRemove)
		{
			const FSoftObjectPath& ObjectPath = ReleaseAuthority.Key;
			const TArray<FGuid>& StreamsToRemove = Project(ReleaseAuthority.Value);

			// Remember which streams we actually removed, so we can revert it again.
			TArray<FGuid> StreamsActuallyRemoved = ReplicationDataSource.GetStreamsOwningObject(ObjectPath);
			StreamsActuallyRemoved.SetNum(Algo::RemoveIf(StreamsActuallyRemoved, [&StreamsToRemove](const FGuid& StreamId)
			{
				return !StreamsToRemove.Contains(StreamId);
			}));
			RevertedContent.Add(ObjectPath, StreamsActuallyRemoved);
			
			ReplicationDataSource.RemoveReplicatedObjectStreams(ObjectPath, StreamsToRemove);
		}
		return RevertedContent;
	}

	/** @return Whether RemoveObjectsAndTrackRemovedAuthority would make any changes. This is useful for avoiding broadcasting change events. */
	template<typename TStreamArrayType, typename TProjectStreamArray> requires std::is_invocable_r_v<const TArray<FGuid>&, TProjectStreamArray, const TStreamArrayType&>
	bool HasAuthorityChangesGeneric(
		FClientReplicationDataCollector& ReplicationDataSource,
		const TMap<FSoftObjectPath, TStreamArrayType>& ObjectsToRemove,
		TProjectStreamArray&& Project
	)
	{
		return Algo::AnyOf(ObjectsToRemove, [&ReplicationDataSource, &Project](const TPair<FSoftObjectPath, TStreamArrayType>& Pair)
		{
			const FSoftObjectPath& ObjectPath = Pair.Key;
			return Algo::AnyOf(Project(Pair.Value), [&ObjectPath, &ReplicationDataSource](const FGuid& StreamId)
			{
				return ReplicationDataSource.OwnsObjectInStream(ObjectPath, StreamId);
			});
		});
	}

	inline TMap<FSoftObjectPath, TArray<FGuid>> RemoveObjectsAndTrackRemovedAuthority(
		FClientReplicationDataCollector& ReplicationDataSource,
		const TMap<FSoftObjectPath, FConcertStreamArray>& ObjectsToRemove
		)
	{
		return RemoveObjectsAndTrackRemovedAuthorityGeneric<FConcertStreamArray>(
			ReplicationDataSource, ObjectsToRemove,
			[](const FConcertStreamArray& StreamArray)-> const TArray<FGuid>& { return StreamArray.StreamIds; }
			); 
	}

	inline TMap<FSoftObjectPath, TArray<FGuid>> RemoveObjectsAndTrackRemovedAuthority(
		FClientReplicationDataCollector& ReplicationDataSource,
		const TMap<FSoftObjectPath, TArray<FGuid>>& ObjectsToRemove
		)
	{
		return RemoveObjectsAndTrackRemovedAuthorityGeneric<TArray<FGuid>>(
			ReplicationDataSource, ObjectsToRemove,
			[](const TArray<FGuid>& StreamArray)-> const TArray<FGuid>& { return StreamArray; }
			); 
	}

	inline bool HasAuthorityChanges(FClientReplicationDataCollector& ReplicationDataSource, const TMap<FSoftObjectPath, TArray<FGuid>>& ObjectsToRemove)
	{
		return HasAuthorityChangesGeneric<TArray<FGuid>>(ReplicationDataSource, ObjectsToRemove,
			[](const TArray<FGuid>& StreamArray)-> const TArray<FGuid>& { return StreamArray; }
			);
	}

	inline bool HasAuthorityChanges(FClientReplicationDataCollector& ReplicationDataSource, const TMap<FSoftObjectPath, FConcertStreamArray>& ObjectsToRemove)
	{
		return HasAuthorityChangesGeneric<FConcertStreamArray>(ReplicationDataSource, ObjectsToRemove,
			[](const FConcertStreamArray& StreamArray)-> const TArray<FGuid>& { return StreamArray.StreamIds; }
			);
	}
}