// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentComparisionUtils.h"

#include "Replication/Data/ClientQueriedInfo.h"
#include "Replication/Data/ReplicationStream.h"
#include "Replication/Processing/ClientReplicationDataCollector.h"

namespace UE::ConcertSyncClient::Replication
{
	bool AreStreamsEquivalent(TConstArrayView<FConcertBaseStreamInfo> NewStreams, TConstArrayView<FConcertReplicationStream> RegisteredStreams)
	{
		if (NewStreams.Num() != RegisteredStreams.Num())
		{
			return false;
		}
		
		for (int32 i = 0; i < NewStreams.Num(); i++)
		{
			const FConcertBaseStreamInfo& NewStream = NewStreams[i];
			const FGuid& StreamId = NewStream.Identifier;
			const FConcertReplicationStream* StreamInfo = RegisteredStreams.FindByPredicate([&StreamId](const FConcertReplicationStream& Stream)
			{
				return Stream.BaseDescription.Identifier == StreamId;
			});
			if (!StreamInfo)
			{
				return false;
			}

			if (NewStream != StreamInfo->BaseDescription)
			{
				return false;
			}
		}
		
		return true;
	}

	bool IsAuthorityEquivalent(TConstArrayView<FConcertAuthorityClientInfo> NewAuthority, const FClientReplicationDataCollector& Replicator)
	{
		bool bIsEquivalent = true;
		
		// NewAuthority contains all of Replicator?
		for (const FConcertAuthorityClientInfo& Authority : NewAuthority)
		{
			for (const FSoftObjectPath& ObjectPath : Authority.AuthoredObjects)
			{
				if (!Replicator.OwnsObjectInStream(ObjectPath, Authority.StreamId))
				{
					return false;
				}
			}
		}
		
		// Replicator contains all of NewAuthority?
		Replicator.ForEachOwnedObjectAndStream([&NewAuthority, &bIsEquivalent](const FSoftObjectPath& ReplicatedObject, const FGuid& StreamId)
		{
			const FConcertAuthorityClientInfo* ClientInfo = NewAuthority.FindByPredicate([&StreamId](const FConcertAuthorityClientInfo& Info)
			{
				return Info.StreamId == StreamId;
			});
			bIsEquivalent &= ClientInfo && ClientInfo->AuthoredObjects.Contains(ReplicatedObject);
			return bIsEquivalent ? EBreakBehavior::Continue : EBreakBehavior::Break;
		});
		
		return bIsEquivalent;
	}
}
