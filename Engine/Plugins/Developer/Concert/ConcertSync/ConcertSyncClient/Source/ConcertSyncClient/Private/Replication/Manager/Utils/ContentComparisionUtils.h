// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"

struct FConcertAuthorityClientInfo;
struct FConcertBaseStreamInfo;
struct FConcertReplicationStream;

namespace UE::ConcertSyncClient::Replication
{
	class FClientReplicationDataCollector;
	
	/** @return Whether NewStreams would change nothing about RegisteredStreams if replaced. */
	bool AreStreamsEquivalent(TConstArrayView<FConcertBaseStreamInfo> NewStreams, TConstArrayView<FConcertReplicationStream> RegisteredStreams);

	/** @return Whether assigning NewAuthority to Replicator would change any authority. */
	bool IsAuthorityEquivalent(TConstArrayView<FConcertAuthorityClientInfo> NewAuthority, const FClientReplicationDataCollector& Replicator);
}
