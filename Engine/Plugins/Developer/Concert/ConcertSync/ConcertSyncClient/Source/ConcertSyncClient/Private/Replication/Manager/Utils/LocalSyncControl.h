// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IConcertSession.h"
#include "Replication/Messages/SyncControl.h"

#include "HAL/Platform.h"
#include "Replication/SyncControlState.h"
#include "Templates/UnrealTemplate.h"

namespace UE::ConcertSyncClient::Replication
{
	/**
	 * Receives network messages that explicitly and implicitly change sync control.
	 * Those network messages are fed into FSyncControlState, which parses the messages.
	 * 
	 * Sync control is the set of objects this client is allowed to replicate.
	 */
	class FLocalSyncControl
		: public FNoncopyable
		, ConcertSyncCore::Replication::FSyncControlState
	{
		using Detail = FSyncControlState;
	public:

		explicit FLocalSyncControl(IConcertSession& InSession UE_LIFETIMEBOUND)
			: Session(InSession)
		{
			Session.RegisterCustomEventHandler<FConcertReplication_ChangeSyncControl>(this, &FLocalSyncControl::OnChangeSyncControl);
		}

		~FLocalSyncControl()
		{
			Session.UnregisterCustomEventHandler<FConcertReplication_ChangeSyncControl>(this);
		}

		using FSyncControlState::IsObjectAllowed;
		using FSyncControlState::Num;
		using FSyncControlState::EnumerateAllowedObjects;
		using FSyncControlState::EnumerateChanges;
		using FPredictedObjectRemoval = FSyncControlState::FPredictedObjectRemoval;
		// No using for FSyncControlState::Aggregate because we want to limit usage to below ProcessXChange functions

		/** Stores the given change in this data structure. */
		void ProcessSyncControlChange(const FConcertReplication_ChangeSyncControl& Event)
		{
			OnPreSyncControlChangedDelegate.Broadcast();
			AppendChanges(Event);
			OnPostSyncControlChangedDelegate.Broadcast();
		}

		/** Applies implicit and explicit changes to the client's sync control resulting from a completed authority change request and response. */
		void ProcessAuthorityChange(const FConcertReplication_ChangeAuthority_Request& Request, const FConcertReplication_ChangeSyncControl& Response)
		{
			OnPreSyncControlChangedDelegate.Broadcast();
			AppendAuthorityChange(Request, Response);
			OnPostSyncControlChangedDelegate.Broadcast();
		}

		/**
		 * Applies implicit changes to the client's sync control resulting from losing authority from objects removed from the stream.
		 * You must validate that the request has also been accepted by the server!
		 */
		void ProcessStreamChange(const FConcertReplication_ChangeStream_Request& Request)
		{
			OnPreSyncControlChangedDelegate.Broadcast();
			AppendStreamChange(Request);
			OnPostSyncControlChangedDelegate.Broadcast();
		}

		/**
		 * Applies the implicit changes made by the request assuming the request will be accepted.
		 * @return The removed objects to be passed to ApplyOrRevertMuteResponse.
		 */
		FPredictedObjectRemoval PredictAndApplyMuteChanges(const FConcertReplication_ChangeMuteState_Request& Request)
		{
			bool bMadeChange = false;
			FPredictedObjectRemoval Predicition = Detail::PredictAndApplyMuteChanges(Request, [this, &bMadeChange](const FConcertObjectInStreamID&)
			{
				if (!bMadeChange)
				{
					OnPreSyncControlChangedDelegate.Broadcast();
				}
				bMadeChange = true;
			});
			if (bMadeChange)
			{
				OnPostSyncControlChangedDelegate.Broadcast();
			}

			return Predicition;
		}

		/** Either reverts previous changes made if the request was rejected, or applies the sync control returned by the server otherwise. */
		void ApplyOrRevertMuteResponse(const FPredictedObjectRemoval& RemovedByRequest, const FConcertReplication_ChangeMuteState_Response& Response)
		{
			bool bMadeChange = false;
			Detail::ApplyOrRevertMuteResponse(RemovedByRequest, Response, [this, &bMadeChange](const FConcertObjectInStreamID&)
			{
				if (!bMadeChange)
				{
					OnPreSyncControlChangedDelegate.Broadcast();
				}
				bMadeChange = true;
			});
			if (bMadeChange)
			{
				OnPostSyncControlChangedDelegate.Broadcast();
			}
		}

		
		/**
		 * Applies the implicit changes made by the request assuming the request will be accepted.
		 * @return The removed objects to be passed to ApplyOrRevertRestoreContentResponse.
		 */
		FPredictedObjectRemoval PredictAndApplyRestoreContentChanges(const FConcertReplication_RestoreContent_Request& Request)
		{
			bool bMadeChange = false;
			FPredictedObjectRemoval Predicition = Detail::PredictAndApplyRestoreContentChanges(Request, [this, &bMadeChange](const FConcertObjectInStreamID&)
			{
				if (!bMadeChange)
				{
					OnPreSyncControlChangedDelegate.Broadcast();
				}
				bMadeChange = true;
			});
			if (bMadeChange)
			{
				OnPostSyncControlChangedDelegate.Broadcast();
			}

			return Predicition;
		}

		/** Either reverts previous changes made if the request was rejected, or applies the sync control returned by the server otherwise. */
		void ApplyOrRevertRestoreContentResponse(const FPredictedObjectRemoval& RemovedByRequest, const FConcertReplication_RestoreContent_Response& Response)
		{
			bool bMadeChange = false;
			const auto OnChange = [this, &bMadeChange](const FConcertObjectInStreamID&)
			{
				if (!bMadeChange)
				{
					OnPreSyncControlChangedDelegate.Broadcast();
				}
				bMadeChange = true;
			};
			Detail::ApplyOrRevertRestoreContentResponse(RemovedByRequest, Response, OnChange, OnChange);
			
			if (bMadeChange)
			{
				OnPostSyncControlChangedDelegate.Broadcast();
			}
		}

		void PredictAndApplyPutStateChanges(const FConcertReplication_RestoreContent_Request& Request);
		
		
		DECLARE_MULTICAST_DELEGATE(FSyncControlChanged);
		FSyncControlChanged& OnPreSyncControlChanged() { return OnPreSyncControlChangedDelegate; }
		FSyncControlChanged& OnPostSyncControlChanged() { return OnPostSyncControlChangedDelegate; }

	private:

		/** The session to receive sync control changes on. */
		IConcertSession& Session;
		
		FSyncControlChanged OnPreSyncControlChangedDelegate;
		FSyncControlChanged OnPostSyncControlChangedDelegate;

		void OnChangeSyncControl(const FConcertSessionContext&, const FConcertReplication_ChangeSyncControl& Event) { ProcessSyncControlChange(Event); }
	};
}

