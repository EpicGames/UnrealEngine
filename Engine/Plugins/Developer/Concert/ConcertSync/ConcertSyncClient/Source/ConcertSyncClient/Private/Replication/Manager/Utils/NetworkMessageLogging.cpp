// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkMessageLogging.h"

namespace UE::ConcertSyncClient::Replication
{
	TAutoConsoleVariable<bool> CVarSimulateAuthorityTimeouts(
		TEXT("Concert.Replication.SimulateAuthorityTimeouts"),
		false,
		TEXT("Whether the client should pretend that authority requests timed out instead of sending to the server.")
		);
	TAutoConsoleVariable<bool> CVarSimulateQueryTimeouts(
		TEXT("Concert.Replication.SimulateQueryTimeouts"),
		false,
		TEXT("Whether the client should pretend that query requests timed out instead of sending to the server.")
		);
	TAutoConsoleVariable<bool> CVarSimulateStreamChangeTimeouts(
		TEXT("Concert.Replication.SimulateStreamChangeTimeouts"),
		false,
		TEXT("Whether the client should pretend that stream change requests timed out instead of sending to the server.")
		);

	TAutoConsoleVariable<bool> CVarSimulateAuthorityRejection(
		TEXT("Concert.Replication.SimulateAuthorityRejection"),
		false,
		TEXT("Whether the client should pretend that authority change requests were rejected.")
		);
	TAutoConsoleVariable<bool> CVarSimulateMuteRequestRejection(
		TEXT("Concert.Replication.SimulateMuteRejection"),
		false,
		TEXT("Whether the client should pretend that mute change requests were rejected.")
		);

	TAutoConsoleVariable<bool> CVarLogStreamRequestsAndResponsesOnClient(
		TEXT("Concert.Replication.LogStreamRequestsAndResponsesOnClient"),
		false,
		TEXT("Whether to log changes to streams.")
		);
	TAutoConsoleVariable<bool> CVarLogAuthorityRequestsAndResponsesOnClient(
		TEXT("Concert.Replication.LogAuthorityRequestsAndResponsesOnClient"),
		false,
		TEXT("Whether to log changes to authority.")
		);
	TAutoConsoleVariable<bool> CVarLogMuteRequestsAndResponsesOnClient(
		TEXT("Concert.Replication.LogMuteRequestsAndResponsesOnClient"),
		false,
		TEXT("Whether to log changes to the mute state.")
		);
	TAutoConsoleVariable<bool> CVarLogRestoreContentRequestsAndResponsesOnClient(
		TEXT("Concert.Replication.LogRestoreContentRequestsAndResponsesOnClient"),
		false,
		TEXT("Whether to log restore content requests and responses.")
		);
		
	TAutoConsoleVariable<bool> CVarLogChangeClientsRequestsAndResponsesOnClient(
	TEXT("Concert.Replication.LogChangeClientsRequestsAndResponsesOnClient"),
	false,
	TEXT("Whether to log requests and responses that change multiple clients in one go.")
	);
	TAutoConsoleVariable<bool> CVarLogChangeClientEventsOnClient(
		TEXT("Concert.Replication.LogChangeClientEventsOnClient"),
		false,
		TEXT("Whether to log messages from the server that notify us that the client's content has changed.")
		);
}