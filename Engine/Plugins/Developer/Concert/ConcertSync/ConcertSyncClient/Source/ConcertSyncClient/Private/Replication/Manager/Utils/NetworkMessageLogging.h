// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertLogGlobal.h"

#include "Containers/UnrealString.h"
#include "HAL/IConsoleManager.h"
#include "JsonObjectConverter.h"

namespace UE::ConcertSyncClient::Replication
{
	extern TAutoConsoleVariable<bool> CVarSimulateAuthorityTimeouts;
	/** Whether the client should pretend that query requests timed out instead of sending to the server. */
	extern TAutoConsoleVariable<bool> CVarSimulateQueryTimeouts;
	/** Whether the client should pretend that stream change requests timed out instead of sending to the server. */
	extern TAutoConsoleVariable<bool> CVarSimulateStreamChangeTimeouts;

	/** Whether the client should pretend that authority change requests were rejected. */
	extern TAutoConsoleVariable<bool> CVarSimulateAuthorityRejection;
	/** Whether the client should pretend that mute change requests were rejected. */
	extern TAutoConsoleVariable<bool> CVarSimulateMuteRequestRejection;

	/** Whether to log changes to streams. */
	extern TAutoConsoleVariable<bool> CVarLogStreamRequestsAndResponsesOnClient;
	/** Whether to log changes to authority. */
	extern TAutoConsoleVariable<bool> CVarLogAuthorityRequestsAndResponsesOnClient;
	/** Whether to log changes to the mute state. */
	extern TAutoConsoleVariable<bool> CVarLogMuteRequestsAndResponsesOnClient;
	/** Whether to log restore content requests and responses. */
	extern TAutoConsoleVariable<bool> CVarLogRestoreContentRequestsAndResponsesOnClient;
	/** Whether to log requests and responses that change multiple clients in one go. */
	extern TAutoConsoleVariable<bool> CVarLogChangeClientsRequestsAndResponsesOnClient;
	/** Whether to log messages from the server that notify us that the client's content has changed. */
	extern TAutoConsoleVariable<bool> CVarLogChangeClientEventsOnClient;
	
	template<typename TMessage>
	static void LogNetworkMessage(const TAutoConsoleVariable<bool>& ShouldLog, const TMessage& Message)
	{
		if (ShouldLog.GetValueOnAnyThread())
		{
			FString JsonString;
			FJsonObjectConverter::UStructToJsonObjectString(TMessage::StaticStruct(), &Message, JsonString, 0, 0);
			UE_LOGF(LogConcert, Log, "%ls\n%ls", *TMessage::StaticStruct()->GetName(), *JsonString);
		}
	}
}
