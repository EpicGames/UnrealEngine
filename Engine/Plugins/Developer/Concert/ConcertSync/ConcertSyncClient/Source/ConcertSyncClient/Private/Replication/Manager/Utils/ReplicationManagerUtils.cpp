// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplicationManagerUtils.h"

#include "Replication/IConcertClientReplicationManager.h"

#include "Algo/Transform.h"
#include "Containers/Set.h"

namespace UE::ConcertSyncClient::Replication
{
	TFuture<FConcertReplication_ChangeAuthority_Response> RejectAll(FConcertReplication_ChangeAuthority_Request&& Args)
	{
		return MakeFulfilledPromise<FConcertReplication_ChangeAuthority_Response>(
			FConcertReplication_ChangeAuthority_Response{ EReplicationResponseErrorCode::Handled, MoveTemp(Args.TakeAuthority) }
			).GetFuture();
	}

	TMap<FSoftObjectPath, TArray<FGuid>> ComputeRemovedObjects(
		const TConstArrayView<FConcertReplicationStream> RegisteredStreams,
		const FConcertReplication_ChangeStream_Request& Request)
	{
		TMap<FSoftObjectPath, TArray<FGuid>> BundledRemovedObjects;
		for (const FConcertObjectInStreamID& RemovedObject : Request.ObjectsToRemove)
		{
			BundledRemovedObjects.FindOrAdd(RemovedObject.Object).AddUnique(RemovedObject.StreamId);
		}
		for (const FConcertReplicationStream& Stream : RegisteredStreams)
		{
			const FGuid& StreamId = Stream.BaseDescription.Identifier;
			if (!Request.StreamsToRemove.Contains(StreamId))
			{
				continue;
			}
			for (const TPair<FSoftObjectPath, FConcertReplicatedObjectInfo>& Pair : Stream.BaseDescription.ReplicationMap.ReplicatedObjects)
			{
				BundledRemovedObjects.FindOrAdd(Pair.Key).AddUnique(StreamId);
			}
		}
		return BundledRemovedObjects;
	}
}
