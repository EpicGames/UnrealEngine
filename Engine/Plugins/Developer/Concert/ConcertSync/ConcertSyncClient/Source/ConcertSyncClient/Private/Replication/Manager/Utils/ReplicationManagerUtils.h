// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"

struct FConcertReplicationStream;
struct FConcertReplication_ChangeStream_Request;
struct FConcertReplication_ChangeAuthority_Response;
struct FConcertReplication_ChangeAuthority_Request;

namespace UE::ConcertSyncClient::Replication
{
	/** Creates a fulfilled future which rejects all items in FConcertChangeAuthority_Request::TakeAuthority. */
	TFuture<FConcertReplication_ChangeAuthority_Response> RejectAll(FConcertReplication_ChangeAuthority_Request&& Args);

	/** Enumerates the objects that will be removed by Request. */
	TMap<FSoftObjectPath, TArray<FGuid>> ComputeRemovedObjects(
		const TConstArrayView<FConcertReplicationStream> RegisteredStreams,
		const FConcertReplication_ChangeStream_Request& Request
		);
};
