// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Misc/ClientPredictionUtils.h"

#include "IConcertClientWorkspace.h"
#include "Replication/Messages/ReplicationActivity.h"
#include "Replication/Misc/StreamAndAuthorityPredictionUtils.h"

namespace UE::ConcertSyncClient::Replication
{
	TOptional<int64> IncrementalBacktrackActivityHistoryForActivityThatSetsContent(
		const IConcertClientWorkspace& Workspace,
		const FConcertClientInfo& ClientInfo,
		TArray<FConcertBaseStreamInfo>& OutStreams,
		TArray<FConcertObjectInStreamID>& OutAuthority,
		int64 MaxToFetch,
		int64 MinActivityIdCutoff
		)
	{
		const int64 LastActivityId = Workspace.GetLastActivityId();
		if (MinActivityIdCutoff > LastActivityId)
		{
			return {};
		}
		
		MaxToFetch = FMath::Max(1, MaxToFetch);
		MinActivityIdCutoff = FMath::Max(1, MinActivityIdCutoff); // ActivityIds start at 1
		int64 LastFirstToFetch = LastActivityId + 1;

		bool bShouldStop = false;
		// We'll walk backwards in history until we find the last state that has affect the client.
		while (!bShouldStop) 
		{
			// From the back of the history take chunks of size MaxToFetch until we find what we're looking for
			const int64 NextFirstToFetch = FMath::Max(MinActivityIdCutoff, LastFirstToFetch - MaxToFetch);
			const int64 NumToFetch = FMath::Max(MinActivityIdCutoff, LastFirstToFetch - NextFirstToFetch);
			LastFirstToFetch = NextFirstToFetch;
			bShouldStop = NextFirstToFetch == MinActivityIdCutoff;
				
			TArray<FConcertSessionActivity> Activities;
			TMap<FGuid, FConcertClientInfo> Endpoints;
			Workspace.GetActivities(NextFirstToFetch, NumToFetch, Endpoints, Activities);

			const auto IsEquivalentClient = [&ClientInfo, &Endpoints](const FGuid& EndpointId)
			{
				return ConcertSyncCore::Replication::AreLogicallySameClients(ClientInfo, Endpoints[EndpointId]);
			};
			const auto GetReplicationEvent = [&Workspace](const int64 EventId, TFunctionRef<void(const FConcertSyncReplicationEvent&)> Callback)
			{
				FConcertSyncReplicationEvent Event;
				if (Workspace.FindReplicationEvent(EventId, Event))
				{
					Callback(Event);
				}
			};
			const TOptional<int64> UsedActivityId = ConcertSyncCore::Replication::BacktrackActivityHistoryForActivityThatSetsContent(
				Activities, IsEquivalentClient, GetReplicationEvent,  OutStreams, OutAuthority
				);
			if (UsedActivityId.IsSet())
			{
				return UsedActivityId;
			}
		}
		return {};
	}
}
