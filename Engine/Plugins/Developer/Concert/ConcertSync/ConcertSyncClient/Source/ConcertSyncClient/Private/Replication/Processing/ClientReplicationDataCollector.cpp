// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClientReplicationDataCollector.h"

#include "Replication/Data/ObjectIds.h"
#include "Replication/Data/ReplicationStream.h"
#include "Replication/Formats/IObjectReplicationFormat.h"
#include "Replication/IConcertClientReplicationBridge.h"
#include "Replication/Manager/Utils/LocalSyncControl.h"
#include "Replication/Processing/IReplicationDataItem.h"
#include "Replication/ReplicationPropertyFilter.h"

#include "Algo/RemoveIf.h"
#include "Misc/EBreakBehavior.h"

namespace UE::ConcertSyncClient::Replication
{
namespace DataCollector
{
class FDataCollectorItem : public ConcertSyncCore::IReplicationDataItem
{
	FObjectInfo& ObjectInfo;
	const FConcertReplicatedObjectId& ObjectToProcess;
	
	IConcertClientReplicationBridge& Bridge;
	const FLocalSyncControl& SyncControl;
	ConcertSyncCore::IObjectReplicationFormat& ReplicationFormat;
	
public:
	
	explicit FDataCollectorItem(
		FObjectInfo& ObjectInfo UE_LIFETIMEBOUND, 
		const FConcertReplicatedObjectId& ObjectToProcess UE_LIFETIMEBOUND, 
		IConcertClientReplicationBridge& Bridge UE_LIFETIMEBOUND,
		const FLocalSyncControl& SyncControl UE_LIFETIMEBOUND, 
		ConcertSyncCore::IObjectReplicationFormat& ReplicationFormat UE_LIFETIMEBOUND
		)
		: ObjectInfo(ObjectInfo)
		, ObjectToProcess(ObjectToProcess)
		, Bridge(Bridge)
		, SyncControl(SyncControl)
		, ReplicationFormat(ReplicationFormat)
	{}

	virtual void ExtractReplicationDataForObject(
		TFunctionRef<void(const FConcertSessionSerializedPayload& Payload)> ProcessCopyable,
		TFunctionRef<void(FConcertSessionSerializedPayload&& Payload)> ProcessMoveable
		) override
	{
		UObject* Object = Bridge.FindObjectIfAvailable(ObjectToProcess.Object);
		// Ask the bridge to resolve the object for us. The bridge has the object cached and handles the object getting renamed, etc.
		// This should resolve. If it does not: same logic as above - either this call is invalid or ConsumePendingObjects lied.
		if (!ensure(Object) || !ensure(SyncControl.IsObjectAllowed(ObjectToProcess)))
		{
			return;
		}

		ConcertSyncCore::FReplicationPropertyFilter Filter(ObjectInfo.SelectedProperties);
		TOptional<FConcertSessionSerializedPayload> Payload = ReplicationFormat.CreateReplicationEvent(
			*Object,
			[&Filter](const FArchiveSerializedPropertyChain* Chain, const FProperty& Property)
			{
				return Filter.ShouldSerializeProperty(Chain, Property);
			}
		);
		if (Payload)
		{
			ProcessMoveable(MoveTemp(*Payload));
			++ObjectInfo.ReplicationSequenceId;
		}
	}
};
}

	FClientReplicationDataCollector::FClientReplicationDataCollector(
		IConcertClientReplicationBridge& InReplicationBridge,
		ConcertSyncCore::IObjectReplicationFormat& InReplicationFormat,
		const FLocalSyncControl& SyncControl,
		FGetClientStreams InGetStreamsDelegate,
		const FGuid& InClientId
		)
		: Bridge(InReplicationBridge)
		, ReplicationFormat(InReplicationFormat)
		, SyncControl(SyncControl)
		, GetStreamsDelegate(MoveTemp(InGetStreamsDelegate))
		, ClientId(InClientId)
	{
		check(GetStreamsDelegate.IsBound());
		Bridge.OnObjectDiscovered().AddRaw(this, &FClientReplicationDataCollector::StartTrackingObject);
		Bridge.OnObjectHidden().AddRaw(this, &FClientReplicationDataCollector::StopTrackingObject);
	}

	FClientReplicationDataCollector::~FClientReplicationDataCollector()
	{
		Bridge.OnObjectDiscovered().RemoveAll(this);
		Bridge.OnObjectHidden().RemoveAll(this);

		for (const TPair<FSoftObjectPath, TArray<FObjectInfo>>& Pair : ObjectsToReplicate)
		{
			Bridge.PopTrackedObjects({ Pair.Key });
		}
	}
	
	void FClientReplicationDataCollector::AddReplicatedObjectStreams(const FSoftObjectPath& Object, TArrayView<const FGuid> AddedStreams)
	{
		if (AddedStreams.IsEmpty())
		{
			return;
		}
		
		TArray<FObjectInfo>& ReplicatedObjectInfo = ObjectsToReplicate.FindOrAdd(Object);
		const bool bIsNewReplicatedObject = ReplicatedObjectInfo.IsEmpty();
		
		const TArray<FConcertReplicationStream>& RegisteredStreams = *GetStreamsDelegate.Execute();
		ReplicatedObjectInfo.Reserve(ReplicatedObjectInfo.Num() + AddedStreams.Num());
		for (const FGuid& StreamId : AddedStreams)
		{
			const FConcertReplicationStream* Stream = RegisteredStreams.FindByPredicate([&StreamId](const FConcertReplicationStream& Stream)
			{
				return Stream.BaseDescription.Identifier == StreamId;
			});
			const FConcertReplicatedObjectInfo* ObjectInfo = Stream ? Stream->BaseDescription.ReplicationMap.ReplicatedObjects.Find(Object) : nullptr;
			if (ensureAlwaysMsgf(ObjectInfo, TEXT("Client's registered streams cache is out of sync")))
			{
				ReplicatedObjectInfo.Add({ StreamId, ObjectInfo->PropertySelection });
			}
		}
		
		// FClientReplicationDataCollector should not push Object more than once because each push increments an internal counter
		if (bIsNewReplicatedObject)
		{
			Bridge.PushTrackedObjects({ Object });
		}
	}

	void FClientReplicationDataCollector::RemoveReplicatedObjectStreams(const FSoftObjectPath& Object, TArrayView<const FGuid> RemovedStreams)
	{
		TArray<FObjectInfo>* ReplicatedObjectInfo = ObjectsToReplicate.Find(Object);
		if (!ReplicatedObjectInfo)
		{
			// This object is not being replicated
			return;
		}

		ReplicatedObjectInfo->SetNum(Algo::RemoveIf(*ReplicatedObjectInfo, [&RemovedStreams](const FObjectInfo& ObjectInfo)
		{
			return RemovedStreams.Contains(ObjectInfo.StreamId);
		}));

		if (ReplicatedObjectInfo->IsEmpty())
		{
			ObjectsToReplicate.Remove(Object);
			Bridge.PopTrackedObjects({ Object });
		}
	}

	void FClientReplicationDataCollector::OnObjectStreamModified(const FSoftObjectPath& Object, TArrayView<const FGuid> PutStreams)
	{
		TArray<FObjectInfo>* ReplicatedObjectInfo = ObjectsToReplicate.Find(Object);
		if (!ReplicatedObjectInfo)
		{
			// This object is not being replicated
			return;
		}

		const TArray<FConcertReplicationStream>& RegisteredStreams = *GetStreamsDelegate.Execute();
		for (const FGuid& StreamId : PutStreams)
		{
			const FConcertReplicationStream* Stream = RegisteredStreams.FindByPredicate([&StreamId](const FConcertReplicationStream& Stream)
			{
				return Stream.BaseDescription.Identifier == StreamId;
			});
			const FConcertReplicatedObjectInfo* ObjectInfo = Stream ? Stream->BaseDescription.ReplicationMap.ReplicatedObjects.Find(Object) : nullptr;
			if (!ensureAlwaysMsgf(ObjectInfo, TEXT("Client's registered streams cache is out of sync")))
			{
				continue;
			}

			// The PutObject request either just created ObjectInfo or updated it.
			FObjectInfo* ReplicatedObject = ReplicatedObjectInfo->FindByPredicate([&](const FObjectInfo& Existing)
			{
				return Existing.StreamId == StreamId;
			});
			if (ReplicatedObject)
			{
				// PutObject updated existing object in stream
				ReplicatedObject->SelectedProperties = ObjectInfo->PropertySelection;
			}
			else
			{
				// PutObject added object to stream
				ReplicatedObjectInfo->Add({ StreamId, ObjectInfo->PropertySelection });
			}
		}
	}

	void FClientReplicationDataCollector::ClearReplicatedObjects()
	{
		for (const TPair<FSoftObjectPath, TArray<FObjectInfo>>& Pair : ObjectsToReplicate)
		{
			Bridge.PopTrackedObjects({ Pair.Key });
		}
		ObjectsToReplicate.Reset();
	}

	void FClientReplicationDataCollector::AppendOwningStreamsForObject(
		const FSoftObjectPath& ObjectPath,
		TSet<FGuid>& Paths
		) const
	{
		if (const TArray<FObjectInfo>* ObjectInfo = ObjectsToReplicate.Find(ObjectPath))
		{
			Algo::Transform(*ObjectInfo, Paths, [](const FObjectInfo& ObjectInfo)
			{
				return ObjectInfo.StreamId;
			});
		}
	}

	bool FClientReplicationDataCollector::OwnsObjectInStream(const FSoftObjectPath& ObjectPath, const FGuid& StreamId) const
	{
		const TArray<FObjectInfo>* ObjectInfo = ObjectsToReplicate.Find(ObjectPath);
		return ObjectInfo && ObjectInfo->ContainsByPredicate([&StreamId](const FObjectInfo& ObjectInfo){ return ObjectInfo.StreamId == StreamId; });
	}

	bool FClientReplicationDataCollector::OwnsObjectInAnyStream(const FSoftObjectPath& ObjectPath) const
	{
		return ObjectsToReplicate.Contains(ObjectPath);
	}

	TArray<FGuid> FClientReplicationDataCollector::GetStreamsOwningObject(const FSoftObjectPath& ObjectPath) const
	{
		const TArray<FObjectInfo>* ObjectInfo = ObjectsToReplicate.Find(ObjectPath);
		if (!ObjectInfo)
		{
			return {};
		}

		TArray<FGuid> Result;
		Algo::Transform(*ObjectInfo, Result, [](const FObjectInfo& ObjectInfo){ return ObjectInfo.StreamId; });
		return Result;
	}

	void FClientReplicationDataCollector::ConsumePendingObjects(
		TFunctionRef<void(const ConcertSyncCore::FPendingObjectReplicationInfo& Info, ConcertSyncCore::IReplicationDataItem& Item)> InProcess
		)
	{
		for (TPair<FSoftObjectPath, TArray<FObjectInfo>>& Pair : ObjectsToReplicate)
		{
			for (FObjectInfo& ObjectInfo : Pair.Value)
			{
				const FConcertReplicatedObjectId ObjectId{
					FConcertObjectInStreamID{ ObjectInfo.StreamId, Pair.Key },
					ClientId
				};
				if (!SyncControl.IsObjectAllowed(ObjectId))
				{
					continue;
				}
					
				// The bridge has the object cached. We do not cache the object ourselves!
				// Funky Unreal flows can cause the object to be renamed out from under us and replaced by a different instance (just by using UObject::Rename()).
				// The bridge is aware of these flows and FindObjectIfAvailable will catch them.
				// If FindObjectIfAvailable fails to resolve, OnObjectHidden() is triggered if the object was previously visible.
				const UObject* Object = Bridge.FindObjectIfAvailable(Pair.Key);
					
				const FSoftObjectPath ObjectPath = Object;
				if (Object && ensureMsgf(ObjectPath == Object, TEXT("Sanity check: the bridge gave us an object with a different path!")))
				{
					ConcertSyncCore::FPendingObjectReplicationInfo Info
					{
						FConcertReplicatedObjectId {FConcertObjectInStreamID{ ObjectInfo.StreamId, Object },ClientId },
						ObjectInfo.ReplicationSequenceId
					};

					DataCollector::FDataCollectorItem Item(ObjectInfo, Info.ObjectId, Bridge, SyncControl, ReplicationFormat);
					InProcess(Info, Item);
				}
			}
		}
	}

	void FClientReplicationDataCollector::StartTrackingObject(UObject& Object)
	{
		if (TArray<FObjectInfo>* ObjectInfos = ObjectsToReplicate.Find(&Object))
		{
			++NumTrackedObjects;
		}
	}

	void FClientReplicationDataCollector::StopTrackingObject(const FSoftObjectPath& ObjectPath)
	{
		if (TArray<FObjectInfo>* ObjectInfos = ObjectsToReplicate.Find(ObjectPath))
		{
			--NumTrackedObjects;
		}
	}
}
