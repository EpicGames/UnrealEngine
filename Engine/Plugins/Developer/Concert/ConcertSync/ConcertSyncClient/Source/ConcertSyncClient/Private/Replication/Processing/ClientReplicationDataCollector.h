// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ObjectInfo.h"
#include "Replication/Processing/IReplicationDataSource.h"

#include "Containers/ArrayView.h"
#include "Containers/Set.h"
#include "Replication/Data/ConcertPropertySelection.h"
#include "Replication/Data/ObjectIds.h"
#include "Templates/SharedPointer.h"
#include "UObject/SoftObjectPtr.h"

class FConcertSyncClientLiveSession;
class IConcertClientReplicationBridge;
class UObject;
enum class EBreakBehavior : uint8;
struct FConcertPropertySelection;
struct FConcertReplicationStream;

namespace UE::ConcertSyncCore
{
	class IObjectReplicationFormat;
	class FObjectReplicationProcessor;
}

namespace UE::ConcertSyncClient::Replication
{
	class FLocalSyncControl;
	
	/**
	 * Exposes UObject instances to an FObjectReplicationProcessor.
	 * IConcertClientReplicationBridge tracks UObject lifetime, this class exposes them.
	 */
	class FClientReplicationDataCollector : public ConcertSyncCore::IReplicationDataSource
	{
	public:

		DECLARE_DELEGATE_RetVal(const TArray<FConcertReplicationStream>*, FGetClientStreams);

		FClientReplicationDataCollector(
			IConcertClientReplicationBridge& InReplicationBridge UE_LIFETIMEBOUND,
			ConcertSyncCore::IObjectReplicationFormat& InReplicationFormat UE_LIFETIMEBOUND,
			const FLocalSyncControl& SyncControl UE_LIFETIMEBOUND,
			FGetClientStreams InGetStreamsDelegate,
			const FGuid& InClientId
			);
		virtual ~FClientReplicationDataCollector() override;
		
		/**
		 * Indicates that this object should start replicating for the given streams.
		 * @param Object The object that should start replicating
		 * @param AddedStreams The streams determine which properties are to be replicated
		 */
		void AddReplicatedObjectStreams(const FSoftObjectPath& Object, TArrayView<const FGuid> AddedStreams);
		/**
		 * Indicates that certain properties of an object should no longer be replicated.
		 * @param Object The object that should is affected by the stream change
		 * @param RemovedStreams The properties in these streams will no longer be replicated
		 */
		void RemoveReplicatedObjectStreams(const FSoftObjectPath& Object, TArrayView<const FGuid> RemovedStreams);
		/**
		 * Called when the client modifies pre-existing object. Adjusts any inflight replication if needed.
		 * @param Object The object that should start replicating
		 * @param PutStreams The streams determine which properties are to be replicated
		 */
		void OnObjectStreamModified(const FSoftObjectPath& Object, TArrayView<const FGuid> PutStreams);

		/** Clears all currently replicated objects. */
		void ClearReplicatedObjects();

		/** Iterates every object for which there is at least one owning stream. */
		template<typename TCallback> requires std::is_invocable_r_v<EBreakBehavior, TCallback, const FSoftObjectPath&>
		void ForEachOwnedObject(TCallback&& Callback) const;
		/** Iterates every object and stream assigned to it. */
		template<typename TCallback> requires std::is_invocable_r_v<EBreakBehavior, TCallback, const FSoftObjectPath&, const FGuid&>
		void ForEachOwnedObjectAndStream(TCallback&& Callback) const;
		/** Writes all owning streams for ObjectPath into Paths. */
		void AppendOwningStreamsForObject(const FSoftObjectPath& ObjectPath, TSet<FGuid>& Paths) const;

		/** @return Whether ObjectPath is owned with StreamId. */
		bool OwnsObjectInStream(const FSoftObjectPath& ObjectPath, const FGuid& StreamId) const;
		/** @return Whether ObjectPath owns ObjectPath in any stream. */
		bool OwnsObjectInAnyStream(const FSoftObjectPath& ObjectPath) const;

		/** @return All streams containing ObjectPath that we have authority over within that stream. */
		TArray<FGuid> GetStreamsOwningObject(const FSoftObjectPath& ObjectPath) const;

		//~ Begin IReplicationDataSource Interface
		virtual void ConsumePendingObjects(TFunctionRef<void(const ConcertSyncCore::FPendingObjectReplicationInfo& Info, ConcertSyncCore::IReplicationDataItem& Item)> InProcess) override;
		//~ End IReplicationDataSource Interface

	private:

		/** Gets and tracks replicated objects */
		IConcertClientReplicationBridge& Bridge;
		/** Used to create the replication data sent to the server. */
		ConcertSyncCore::IObjectReplicationFormat& ReplicationFormat;

		/** Tells us whether we're allowed to replicate an object in a stream. */
		const FLocalSyncControl& SyncControl;

		/** Gets the stream of the managed client. */
		const FGetClientStreams GetStreamsDelegate;

		/** Endpoint ID of the client */
		const FGuid ClientId;
		
		/** The objects and their properties to replicate */
		TMap<FSoftObjectPath, TArray<FObjectInfo>> ObjectsToReplicate;
		/** Cached number of FObjectInfo in ObjectsToReplicate that have a valid FObjectInfo::ObjectCache. */
		int32 NumTrackedObjects = 0;

		// Handle events from bridge
		void StartTrackingObject(UObject& Object);
		void StopTrackingObject(const FSoftObjectPath& ObjectPath);
	};

	template <typename TCallback> requires std::is_invocable_r_v<EBreakBehavior, TCallback, const FSoftObjectPath&>
	void FClientReplicationDataCollector::ForEachOwnedObject(TCallback&& Callback) const
	{
		for (const TPair<FSoftObjectPath, TArray<FObjectInfo>>& Pair : ObjectsToReplicate)
		{
			if (Callback(Pair.Key) == EBreakBehavior::Break)
			{
				break;
			}
		}
	}

	template <typename TCallback> requires std::is_invocable_r_v<EBreakBehavior, TCallback, const FSoftObjectPath&, const FGuid&>
	void FClientReplicationDataCollector::ForEachOwnedObjectAndStream(TCallback&& Callback) const
	{
		for (const TPair<FSoftObjectPath, TArray<FObjectInfo>>& Pair : ObjectsToReplicate)
		{
			for (const FObjectInfo& ObjectInfo : Pair.Value)
			{
				if (Callback(Pair.Key, ObjectInfo.StreamId) == EBreakBehavior::Break)
				{
					return;
				}
			}
		}
	}
}
