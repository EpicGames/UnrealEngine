// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClientReplicationDataQueuer.h"

#include "Replication/IConcertClientReplicationBridge.h"

namespace UE::ConcertSyncClient::Replication
{
	TSharedRef<FClientReplicationDataQueuer> FClientReplicationDataQueuer::Make(
		IConcertClientReplicationBridge& ReplicationBridge,
		ConcertSyncCore::FObjectReplicationCache& InReplicationCache
		)
	{
		TSharedRef<FClientReplicationDataQueuer> Result = MakeShared<FClientReplicationDataQueuer>(ReplicationBridge);
		Result->BindToCache(InReplicationCache);
		return Result;
	}
	
	FClientReplicationDataQueuer::FClientReplicationDataQueuer(IConcertClientReplicationBridge& ReplicationBridge)
		: ReplicationBridge(ReplicationBridge)
	{}

	bool FClientReplicationDataQueuer::WantsToAcceptObject(const FConcertReplicatedObjectId& Object) const
	{
		// The server tries not to send objects to are unavailable but it could happen the object becomes unavailable while data is underway
		const bool bIsAvailable = ReplicationBridge.IsObjectAvailable(Object.Object);
		return bIsAvailable;
	}
}
