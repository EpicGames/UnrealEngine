// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Processing/ReplicationDataQueuer.h"
#include "Templates/SharedPointer.h"

class IConcertClientReplicationBridge;

namespace UE::ConcertSyncClient::Replication
{
	/** Queues events received from the server. */
	class FClientReplicationDataQueuer : public ConcertSyncCore::FReplicationDataQueuer
	{
		template <typename ObjectType, ESPMode Mode>
		friend class SharedPointerInternals::TIntrusiveReferenceController;
	public:

		static TSharedRef<FClientReplicationDataQueuer> Make(IConcertClientReplicationBridge& ReplicationBridge UE_LIFETIMEBOUND, ConcertSyncCore::FObjectReplicationCache& InReplicationCache UE_LIFETIMEBOUND);
		
		//~ Begin IReplicationCacheUser Interface
		virtual bool WantsToAcceptObject(const FConcertReplicatedObjectId& Object) const override;
		//~ End IReplicationCacheUser Interface

	private:
		
		FClientReplicationDataQueuer(IConcertClientReplicationBridge& ReplicationBridge UE_LIFETIMEBOUND);

		/** Used to determine whether the object is currently alive. */
		IConcertClientReplicationBridge& ReplicationBridge;
	};
}

