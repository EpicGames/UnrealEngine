// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"
#include "Replication/Data/ConcertPropertySelection.h"
#include "Replication/Data/SequenceId.h"

namespace UE::ConcertSyncClient
{
struct FObjectInfo
{
	/** The replication stream producing this object's data */
	FGuid StreamId;
	/** The properties to replicate */
	FConcertPropertySelection SelectedProperties;
			
	/** Incremented every time replication data is sent out. Used for performance tracing. */
	ConcertSyncCore::FSequenceId ReplicationSequenceId = 0;
};
}
