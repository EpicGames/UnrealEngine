// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectReplicationApplierProcessor.h"

#include "ConcertLogGlobal.h"
#include "ConcertSyncSettings.h"
#include "HAL/IConsoleManager.h"
#include "Replication/Formats/IObjectReplicationFormat.h"
#include "Replication/IConcertClientReplicationBridge.h"
#include "Replication/Processing/Actions/ReplicationActionDispatcher.h"
#include "Replication/Processing/IReplicationDataItem.h"
#include "Trace/ConcertProtocolTrace.h"

namespace UE::ConcertSyncClient::Replication
{
	static TAutoConsoleVariable<bool> CVarDebugReplicationActions(
		TEXT("Concert.Replication.DebugReplicationActions"),
		false,
		TEXT("Whether to debug the actions taken by UConcertSyncReplicationConfig::PostReplicateActions")
	);
	
	FObjectReplicationApplierProcessor::FObjectReplicationApplierProcessor(
		IConcertClientReplicationBridge& ReplicationBridge,
		ConcertSyncCore::IObjectReplicationFormat& ReplicationFormat,
		ConcertSyncCore::IReplicationDataSource& DataSource
		)
		: FObjectReplicationProcessor(DataSource)
		, ReplicationBridge(ReplicationBridge)
		, ReplicationFormat(ReplicationFormat)
	{}

	void FObjectReplicationApplierProcessor::ProcessObject(const FObjectProcessArgs& Args)
	{
		CONCERT_TRACE_REPLICATION_OBJECT_SCOPE(ApplyReceivedObject, Args.ObjectInfo.ObjectId.Object, Args.ObjectInfo.SequenceId);
		UObject* Object = ReplicationBridge.FindObjectIfAvailable(Args.ObjectInfo.ObjectId.Object);
		if (!Object)
		{
			UE_LOGF(LogConcert, Error, "Replication: Object %ls is unavailable. The data source should not have reported it.", *Args.ObjectInfo.ObjectId.Object.ToString());
			return;
		}

		// Get the post replication actions
		const TArray<FConcertReplicationActionEntry> DummyActions;
		const TArray<FConcertReplicationActionEntry>& Actions = UConcertSyncConfig::Get()->PostReplicateActions;
		const bool bDebugActions = CVarDebugReplicationActions.GetValueOnGameThread();
		
		bool bAppliedData = false;
		Args.Item.ExtractReplicationDataForObject(
			[this, &Args, Object, Actions, bDebugActions, &bAppliedData](const FConcertSessionSerializedPayload& Payload)
			{
				CONCERT_TRACE_REPLICATION_OBJECT_SCOPE(SerializeReceivedObject, Args.ObjectInfo.ObjectId.Object, Args.ObjectInfo.SequenceId);
				bAppliedData = true;

				// We're technically modifying the package so mark it modified. This will make Concert / Multi-User revert the changes on leaving the session.
				// This could be made faster by caching the package. For now leave it and possibly profile in future if needed.
				Object->MarkPackageDirty();

				ConcertSyncCore::FReplicationActionDispatcher ReplicationActionDispatcher(Actions, bDebugActions);
				UE_CLOGF(bDebugActions, LogConcert, Log, "===== %ls actions ====", *Object->GetPathName());
				const auto ProcessReplicatedProperty = [&ReplicationActionDispatcher](const FArchiveSerializedPropertyChain& Chain, FProperty& Property)
				{
					ReplicationActionDispatcher.OnReplicateProperty(Property);
				};
				ReplicationFormat.ApplyReplicationEvent(*Object, Payload, ProcessReplicatedProperty);

				// Execute any property dependent conditions, such as UpdateComponentToWorld for RelativeLocation, etc.
				const ConcertSyncCore::FReplicationActionArgs PostReplicationArgs(Args.ObjectInfo.ObjectId, Object);
				ReplicationActionDispatcher.ExecuteActions(PostReplicationArgs);
			});
		
		// This should not happen. If it does, we're wasting network bandwidth.
		UE_CLOGF(!bAppliedData, LogConcert, Warning, "Replication: Server sent data that could not be applied (likely it was empty) for object %ls from stream %ls", *Args.ObjectInfo.ObjectId.Object.ToString(), *Args.ObjectInfo.ObjectId.StreamId.ToString());
		CONCERT_TRACE_REPLICATION_OBJECT_SINK(Processed, Args.ObjectInfo.ObjectId.Object, Args.ObjectInfo.SequenceId);
	}
}
