// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Processing/ObjectReplicationProcessor.h"
#include "Replication/Processing/ReplicationDataQueuer.h"

class UConcertSyncConfig;
class IConcertClientReplicationBridge;

namespace UE::ConcertSyncClient::Replication
{
	/** Applies replication data to the object is is meant for. */
	class FObjectReplicationApplierProcessor : public ConcertSyncCore::FObjectReplicationProcessor 
	{
	public:

		explicit FObjectReplicationApplierProcessor(
			IConcertClientReplicationBridge& ReplicationBridge UE_LIFETIMEBOUND,
			ConcertSyncCore::IObjectReplicationFormat& ReplicationFormat UE_LIFETIMEBOUND,
			ConcertSyncCore::IReplicationDataSource& DataSource UE_LIFETIMEBOUND
			);
		
	protected:

		//~ Begin FObjectReplicationProcessor Interface
		virtual void ProcessObject(const FObjectProcessArgs& Args) override;
		//~ End FObjectReplicationProcessor Interface

	private:

		/** Retrieves objects for applying replication data to. */
		IConcertClientReplicationBridge& ReplicationBridge;

		/** Unpacks replication data and applies it to an UObject instance. */
		ConcertSyncCore::IObjectReplicationFormat& ReplicationFormat;
	};
}

