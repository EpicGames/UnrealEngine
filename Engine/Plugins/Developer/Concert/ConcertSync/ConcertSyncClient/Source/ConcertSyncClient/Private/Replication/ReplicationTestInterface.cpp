// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertClientReplicationBridge.h"
#include "ConcertSyncSessionFlags.h"
#include "Replication/IConcertClientReplicationBridge.h"
#include "Replication/IConcertClientReplicationManager.h"
#include "Manager/ReplicationManager.h"

/**
 * Exposes functions that are required for testing.
 * 
 * These functions are technically exported but conceptually not part of the public interface
 * and should only be used for the purpose of automated testing.
 */
namespace UE::ConcertSyncClient::TestInterface
{
	CONCERTSYNCCLIENT_API TSharedRef<IConcertClientReplicationManager> CreateClientReplicationManager(
		TSharedRef<IConcertClientSession> InLiveSession,
		IConcertClientReplicationBridge& InBridge,
		EConcertSyncSessionFlags SessionFlags
		)
	{
		const TSharedRef<Replication::FReplicationManager> Result = MakeShared<Replication::FReplicationManager>(MoveTemp(InLiveSession), InBridge, SessionFlags);
		Result->StartAcceptingJoinRequests();
		return Result;
	}

	CONCERTSYNCCLIENT_API TSharedRef<IConcertClientReplicationBridge> CreateClientReplicationBridge()
	{
		return MakeShared<Replication::FConcertClientReplicationBridge>();
	}
}
