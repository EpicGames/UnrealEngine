// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertClientPackageManager.h"
#include "HAL/Platform.h"
#include "Interface/DefaultPackageReloadHandler.h"
#include "Types/Package/HotReloadPackageArgs.h"
#include "Types/Package/PurgePackageArgs.h"

class FConcertClientPackageManager;

namespace UE::ConcertSyncClient
{
/** Enqueues packages for hot-reload and purge when the sandbox is being shutdown. */
class FConcertPackageReloadHandler : public FileSandboxCore::FDefaultPackageReloadHandler
{
	using Super = FDefaultPackageReloadHandler;
public:
	
	explicit FConcertPackageReloadHandler(FConcertClientPackageManager& InPackageManager UE_LIFETIMEBOUND)
		: PackageManager(InPackageManager)
	{}
	
	virtual void PurgePackages(const FileSandboxCore::FPurgePackageArgs& InArgs) override
	{
		switch (InArgs.Phase)
		{
		case FileSandboxCore::ESandboxPackageReloadPhase::Startup:
			// Concert already handles replaying packages at startup. So skip the operation.
			break;
			
		case FileSandboxCore::ESandboxPackageReloadPhase::Sandboxed:
			FDefaultPackageReloadHandler::PurgePackages(InArgs);
			break;
			
		case FileSandboxCore::ESandboxPackageReloadPhase::Shutdown:
			for (const FName& Package : InArgs.PackageNames)
			{
				PackageManager.EnqueuePurge(Package);
			}
			break;
		}
	}
	virtual void HotReloadPackages(const FileSandboxCore::FHotReloadPackageArgs& InArgs) override
	{
		switch (InArgs.Phase)
		{
		case FileSandboxCore::ESandboxPackageReloadPhase::Startup:
			// Concert already handles replaying packages at startup. So skip the operation.
			break;
			
		case FileSandboxCore::ESandboxPackageReloadPhase::Sandboxed:
			FDefaultPackageReloadHandler::HotReloadPackages(InArgs);
			break;
			
		case FileSandboxCore::ESandboxPackageReloadPhase::Shutdown:
			for (const FName& Package : InArgs.PackageNames)
			{
				PackageManager.EnqueueHotReload(Package);
			}
			break;
		}
	}
	
private:
	
	FConcertClientPackageManager& PackageManager;
};
}
