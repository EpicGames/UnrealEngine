// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertPackageSandbox.h"

#include "ConcertClientPersistData.h"
#include "ConcertLogGlobal.h"
#include "ConcertPackageReloadHandler.h"
#include "ConcertSession.h"
#include "ConcertSyncClientLiveSession.h"
#include "IFileSandboxCoreModule.h"
#include "ISandboxInstance.h"
#include "ISandboxManager.h"
#include "Interface/Feedback/AppendTextErrorHandler.h"
#include "Misc/PackageName.h"
#include "Sandbox/ConcertSandboxLock.h"
#include "Types/Manager/LoadSandboxByNameArgs.h"
#include "Types/Manager/NewSandboxArgs.h"
#include "Types/Manager/SandboxCreationResult.h"
#include "Types/Sandbox/RevertResult.h"

namespace UE::ConcertSyncClient
{
namespace Private
{
static bool CreateNewSandbox(
	FileSandboxCore::ISandboxManager& InSandboxManager, const FileSandboxCore::FSandboxInitArgs& InInitArgs,
	const FString& InSandboxName, const FString& InWorkspaceDirectory
	)
{
	using namespace FileSandboxCore;
	
	FNewSandboxArgs Args(InSandboxName);
	Args.SandboxBasePath = InWorkspaceDirectory;
	Args.InitArgs = InInitArgs;
	
	const FNewSandboxResult Result = InSandboxManager.CreateNewSandbox(Args);
	
	const bool bIsSuccess = Result.HasValue();
	UE_CLOGF(!bIsSuccess, LogConcert, Error, "Failed to create sandbox. Reason: %ls", *LexToString(Result.GetError().Reason));
	return bIsSuccess;
}
	
static bool LoadSandbox(
	FileSandboxCore::ISandboxManager& InSandboxManager, const FileSandboxCore::FSandboxInitArgs& InInitArgs, 
	const FString& InSandboxName, const FString& InWorkspaceDirectory
	)
{
	using namespace FileSandboxCore;
	
	FLoadSandboxByNameArgs Args(InSandboxName, InWorkspaceDirectory);
	Args.InitArgs = InInitArgs;
	
	const FLoadSandboxResult Result = InSandboxManager.LoadNamedSandbox(Args);
	
	const bool bIsSuccess = Result.HasValue();
	UE_CLOGF(!bIsSuccess, LogConcert, Error, "Failed to load sandbox. Reason: %ls", *LexToString(Result.GetError().Reason));
	return bIsSuccess;
}
}
	
TUniquePtr<FConcertPackageSandbox> FConcertPackageSandbox::CreateOrLoad(
	FConcertClientPackageManager& InPackageManager,
	const TSharedRef<FConcertSyncClientLiveSession>& InLiveSession,
	const FString& InClientRole
	)
{
	using namespace FileSandboxCore;
	ISandboxManager& SandboxManager = IFileSandboxCoreModule::Get().GetSandboxManager();

	// TODO DP: We should ask the user nicely whether they are sure to leave the currently active sandbox
	if (SandboxManager.HasActiveSandbox())
	{
		SandboxManager.LeaveSandbox();
	}

	const FString SandboxName(TEXT("Sandbox"));
	const FString WorkspaceDirectory = InLiveSession->GetSession().GetSessionWorkingDirectory();
	const TOptional<FString> ExistingDirectory = FindSandboxByName(SandboxName, WorkspaceDirectory);
	
	const TSharedRef<FConcertSandboxLock> Lock = MakeShared<FConcertSandboxLock>(InClientRole);
	const FSandboxInitArgs InitArgs
	{ 
		// This causes ISourceControlState::IsLocal to return false.
		// Concert live propagates assets to other clients, so no assets can be local
		// This is used to determine whether redirector assets are left when things are renamed
		.Flags = ESandboxInitFlags::ForceRedirectorsOnRenameInSourceControl,
		.Lock = Lock,
		.ReloadHandler = MakeShared<FConcertPackageReloadHandler>(InPackageManager)
	};
	
	if (ExistingDirectory && Private::LoadSandbox(SandboxManager, InitArgs, SandboxName, WorkspaceDirectory))
	{
		return MakeUnique<FConcertPackageSandbox>(Lock);
	}
	
	if (Private::CreateNewSandbox(SandboxManager, InitArgs, SandboxName, WorkspaceDirectory))
	{
		return MakeUnique<FConcertPackageSandbox>(Lock);
	}
	
	return nullptr;
}

FConcertPackageSandbox::FConcertPackageSandbox(const TSharedRef<FConcertSandboxLock> InLock)
	: Lock(InLock)
{}

FConcertPackageSandbox::~FConcertPackageSandbox()
{
	Lock->ReleaseLock();
	FileSandboxCore::IFileSandboxCoreModule::Get().GetSandboxManager().LeaveSandbox();
}

FPersistResult FConcertPackageSandbox::PersistSandbox(TArrayView<const FString> InFiles, FPersistParameters InParams)
{
	using namespace FileSandboxCore;
	
	ISandboxManager& SandboxManager = IFileSandboxCoreModule::Get().GetSandboxManager();
	if (ISandboxInstance* Sandbox = SandboxManager.GetActiveSandboxInstance())
	{
		FAppendTextPersistFeedback ErrorAsTextFeedback;
		const FPersistArgs Args { InFiles, InParams.bShouldMakeWritableIfNoSourceControl, &ErrorAsTextFeedback };
		const FPersistResult Result = Sandbox->PersistSandbox(Args);
		return ::FPersistResult(ErrorAsTextFeedback.AllErrors, static_cast<::EPersistStatus>(Result.PersistStatus));
	}

	return ::FPersistResult { .PersistStatus = ::EPersistStatus::Failure };
}

bool FConcertPackageSandbox::DeletedPackageExistsInNonSandbox(FString InFilename) const
{
	using namespace FileSandboxCore;
	ISandboxManager& SandboxManager = IFileSandboxCoreModule::Get().GetSandboxManager();
	ISandboxInstance* Sandbox = SandboxManager.GetActiveSandboxInstance();
	return Sandbox && Sandbox->DeletedPackageExistsInNonSandbox(InFilename);
}
}