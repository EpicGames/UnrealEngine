// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IConcertSandbox.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"

class FConcertClientPackageManager;
class FConcertSyncClientLiveSession;

namespace UE::FileSandboxCore { class ISandboxInstance; }

namespace UE::ConcertSyncClient
{
class FConcertSandboxLock;

/** Handles sandboxing changes made to files while in a Concert session. */
class FConcertPackageSandbox : public FNoncopyable, public IConcertSandbox
{
public:
	
	/** Creates a file sandbox. */
	static TUniquePtr<FConcertPackageSandbox> CreateOrLoad(
		FConcertClientPackageManager& InPackageManager UE_LIFETIMEBOUND,
		const TSharedRef<FConcertSyncClientLiveSession>& InLiveSession, const FString& InClientRole
		);
	
	explicit FConcertPackageSandbox(const TSharedRef<FConcertSandboxLock> InLock);
	virtual ~FConcertPackageSandbox() override;

	//~ Begin IConcertSandbox Interface
	virtual FPersistResult PersistSandbox(TArrayView<const FString> InFiles, FPersistParameters InParams) override;
	virtual bool DeletedPackageExistsInNonSandbox(FString InFilename) const override;
	//~ Begin IConcertSandbox Interface
	
private:
	
	/** Interacts with the sandbox system to prevent sandbox from being left while in Concert session. */
	const TSharedRef<FConcertSandboxLock> Lock;
};
}