// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertSandboxLock.h"

#include "ConcertSyncClientLiveSession.h"
#include "ConcertClient/Private/ConcertClientSession.h"

namespace UE::ConcertSyncClient
{
namespace LockDetail
{
static FText RoleToReason(const FString& InString)
{
	// TODO: Display label for the client should be promoted to a higher level concept and should not be implemented here.
	if (InString.Equals(TEXT("MultiUser")))
	{
		return NSLOCTEXT("FConcertSandboxLock", "LockReason", "The active Multi-User session is sandboxing the engine.");
	}
	
	return NSLOCTEXT("FConcertSandboxLock", "LockReason.Concert", "The active Concert session is sandboxing the engine.");
}
}

FConcertSandboxLock::FConcertSandboxLock(const FString& InClientRole)
	: ClientRole(InClientRole)
{}

void FConcertSandboxLock::ReleaseLock()
{
	check(bIsLocked);
	bIsLocked = false;
}

#define SET_REASON(Reason) if (OutReason) { *OutReason = Reason; }
bool FConcertSandboxLock::CanLeaveSandbox(FText* OutReason) const
{
	if (bIsLocked)
	{
		SET_REASON(LockDetail::RoleToReason(ClientRole));
		return false;
	}
	
	return true;
}

bool FConcertSandboxLock::RequestLiftLock(FText* OutReason)
{
	// TODO UE-356773: Unsupported for now. 
	// We need some mechanism to tell Multi-User to go through the workflow of leaving a session, which causes the persist changes UI to be shown.
	return false;
}

bool FConcertSandboxLock::SupportsLiftingLock() const
{
	// TODO UE-356773: Unsupported for now. 
	// We need some mechanism to tell Multi-User to go through the workflow of leaving a session, which causes the persist changes UI to be shown.
	return false;
}
}
