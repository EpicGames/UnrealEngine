// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interface/ISandboxLock.h"
#include "Templates/SharedPointer.h"

class FConcertSyncClientLiveSession;
class FConcertSyncClient;

namespace UE::ConcertSyncClient
{
/** Prevents the sandbox system from leaving the current sandbox for as long as Concert is in a session. */
class FConcertSandboxLock : public FileSandboxCore::ISandboxLock
{
public:
	
	explicit FConcertSandboxLock(const FString& InClientRole);
	
	/** Lifts the lock. Called when the session stops. */
	void ReleaseLock();
	
	//~ Begin ISandboxLock Interface
	virtual bool CanLeaveSandbox(FText* OutReason) const override;
	virtual bool RequestLiftLock(FText* OutReason) override;
	virtual bool SupportsLiftingLock() const override;
	//~ End ISandboxLock Interface
	
private:
	
	/** The client role. Relevant for lock reason generation. */
	const FString ClientRole;
	
	/** Whether the sandbox is currently locked. */
	bool bIsLocked = true;
};
}

