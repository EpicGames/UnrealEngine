// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Templates/UniquePtr.h"

class FName;
class FString;
struct FDateTime;
struct FPersistParameters;
struct FPersistResult;

namespace UE::ConcertSyncClient
{
class IConcertSandbox
{
public:
	
	/**
	 *	Persist the file list from the sandbox state onto the real files Will mark files for which the operation was
	 *	succesful as persisted.
	 *
	 *	@param InFiles	List of files names to persist as full paths.
	 *	@param InParams	Parameters controlling the persist options specified by the user.
	 *	@return Returns a persist result object that contains result status.
	 */
	virtual FPersistResult PersistSandbox(TArrayView<const FString> InFiles, FPersistParameters InParams) = 0;

	/** @return true if the given package file exists on non sandbox path. */
	virtual bool DeletedPackageExistsInNonSandbox(FString InFilename) const = 0;

	virtual ~IConcertSandbox() = default;
};
}
