// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"

DECLARE_DELEGATE(FOnExecuteAction);

/** Defines the possible type of actions this module can provide. The type of action can be used to map a button type/color by the UI. */
enum class EConcertActionType : uint8
{
	Normal,
	Primary,
	Info,
	Success,
	Warning,
	Danger,
	NUM,
};

/**
 * Defines actions for a given context. The actions are usually bound to UI buttons. This allows the module to
 * expose contextual functionalities without exposing some private part of the API.
  */
struct FConcertActionDefinition
{
	FConcertActionDefinition() : IsVisible(true), IsEnabled(true) {}
	EConcertActionType Type = EConcertActionType::Normal;
	TAttribute<bool>  IsVisible;   // If the action should be displayed or not.
	TAttribute<bool>  IsEnabled;   // If the action is enabled or not.
	TAttribute<FText> Text;        // The action name. Usually correspond to a button caption in UI.
	TAttribute<FText> ToolTipText; // The action tooltip. Usually correspond to a button tooltip.
	FOnExecuteAction  OnExecute;   // The function to call to execute the action.
	TAttribute<FName> IconStyle;   // The icon brush style name (if set, it replaces the text).
};
