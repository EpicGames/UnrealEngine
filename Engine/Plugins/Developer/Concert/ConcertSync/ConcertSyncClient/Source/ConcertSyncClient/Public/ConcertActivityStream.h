// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConcertMessageData.h"

#define UE_API CONCERTSYNCCLIENT_API

struct FConcertSessionActivity;
class IConcertClient;

/**
 * Streams an archived session activities asynchronously and caches them until the client poll and consume them synchronously. The
 * streams also caches the corresponding client info for lookup.
 */
class FConcertActivityStream
{
public:
	/**
	 * Construct a input activity stream used to read a Concert session activities from the most recent to the oldest.
	 * @param InConcertClient The client used to request the session to the server.
	 * @param InAdminEndpointId The server hosting the session from which the activities should be streamed.
	 * @param InSessionId The session ID from which the activity should be streamed.
	 * @param bInIncludeActivityDetails Includes extra information to inspect the activity. If true, the FConcertClientSessionActivity::EventPayload field will be set for transaction and package activity.
	 * @param InBufferSize The stream buffer cache size. The stream will fill its buffer and stop pulling them from the server until some activities are consumed.
	 * @see FConcertClientSessionActivity
	 */
	UE_API FConcertActivityStream(TWeakPtr<IConcertClient, ESPMode::ThreadSafe> InConcertClient, const FGuid& InAdminEndpointId, const FGuid& InSessionId, bool bInIncludeActivityDetails, uint32 InBufferSize = 1024);

	/**
	 * Reads all buffered activities. Call this function periodically (between frames, not in a tight loop). Every times this function is called,
	 * all buffered activities are consumed (appended to InOutActivities buffer) and a new request is fired to refill the internal buffer. The activities
	 * are streamed from the most recent to the oldest.
	 * @param InOutActivities The array to which the activities should be appended.
	 * @param OutReadCount The number of activities consumed and appended to InOutActivities. Can be 0 if consumed faster than received form the server.
	 * @param OutErrorMsg Contains the last error message, if any or an empty text.
	 * @return True if all activities were consumed, false otherwise.
	 */
	UE_API bool Read(TArray<TSharedPtr<FConcertSessionActivity>>& InOutActivities, int32& OutReadCount, FText& OutErrorMsg);

	/** Returns the client info corresponding to an activities. The stream maintains this state as an optimization. */
	const FConcertClientInfo* GetActivityClientInfo(const FGuid& ActivityEndpointId) const { return EndpointClientInfoMap.Find(ActivityEndpointId); }

private:
	/** Run asynchronous requests in background and caches the activities until consumed.*/
	UE_API void Fetch(int64 ActivityCount);

	/** Returns how many activities should be requested per request. */
	constexpr int64 GetRequestBatchSize() { return BufferSize; }

private:
	TWeakPtr<IConcertClient, ESPMode::ThreadSafe> WeakConcertClient;
	FGuid AdminEndpointId;
	FGuid SessionId;
	int64 LowestFetchedActivityId = 0; // The next Activity ID to fetch. Activity IDs are 1-based.
	int64 TotalActivityCount = 0;
	TArray<TSharedPtr<FConcertSessionActivity>> AvailableActivities; // Activities currently cached and ready for consumption.
	TMap<FGuid, FConcertClientInfo> EndpointClientInfoMap;
	TSharedPtr<uint8, ESPMode::ThreadSafe> FetchRequestContinuationToken; // Used to disarm TFuture continuation.
	FText LastErrorMsg;
	uint32 BufferSize;
	bool bAllActivitiesFetched = false;
	bool bFetching = false;
	bool bIncludeActivityDetails = false;
};

#undef UE_API
