// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ConcertClientPresenceActor.h"
#include "ConcertClientDesktopPresenceActor.generated.h"


struct FDesktopPresenceUpdateEvent;
class UMaterialInstanceDynamic;
class USplineMeshComponent;
class UStaticMeshComponent;

/**
  * A AConcertClientDesktopPresenceActor is a child of AConcertClientPresenceActor that is used to represent users in desktop 
  */
UCLASS(Abstract, Transient, NotPlaceable, Blueprintable)
class AConcertClientDesktopPresenceActor : public AConcertClientPresenceActor
{
	GENERATED_UCLASS_BODY()

public:
	/** UObject interface */
	virtual void BeginDestroy() override;

	/** Set color of presence mesh */
	virtual void SetPresenceColor(const FLinearColor& InColor) override;

	/** AConcertClientPresenceActor Interface */
	virtual void InitPresence(const class UConcertAssetContainer& InAssetContainer, FName DeviceType) override;

	/** Handle presence update events */
	virtual void HandleEvent(const FStructOnScope& InEvent) override;

	virtual void Tick(float DeltaSeconds) override;

	void HideLaser();
	void ShowLaser();

private:

	void SetLaserTimer(FTimerManager& InTimerManager, bool bLaserShouldShow);

	/** The camera mesh component to show visually where the camera is placed */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = "Rendering", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UStaticMeshComponent> DesktopMeshComponent;

	/** Spline mesh representing laser */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = "Rendering", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<USplineMeshComponent> LaserPointer;

	/** Dynamic material for the laser */
	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> LaserMid;

	/** Dynamic material for the laser */
	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> LaserCoreMid;

	UPROPERTY()
	bool bMovingCamera;

	UPROPERTY()
	FVector LastEndPoint;

	UPROPERTY()
	bool bIsLaserVisible;

	/** Handle for efficient management of laser timer */
	FTimerHandle LaserTimerHandle;

	/** Most recently received value of bMovingCamera */
	bool bLastKnownMovingCamera;

	/** Movement smoothing for laser start */
	TOptional<FConcertClientMovement> LaserStartMovement;

	/** Movement smoothing for laser start */
	TOptional<FConcertClientMovement> LaserEndMovement;
};


