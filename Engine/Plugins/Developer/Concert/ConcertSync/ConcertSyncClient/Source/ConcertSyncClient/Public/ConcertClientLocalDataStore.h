// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConcertDataStoreMessages.h"

class FConcertDataStore;

/**
 * Maintains a type-safe key/value local map where the values are USTRUCT() struct
 * or a supported basic types (int8, uint8, int16, uint16, int32, uint32, int64,
 * uint64, float, double, bool, FName, FText, FString). This class is meant to
 * be used as a local/private client store where the IConcertClientDataStore interface
 * is meant to be used in a client(s)/server scenario.
 *
 * <b>Usage example</b>
 * @code
 * FConcertClientLocalDataStore MyStore;
 * FName MyKey(TEXT("MyKey1"));
 * uint64 MyValue = 100ull;
 * if (auto Stored = MyStore.FetchOrAdd(MyKey, MyValue))
 * {
 *     if (Stored = MyStore.Store(MyKey, Stored.GetValue() + 10))
 *     {
 *         check(MyStore.FetchAs<uint64>(MyKey).GetValue() == MyValue + 10);
 *         if (Stored = MyStore.CompareExchange(MyKey, MyValue + 10, MyValue + 20))
 *         {
 *             check(MyStore.FetchAs<uint64>(MyKey).GetValue() == MyValue + 20);
 *         }
 *     }
 * }
 * check(MyStore.FetchAs<uint64>(MyKey).GetValue() == MyValue + 20);
 * @endcode
 */
class FConcertClientLocalDataStore
{
public:
	/** Constructs the data store map. */
	CONCERTSYNCCLIENT_API FConcertClientLocalDataStore();

	/**
	 * Searches the store for the specified key, if not found, adds a new key/value pair, otherwise,
	 * if the stored value type matches the initial value type, fetches the stored value. The function
	 * accepts a USTRUCT() type or a supported basic type directly. To store complex types such as TArray<>
	 * TMap<> or TSet<>, wrap the type in a USTRUCT().
	 *
	 * @tparam T Any USTRUCT() struct or a supported basic types (int8, uint8, int16, uint16, int32, uint32, int64, uint64, float, double, bool, FName, FText, FString).
	 * @param Key The value name to fetch or add.
	 * @param InitialValue The value to store if the key doesn't already exist.
	 * @return The operation result. The possible result codes are:
	 *     - EConcertDataStoreResultCode::Added if the key/value was added. The result holds a pointer on the newly added value.
	 *     - EConcertDataStoreResultCode::Fetched if the key was already taken and type matched. The result holds a pointer on the fetched value.
	 *     - EConcertDataStoreResultCode::TypeMismatch if the key was already taken but the types did not match. The result holds a null pointer.
	 * @see [The class documentation for an example.](@ref FConcertClientLocalDataStore)
	 */
	template<typename T>
	TConcertDataStoreResult<T> FetchOrAdd(const FName& Key, const T& InitialValue)
	{
		// Not calling the DataStore::FetchOrAdd() to avoid serializing InitialValue if the value is already cached. (For network call, we would rather serialize and send it with the request)
		FConcertDataStoreResult Result = InternalFetch(Key, TConcertDataStoreType<T>::GetFName());
		if (Result.Code == EConcertDataStoreResultCode::Fetched)
		{
			return TConcertDataStoreResult<T>(MoveTemp(Result));
		}

		return Store<T>(Key, InitialValue);
	}

	/**
	 * Looks up the specified key, if found and type matches, fetches the corresponding value. If the key is not found
	 * or the requested type doesn't match the stored type, the operation fails.
	 *
 	 * @tparam T Any USTRUCT() struct or a supported basic types (int8, uint8, int16, uint16, int32, uint32, int64, uint64, float, double, bool, FName, FText, FString).
	 * @param Key The value name to lookup.
	 * @return The operation result. The possible result codes are:
	 *     - EConcertDataStoreResultCode::Fetched if the key value was retrieved. The result holds a pointer on the fetched value.
	 *     - EConcertDataStoreResultCode::NotFound if the key could not be found. The result holds a null pointer.
	 *     - EConcertDataStoreResultCode::TypeMismatch if the key was found, but the requested type did not match the stored type. The result holds a null pointer.
	 * @see [The class documentation for an example.](@ref FConcertClientLocalDataStore)
	 */
	template<typename T>
	TConcertDataStoreResult<T> FetchAs(const FName& Key) const
	{
		return TConcertDataStoreResult<T>(InternalFetch(Key, TConcertDataStoreType<T>::GetFName()));
	}

	/**
	 * Looks up the specified key, if it doesn't exist yet, adds new key/value pair at version 1, else if the
	 * stored value type matched the specified value type, overwrites the value and increment its version by one,
	 * otherwise, the operation fails.
	 *
	 * @tparam T Any USTRUCT() struct or a supported basic types (int8, uint8, int16, uint16, int32, uint32, int64, uint64, float, double, bool, FName, FText, FString).
	 * @param Key The value name to store.
	 * @param Value The value to store.
	 * @return The operation result. The possible result codes are:
	 *     - EConcertDataStoreResultCode::Added if the key/value was added. The result holds a pointer on the newly added value.
	 *     - EConcertDataStoreResultCode::Exchanged if the existing key value was updated. The result holds a pointer on the latest stored value.
	 *     - EConcertDataStoreResultCode::TypeMismatch if the key was already taken but the value types did not match. The result holds a null pointer.
	 * @see [The class documentation for an example.](@ref FConcertClientLocalDataStore)
	 */
	template<typename T>
	TConcertDataStoreResult<T> Store(const FName& Key, const T& Value)
	{
		const typename TConcertDataStoreType<T>::StructType& StructWrappedValue = TConcertDataStoreType<T>::AsStructType(Value);
		FConcertSessionSerializedPayload SerializedValue;
		SerializedValue.SetPayload(TConcertDataStoreType<T>::StructType::StaticStruct(), &StructWrappedValue);
		return TConcertDataStoreResult<T>(InternalStore(Key, TConcertDataStoreType<T>::GetFName(), SerializedValue));
	}

	/**
	 * Exchanges the stored value to \a Desired if a stored value corresponding to \a Key exists, has the same
	 * type and its value is equal to \a Expected, otherwise, the operation fails.
	 *
	 * @tparam T Any USTRUCT() struct or a supported basic types (int8, uint8, int16, uint16, int32, uint32, int64, uint64, float, double, bool, FName, FText, FString).
	 * @param Key The value name.
	 * @param Expected The value the caller expects to be stored.
	 * @param Desired The value the caller wants to store.
	 * @return The operation result. The possible result codes are:
	 *     - EConcertDataStoreResultCode::Exchanged if the desired value was sucessfully exchanged and stored. The result holds a pointer on the newly stored value.
	 *     - EConcertDataStoreResultCode::Fetched if the stored value was not the expected one. The stored value was fetched instead. The result holds a pointer on the fetched value.
	 *     - EConcertDataStoreResultCode::NotFound if the key could not be found. The result holds a null pointer.
	 *     - EConcertDataStoreResultCode::TypeMismatch if the stored data type did not match the expected/desired type. The result holds a null pointer.
	 * @see [The class documentation for an example.](@ref FConcertClientLocalDataStore)
	 */
	template<typename T>
	TConcertDataStoreResult<T> CompareExchange(const FName& Key, const T& Expected, const T& Desired)
	{
		// If the key exists and types match.
		FConcertDataStoreResult Result = InternalFetch(Key, TConcertDataStoreType<T>::GetFName());
		if (Result.Code == EConcertDataStoreResultCode::Fetched)
		{
			const typename TConcertDataStoreType<T>::StructType& StructWrappedExpected = TConcertDataStoreType<T>::AsStructType(Expected);
			FConcertSessionSerializedPayload ExpectedValueSerialized;
			ExpectedValueSerialized.SetPayload(TConcertDataStoreType<T>::StructType::StaticStruct(), &StructWrappedExpected);

			// If the stored value equals the expected value.
			if (Result.Value->SerializedValue.PayloadBytes.Bytes == ExpectedValueSerialized.PayloadBytes.Bytes )
			{
				return Store<T>(Key, Desired);
			}
		}

		return TConcertDataStoreResult<T>(MoveTemp(Result));
	}

private:
	// Wraps calls to FConcertDataStore to avoid leaking the type publicly.
	CONCERTSYNCCLIENT_API FConcertDataStoreResult InternalFetch(const FName& Key, const FName& TypeName) const;
	CONCERTSYNCCLIENT_API FConcertDataStoreResult InternalStore(const FName& Key, const FName& TypeName, const FConcertSessionSerializedPayload& Value);

	/** Maps keyName/keyValue. */
	TUniquePtr<FConcertDataStore> DataStore;
};
