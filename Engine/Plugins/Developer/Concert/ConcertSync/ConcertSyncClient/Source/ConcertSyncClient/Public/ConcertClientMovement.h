// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FConcertClientMovement
{
public:

	/** Constructor */
	FConcertClientMovement(double InUpdateFrequencyTimeSeconds, double InCurrentTimestamp, const FVector& InPosition);

	/** Constructor */
	FConcertClientMovement(double InUpdateFrequencyTimeSeconds, double InCurrentTimestamp, const FVector& InPosition, const FQuat& InOrientation);

	/** Destructor */
	~FConcertClientMovement();

	/**
	* Move smoothly over time based on previous and last known positions.
	*
	* @param InDeltaTimeSeconds:	Time passed since previous call to MoveSmooth.
	* @param OutPosition:			New position after moving
	* @param OutOrientation:		New orientation after moving
	*/
	void MoveSmooth(double InDeltaTimeSeconds, FVector& OutPosition, FQuat* OutOrientation = nullptr);

	/** Update last known location. */
	void UpdateLastKnownLocation(double InUpdateTimestampSeconds, const FVector& InPosition, const FQuat* InOrientation = nullptr);

private:

	/** Previous actual position. */
	FVector PreviousPosition;

	/** Previous actual orientation. */
	FQuat PreviousOrientation;

	/** Last known position */
	FVector LastKnownPosition;

	/** Last known orientation */
	FQuat LastKnownOrientation;

	/** Most recent position computed by MoveSmooth */
	FVector CurrentPosition;

	/** Most recent orientation computed by MoveSmooth */
	FQuat CurrentOrientation;

	/** Delta since last known location was updated */
	double TimeSecondsSinceLastKnownUpdate;

	/** Time stamp of last known location update */
	double LastKnownUpdateTimestampSeconds;

	/** How frequently last known position is updated */
	double UpdateFrequencyTimeSeconds;

	/** Has orientation */
	bool bHasOrientation;
};


