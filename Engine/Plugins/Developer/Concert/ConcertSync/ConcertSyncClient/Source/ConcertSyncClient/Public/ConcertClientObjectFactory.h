// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "UObject/Object.h"

#include "ConcertClientObjectFactory.generated.h"

/**
 * Factory API that can be implemented to extend the set of objects that Concert understands how to create/destroy.
 * @note This API is const as it operates on CDOs, which also means that a factory implementation cannot store any additional state.
 */
UCLASS(MinimalAPI, Abstract)
class UConcertClientObjectFactory : public UObject
{
	GENERATED_BODY()
	
public:
	//~ UObject
	CONCERTSYNCCLIENT_API virtual void PostInitProperties() override;
	CONCERTSYNCCLIENT_API virtual void BeginDestroy() override;

	/**
	 * Attempt to find a factory that supports the given class.
	 * @note If multiple factories support the given class then the result isn't guaranteed to be deterministic.
	 */
	static CONCERTSYNCCLIENT_API const UConcertClientObjectFactory* FindFactoryForClass(const UClass* Class);

	/**
	 * Does this factory support creating/destroying objects of the given class?
	 */
	virtual bool SupportsClass(const UClass* Class) const
	{
		return false;
	}
	
	/**
	 * Attempt to create a new outer from the given arguments, and fill in OutOuter.
	 * @return True if this function handled the creation attempt (this doesn't mean that OutOuter isn't null!).
	 */
	virtual bool CreateOuter(UObject*& OutOuter, const FString& OuterPathName) const
	{
		return false;
	}

	/**
	 * Attempt to create a new object from the given arguments, and fill in OutObject.
	 * @return True if this function handled the creation attempt (this doesn't mean that OutObject isn't null!), or false if we should fallback to using NewObject.
	 */
	virtual bool CreateObject(UObject*& OutObject, UObject* Outer, const UClass* Class, const FName Name, const EObjectFlags Flags) const
	{
		return false;
	}
	
	/**
	 * Attempt to create a new object from the given source object, and fill in OutObject.
	 * @return True if this function handled the creation attempt (this doesn't mean that OutObject isn't null!), or false if we should fallback to using StaticDuplicateObject.
	 */
	virtual bool DuplicateObject(UObject*& OutObject, const UObject* SourceObject, UObject* Outer, const UClass* Class, const FName Name, const EObjectFlags Flags) const
	{
		return false;
	}

	/**
	 * Should this factory run InitializeObjects before OtherFactory?
	 * @return True if this factory needs to run its InitializeObjects before OtherFactory (to control initialization order between factories), or false if it doesn't matter which order they run.
	 */
	virtual bool ShouldInitializeObjectsBefore(const UConcertClientObjectFactory* OtherFactory) const
	{
		return false;
	}

	/**
	 * Perform any additional initialization on the given objects.
	 * @note This is called after CreateObject/DuplicateObject has been called for every object in the transaction, and can be used for delayed initialization of interdependent objects.
	 */
	virtual void InitializeObjects(TArrayView<UObject* const> Objects) const
	{
	}
	
	/**
	 * Attempt to destroy the given object.
	 * @return True if this function handled the destruction attempt, or false if we should fallback to using MarkAsGarbage.
	 */
	virtual bool DestroyObject(UObject* Object) const
	{
		return false;
	}
};
