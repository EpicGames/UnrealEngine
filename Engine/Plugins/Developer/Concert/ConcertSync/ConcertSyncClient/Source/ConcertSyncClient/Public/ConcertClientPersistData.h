// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "UObject/NameTypes.h"
#include "Containers/Array.h"

class ISourceControlProvider;

/** Input parameters for persistance of an existing session. */
struct FPersistParameters
{
	/** Name of the packages to persist.*/
	TArrayView<const FName> PackagesToPersist;

	/** Boolean to control if we will force the file writable when no source control provider is active. */
	bool bShouldMakeWritableIfNoSourceControl = false;
};

/** Possible outcomes from a session persist. */
enum class EPersistStatus
{
	/** The packages were persisted correctly. */
	Success,
	/** We failed to persist the packages. */
	Failure,
	/** We are not allowed to persist. */
	NotAllowed
};

/** Result of a persist.*/
struct FPersistResult
{
	/** List of reasons for persist failure. */
	TArray<FText> FailureReasons;

	/** Success or failure of the session persist. */
	EPersistStatus PersistStatus = EPersistStatus::NotAllowed;
};
