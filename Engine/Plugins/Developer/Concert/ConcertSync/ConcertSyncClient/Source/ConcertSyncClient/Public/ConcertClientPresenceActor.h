// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "ConcertClientMovement.h"
#include "ConcertClientPresenceActor.generated.h"

class FStructOnScope;
class IConcertClientSession;
class FConcertClientPresenceMode;
class UStaticMeshComponent;
class UTextRenderComponent;
class UMaterialInstanceDynamic;

/**
 * A ConcertClientPresenceActor is a transient actor representing other client presences during a concert client session.
 */
UCLASS(Abstract, Transient, NotPlaceable, Blueprintable)
class AConcertClientPresenceActor : public AActor
{
	GENERATED_BODY()

public:
	AConcertClientPresenceActor(const FObjectInitializer& ObjectInitializer);

	/** AActor interface */
	virtual bool IsEditorOnly() const override final;

#if WITH_EDITOR
	virtual bool IsSelectable() const override final
	{
		return false;
	}

	virtual bool IsListedInSceneOutliner() const override final
	{
		return false;
	}
#endif

	/**
	 * Set the presence name.
	 * Called by the presence manager to set the display name of the client on the actor.
	 * @param InName the display name of the client represented by this actor.
	 */
	virtual void SetPresenceName(const FString& InName);

	/**
	 * Set the presence color.
	 * Called by the presence manager to set the color of the client on the actor.
	 * @param InColor the color of the client represented by this actor.
	 */
	virtual void SetPresenceColor(const FLinearColor& InColor);

	/**
	 * Handle presence event received from the session for this actor
	 * @param InEvent a concert presence event
	 */
	virtual void HandleEvent(const FStructOnScope& InEvent);

	/**
	 * Initialize the presence actor from a set of assets representing
	 * HMD, controllers and desktops
	 * @param InAssetContainer the asset container for controller, HMD and laser meshes
	 * @param DeviceType The device type this actor is representing.
	 */
	virtual void InitPresence(const class UConcertAssetContainer& InAssetContainer, FName DeviceType);

	virtual bool ShouldTickIfViewportsOnly() const override;

	virtual void Tick(float DeltaSeconds) override;

protected:
	/* The device type that this presence represent (i.e Oculus, Vive, Desktop) */
	UPROPERTY(BlueprintReadOnly, Category = "Rendering", meta = (AllowPrivateAccess = "true"))
	FName PresenceDeviceType;

	/** The camera mesh component to show visually where the camera is placed */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = "Rendering", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UStaticMeshComponent> PresenceMeshComponent;
	
	/** The text render component to display the associated client's name */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = "Rendering", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UTextRenderComponent> PresenceTextComponent;

	/** Dynamic material for the presence actor */
	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> PresenceMID;

	/** Dynamic material for the presence text */
	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> TextMID;

	/** Movement object to interpolate presence movement between update events.*/
	TOptional<FConcertClientMovement> PresenceMovement;

private:
	void UpdateVisibleInGameState();

};

