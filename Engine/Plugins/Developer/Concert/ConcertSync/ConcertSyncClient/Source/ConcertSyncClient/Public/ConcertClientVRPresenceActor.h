// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ConcertClientPresenceActor.h"
#include "Engine/Scene.h"
#include "ConcertClientVRPresenceActor.generated.h"

class UMaterialInstanceDynamic;
class USplineComponent;
class UStaticMeshComponent;

/**
  * A ConcertClientVRPresenceActor is a child of AConcertClientPresenceActor that is used to represent users in VR 
  */
UCLASS(Abstract, Transient, NotPlaceable, Blueprintable)
class AConcertClientVRPresenceActor : public AConcertClientPresenceActor
{
	GENERATED_UCLASS_BODY()

public:
	virtual void HandleEvent(const FStructOnScope& InEvent) override;

	virtual void InitPresence(const class UConcertAssetContainer& InAssetContainer, FName DeviceType) override;

	virtual void SetPresenceColor(const FLinearColor& InColor) override;

	virtual void Tick(float DeltaSeconds) override;

	/** The left controller mesh */
	UPROPERTY()
	TObjectPtr<UStaticMeshComponent> LeftControllerMeshComponent;

	/** The right controller mesh */
	UPROPERTY()
	TObjectPtr<UStaticMeshComponent> RightControllerMeshComponent;

	/** Dynamic material for the laser */
	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> LaserMid;

	/** Dynamic material for the laser */
	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> LaserCoreMid;

	UPROPERTY(BlueprintReadWrite, Category="Laser")
	float LaserThickness = 0.5f;

private:
	/** Updates all the segments of the curved laser */
	void UpdateSplineLaser(class USplineComponent* LaserSplineComponent, const TArray<class USplineMeshComponent*>& LaserSplineMeshComponents, const FVector& InStartLocation, const FVector& InEndLocation);

	void HideLeftController();
	void ShowLeftController();

	void HideRightController();
	void ShowRightController();

	void HideLeftLaser();
	void ShowLeftLaser();

	void HideRightLaser();
	void ShowRightLaser();

	/** Spline for the left laser pointer, if any */
	UPROPERTY()
	TObjectPtr<USplineComponent> LeftLaserSplineComponent;

	/** Spline meshes for the left curved laser, if any */
	UPROPERTY()
	TArray<TObjectPtr<USplineMeshComponent>> LeftLaserSplineMeshComponents;

	/** Spline for the right laser pointer, if any */
	UPROPERTY()
	TObjectPtr<USplineComponent> RightLaserSplineComponent;

	/** Spline meshes for the right curved laser, if any */
	UPROPERTY()
	TArray<TObjectPtr<USplineMeshComponent>> RightLaserSplineMeshComponents;

	UPROPERTY()
	bool bIsLeftControllerVisible;

	UPROPERTY()
	bool bIsRightControllerVisible;

	UPROPERTY()
	bool bIsLeftLaserVisible;

	UPROPERTY()
	bool bIsRightLaserVisible;

	/** Movement smoothing for left controller */
	TOptional<FConcertClientMovement> LeftControllerMovement;

	/** Movement smoothing for right controller */
	TOptional<FConcertClientMovement> RightControllerMovement;

	/** Movement smoothing for lasers start */
	TOptional<FConcertClientMovement> LeftLaserStartMovement;

	/** Movement smoothing for laser end */
	TOptional<FConcertClientMovement> LeftLaserEndMovement;

	/** Movement smoothing for lasers start */
	TOptional<FConcertClientMovement> RightLaserStartMovement;

	/** Movement smoothing for laser end */
	TOptional<FConcertClientMovement> RightLaserEndMovement;
};

