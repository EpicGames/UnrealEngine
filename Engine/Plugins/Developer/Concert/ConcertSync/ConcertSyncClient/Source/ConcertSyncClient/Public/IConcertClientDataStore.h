// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConcertMessages.h"
#include "ConcertDataStoreMessages.h"

/** The options flags used to register a key/value handler. */
enum class EConcertDataStoreChangeNotificationOptions : uint8
{
	/** No special options. */
	None = 0,

	/** If the key value already exists, immediately call the handler. This is recommended to avoid calling the Fetch/Register operations in the wrong order. */
	NotifyOnInitialValue = 1 << 0,

	/** If the register handler declares a value type that doesn't match the stored value type, still call the observer, but the reported value (TOptional<>) will not be set. */
	NotifyOnTypeMismatch = 1 << 1,
};

ENUM_CLASS_FLAGS(EConcertDataStoreChangeNotificationOptions)

/**
 * Interacts with a key/value store shared by one or more clients connected to a
 * Concert session. The store is like a TMap<FName, Blob>. The implementation
 * requires the 'Blob' to be defined as a USTRUCT() structure, which provides
 * the serialization mechanism. The system automatically wraps primitive types
 * like integers, floating, bool and strings into USTRUCT() for you. If you need
 * to store container types like TArray<>, TMap<> or TSet<> or custom types, you
 * will need to put them into a USTRUCT() struct similare to the example below
 * showing how to use custom types.
 *
 * The store is type safe, in the sense that a client cannot transmute the type of
 * a stored value into another type. For example if the value "foo" is an integer,
 * it cannot be transformed into a double later one.
 *
 * The store is used to share variables with other clients. For example, it can be
 * used for to manage a distributed counter like "cameraId" to uniquely number
 * cameras created concurrently by multiple users while editing a level.
 *
 * The store API returns TFuture to implement asynchronous or blocking operations.
 * While is far more easier to use the blocking operations model, i.e. waiting
 * on future to gets its result (TFuture::Get()) in the caller thread, it is
 * recommended to use the asynchronous API and use continuations. Since the store
 * implies network operation, expect latency and avoid waiting the response in
 * a thread like the game thread.
 *
 * To implement a sequence of operations using the store asynchronously inside a
 * single thread (game thread), it is recommended to implement it as a finite
 * state machine 'ticked' at each loop.
 *
 * <b>Example: Initialize a shared value.</b>
 * <br/>
 * The code snippet below shows how multiple clients can concurrently create or
 * sync a shared integer value to be ready to compare exchange it later to
 * generate a new unique id.
 *
 * @code
 * void MyClass::InitCameraIdAsync()
 * {
 *     FName Key(TEXT("CameraId")); // The shared variable name.
 *     int64 Value = 0; // The initial value if not existing yet.
 *
 *     // Try to fetch the specified key value (a basic type), if the key doesn't exist, add it with the specified value.
 *     GetDataStore().FetchOrAdd(Key, Value).Next([this, Key, Value](const TConcertDataStoreResult<int64>& Result)
 *     {
 *         // If the key was added or fetched.
 *         if (Result)
 *         {
 *             CameraId = Result.GetValue();
 *             bCameraIdAcquired = true;
 *         }
 *         else
 *         {
 *              // The key already existed, but the value was not a int64.
 *              check(Response.GetCode() == EConcertDataStoreResultCode::TypeMismatch);
 *         }
 *    });
 * @endcode
 *
 * <b>Example: Use custom types.</b>
 * <br/>
 * The code snippet below shows how a user can use a custom type with the data store. For
 * simplicity, the example block until the result is available and assumes the all operations
 * succeeded.
 *
 * @code
 * USTRUCT()
 * struct FPoint2D
 * {
 *     GENERATED_BODY()
 *
 *     UPROPERTY(VisibleAnywhere, Category = "SomeCategory")
 *     int32 x;
 *     UPROPERTY(VisibleAnywhere, Category = "SomeCategory")
 *     int32 y;
 * };
 *
 * USTRUCT()
 * struct FShape
 * {
 *     GENERATED_BODY()
 *
 *     UPROPERTY(VisibleAnywhere, Category = "SomeCategory")
 *     TArray<FPoint2D> Points;
 * };
 *
 * void Example()
 * {
 *     FName Key(TEXT("Point"));
 *     FPoint2D Position{0, 0}
 *     GetDataStore().FetchOrAdd(Key, Position).Get();
 *     UE_LOGF(MyLogCat, Display, "%d", Session->GetDataStore().FetchAs<FPoint2D>(Key).Get().GetValue().x); // Prints 0
 *     GetDataStore().CompareExchange(Key, Position, Point2D{10, 20}).Get();
 *     UE_LOGF(MyLogCat, Display, "%d", Session->GetDataStore().FetchAs<FPoint2D>(Key).Get().GetValue().x); // Prints 10
 *
 *     // Store a shape.
 *     FShape Shape;
 *     Shape.Points.Add(FPoint2D(0, 0));
 *     Shape.Points.Add(FPoint2D(10, 10));
 *     Shape.Points.Add(FPoint2D(0, 10));
 *     GetDataStore().FetchOrAdd(FName(TEXT("Triangle"), Shape).Get(); 
 * }
 * @endcode
 */
class IConcertClientDataStore
{
public:
	/** Destructor. */
	virtual ~IConcertClientDataStore() = default;

	/**
	 * Searches the store for the specified key, if not found, adds a new key/value pair, otherwise,
	 * if the stored value type matches the initial value type, fetches the stored value. The function
	 * accepts a USTRUCT() type or a supported basic type directly. To store complex types such as TArray<>
	 * TMap<> or TSet<>, wrap the type(s) in a USTRUCT().
	 *
	 * @tparam T Any USTRUCT() struct or a supported basic types (int8, uint8, int16, uint16, int32, uint32, int64, uint64, float, double, bool, FName, FText, FString).
	 * @param Key The value name to fetch or add.
	 * @param InitialValue The property value to store if the key doesn't already exist.
	 * @return Whether or not the key/value pair was added, fetched or the operation failed. The result code can be:
	 *     - EConcertDataStoreResultCode::Added if the key/value was inserted.
	 *     - EConcertDataStoreResultCode::Fetched if the key was already taken and type matched.
	 *     - EConcertDataStoreResultCode::TypeMismatch if the key was already taken but the value types did not match.
	 * @see TConcertDataStoreResult<T>::GetValue()
	 */
	template<typename T>
	TFuture<TConcertDataStoreResult<T>> FetchOrAdd(const FName& Key, const T& InitialValue)
	{
		// Use a private indirection to enable the rvalue to bind to a const lvalue. Taking the address of the rvalue
		// to call the virtual function (which take a const void*) fired a warning on VC++.
		return InternalFetchOrAdd<T>(Key, TConcertDataStoreType<T>::AsStructType(InitialValue)).Next([](const FConcertDataStoreResult& Result)
		{
			return TConcertDataStoreResult<T>(Result);
		});
	}

	/**
	 * Looks up the specified key, if found and types match, fetches the corresponding value. If the key is not found
	 * or the requested types doesn't match the stored type, the operation fails.
	 *
 	 * @tparam T Any USTRUCT() struct or a supported basic types (int8, uint8, int16, uint16, int32, uint32, int64, uint64, float, double, bool, FName, FText, FString).
	 * @param Key The value name to lookup.
	 * @return The operation result, as a future. The result code can be:
	 *     - EConcertDataStoreResultCode::Fetched if the key value was retrieved.
	 *     - EConcertDataStoreResultCode::NotFound if the key could not be found.
	 *     - EConcertDataStoreResultCode::TypeMismatch if the key was found, but the requested type did not match the stored type.
	 * @see TConcertDataStoreResult<T>::GetValue()
	 */
	template<typename T>
	TFuture<TConcertDataStoreResult<T>> FetchAs(const FName& Key) const
	{
		return InternalFetchAs(Key, TConcertDataStoreType<T>::StructType::StaticStruct(), TConcertDataStoreType<T>::GetFName()).Next([](const FConcertDataStoreResult& Result)
		{
			return TConcertDataStoreResult<T>(Result);
		});
	}

	/**
	 * Exchanges the stored value to \a Desired if a stored value corresponding to \a Key exists, has the same
	 * type and its value is equal to \a Expected, otherwise, the operation fails.
	 *
	 * @tparam T Any USTRUCT() struct or a supported basic types (int8, uint8, int16, uint16, int32, uint32, int64, uint64, float, double, bool, FName, FText, FString).
	 * @param Key The value name.
	 * @param Expected The value the caller expects to be stored.
	 * @param Desired The value the caller wants to store.
	 * @return Whether or not the key value was exchanged, as a future. The result code can be:
	 *     - EConcertDataStoreResultCode::Exchanged if the desired value was sucessfully exchanged and stored.
	 *     - EConcertDataStoreResultCode::Fetched if the stored value was not the expected one. The stored value was fetched instead.
	 *     - EConcertDataStoreResultCode::NotFound if the key could not be found.
	 *     - EConcertDataStoreResultCode::TypeMismatch if the stored data type did not match the expected/desired type.
	 * @see TConcertDataStoreResult<T>::GetValue()
	 */
	template<typename T>
	TFuture<TConcertDataStoreResult<T>> CompareExchange(const FName& Key, const T& Expected, const T& Desired)
	{
		// Use a private indirection to enable the rvalue to bind to a const lvalue. Taking the address of the rvalue
		// to call the virtual function (which take a const void*) fires a warning on VC++.
		return InternalCompareExchange<T>(Key, TConcertDataStoreType<T>::AsStructType(Expected), TConcertDataStoreType<T>::AsStructType(Desired)).Next([](const FConcertDataStoreResult& Result)
		{
			return TConcertDataStoreResult<T>(Result);
		});
	}

	/**
	 * Registers(or replaces) a handler invoked every time another client successfully adds or updates the specified key.
	 * The server pushes a notification for the added/updated key to all clients except the one that performed the change.
	 * By default, the data store immediately calls back the client if the key value is known. This is recommended to
	 * prevent calling FetchAs() and RegisterChangeNotificationHandler() in a non-safe order where a key/value could be
	 * missed if it appears between a failed fetch and the successful registration of a handler. By default, the handler
	 * will be called even if the key type expected by the client doesn't match the stored type. In such case, the handler
	 * optional value is not set.
	 *
	 * @param InKey The key to observe.
	 * @param Handler The function to call back with the latest value. See the EChangeNotificationOptions flags.
	 * @param Options A set of EChangeNotificationOptions flags.
	 */
	template<typename T>
	void RegisterChangeNotificationHandler(const FName& InKey, const TFunction<void(const FName&, TOptional<T>)>& Handler,
		EConcertDataStoreChangeNotificationOptions Options = EConcertDataStoreChangeNotificationOptions::NotifyOnInitialValue | EConcertDataStoreChangeNotificationOptions::NotifyOnTypeMismatch)
	{
		// Wraps the function using a template argument into a non-template one, so that it can be passed to a virtual function and stored into a map.
		auto HandlerWrapper = [Handler, Options](const FName& Key, const FConcertDataStore_StoreValue* Value)
		{
			// If types match.
			if (Value && TConcertDataStoreType<T>::GetFName() == Value->TypeName)
			{
				Handler(Key, TOptional<T>(Value->DeserializeUnchecked<T>()));
			}
			else if (EnumHasAnyFlags(Options, EConcertDataStoreChangeNotificationOptions::NotifyOnTypeMismatch))
			{
				// Type mismatch, call the function, but do not set the value.
				Handler(Key, TOptional<T>());
			}
		};

		InternalRegisterChangeNotificationHandler(InKey, TConcertDataStoreType<T>::GetFName(), HandlerWrapper, Options);
	}

	/**
	 * Unregisters the function callback corresponding to the specified key (if any) to stop receiving the key change notifications.
	 * @param Key The key for which the handler should be removed.
	 */
	void UnregisterChangeNotificationHander(const FName& Key)
	{
		InternalUnregisterChangeNotificationHandler(Key);
	}

protected:
	/** The function called back when the data store is updated by another client. */
	typedef TFunction<void(const FName&, const FConcertDataStore_StoreValue*)> FChangeNotificationHandler;
	
	/**
	 * Fetches or adds a key/value in the store.
	 * @param Key The value name.
	 * @param Type The value type.
	 * @param TypeName The value type name as returned by TConcertDataStoreType<T>::GetFName().
	 * @param Payload The USTRUCT structure pointer.
	 * @see IConcertClientDataStore::FetchOrAdd()
	 */
	virtual TFuture<FConcertDataStoreResult> InternalFetchOrAdd(const FName& Key, const UScriptStruct* Type, const FName& TypeName, const void* Payload) = 0;

	/**
	 * Fetches a key value from the store.
	 * @param Key The value name.
	 * @param Type The value type to read.
	 * @param TypeName The value type name as returned by TConcertDataStoreType<T>::GetFName().
	 * @see IConcertClientDataStore::FetchAs()
	 */
	virtual TFuture<FConcertDataStoreResult> InternalFetchAs(const FName& Key,const UScriptStruct* Type, const FName& TypeName) const = 0;

	/**
	 * Compares and exchanges a key value from the store.
	 * @param Key The value name.
	 * @param Type The value type.
	 * @param TypeName The value type name as returned by TConcertDataStoreType<T>::GetFName().
	 * @param Expected The USTRUCT structure pointer containing the expected value.
	 * @param Desired The USTRUCT structure pointer containing the desired value.
	 * @see IConcertClientDataStore::CompareExchange()
	 */
	virtual TFuture<FConcertDataStoreResult> InternalCompareExchange(const FName& Key, const UScriptStruct* Type, const FName& TypeName, const void* Expected, const void* Desired) = 0;

	/**
	 * Registers a delegate invoked when the specified key is added or modified.
	 * @param Key The name of the value to observe.
	 * @param TypeName The value type name as returned by TConcertDataStoreType<T>::GetFName().
	 * @param Handler The delegate function invoked when the specified key is added or modified.
	 * @param Options A set of options controlling if handler should be called or not in special situation.
	 * @see IConcertClientDataStore::RegisterChangeNotificationHandler()
	 */
	virtual void InternalRegisterChangeNotificationHandler(const FName& Key, const FName& TypeName, const FChangeNotificationHandler& Handler, EConcertDataStoreChangeNotificationOptions Options) = 0;

	/**
	 * Unregisters the delegate corresponding to the specified key (if any) to stop receiving the key change notifications.
	 * @param Key The key for which the handler should be removed.
	 */
	virtual void InternalUnregisterChangeNotificationHandler(const FName& Key) = 0;

private:
	template<typename T>
	TFuture<FConcertDataStoreResult> InternalFetchOrAdd(const FName& Key, const typename TConcertDataStoreType<T>::StructType& InitialValue)
	{
		// This calls the protected pure virtual function that must be implemented by the derived class.
		return InternalFetchOrAdd(Key, TConcertDataStoreType<T>::StructType::StaticStruct(), TConcertDataStoreType<T>::GetFName(),
			static_cast<const void*>(&InitialValue));
	}

	template<typename T>
	TFuture<FConcertDataStoreResult> InternalCompareExchange(const FName& Key, const typename TConcertDataStoreType<T>::StructType& Expected, const typename TConcertDataStoreType<T>::StructType& Desired)
	{
		// This calls the protected pure virtual function that must be implemented by the derived class.
		return InternalCompareExchange(Key, TConcertDataStoreType<T>::StructType::StaticStruct(), TConcertDataStoreType<T>::GetFName(),
			static_cast<const void*>(&Expected), static_cast<const void*>(&Desired));
	}
};
