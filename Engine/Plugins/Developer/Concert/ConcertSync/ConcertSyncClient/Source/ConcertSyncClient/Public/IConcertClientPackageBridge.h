// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"

class UPackage;
struct FConcertPackageInfo;

DECLARE_MULTICAST_DELEGATE_TwoParams(FOnConcertClientLocalPackageEvent, const FConcertPackageInfo&, const FString&);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnConcertClientLocalPackageDiscarded, UPackage*);

enum class EPackageFilterResult : uint8
{
	Include,
	Exclude,
	UseDefault
};

DECLARE_DELEGATE_RetVal_OneParam(EPackageFilterResult, FPackageFilterDelegate, const FConcertPackageInfo&);
DECLARE_DELEGATE_RetVal_OneParam(bool, FPackageHotReloadHintDelegate, const FConcertPackageInfo&);

/**
 * Bridge between the editor package events and Concert.
 * Deals with converting package update events to Concert package data.
 */
class IConcertClientPackageBridge
{
public:
	/** Scoped struct to ignore a local save */
	struct FScopedIgnoreLocalSave : private TGuardValue<bool>
	{
		FScopedIgnoreLocalSave(IConcertClientPackageBridge& InPackageBridge)
			: TGuardValue(InPackageBridge.GetIgnoreLocalSaveRef(), true)
		{
		}
	};

	/** Scoped struct to ignore a local discard */
	struct FScopedIgnoreLocalDiscard : private TGuardValue<bool>
	{
		FScopedIgnoreLocalDiscard(IConcertClientPackageBridge& InPackageBridge)
			: TGuardValue(InPackageBridge.GetIgnoreLocalDiscardRef(), true)
		{
		}
	};

	virtual ~IConcertClientPackageBridge() = default;

	/**
	 * Called when a local package event happens.
	 */
	virtual FOnConcertClientLocalPackageEvent& OnLocalPackageEvent() = 0;

	/**
	 * Called when a local package is discarded.
	 */
	virtual FOnConcertClientLocalPackageDiscarded& OnLocalPackageDiscarded() = 0;

	/** Register a pending package hot reload hint package payloads. */
	virtual void RegisterPackageHotReloadHint(FName PackageReloadHintName, FPackageHotReloadHintDelegate FilterHandle) = 0;

	/** Returns true if the package can be skipped for hot reload on client nodes.  */
	virtual bool CanSkipHotReload(const FConcertPackageInfo& PackageInfo) const = 0;

	/** Unregister package hot reload hint handler. */
	virtual void UnregisterPackageHotReloadHint(FName PackageReloadHintName) = 0;

	/** Register a package filter to exclude/include certain packagins from the session. */
	virtual void RegisterPackageFilter(FName FilterName, FPackageFilterDelegate FilterHandle) = 0;

	/** Returns true if the package should be filtered from the local sandbox.  */
	virtual EPackageFilterResult IsPackageFiltered(const FConcertPackageInfo& PackageInfo) const = 0;

	/** Unregister package filter for handling package filtering. */
	virtual void UnregisterPackageFilter(FName FilterName) = 0;

protected:
	/**
	 * Function to access the internal bool controlling whether local saves are currently being tracked.
	 * @note Exists to implement FScopedIgnoreLocalSave.
	 */
	virtual bool& GetIgnoreLocalSaveRef() = 0;

	/**
	 * Function to access the internal bool controlling whether local discards are currently being tracked.
	 * @note Exists to implement FScopedIgnoreLocalDiscard.
	 */
	virtual bool& GetIgnoreLocalDiscardRef() = 0;
};
