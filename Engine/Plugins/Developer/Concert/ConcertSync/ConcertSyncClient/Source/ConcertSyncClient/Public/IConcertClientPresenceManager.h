// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IConcertClientPresenceMode.h"

struct FConcertSessionClientInfo;
enum class EEditorPlayMode : uint8;

class IConcertClientPresenceManager
{
public:
	virtual ~IConcertClientPresenceManager() = default;

	/**
	 * Set the presence mode factory 
	 * The factory creates the presence mode for the manager
	 * @param InFactory the factory to use
	 */
	virtual void SetPresenceModeFactory(TSharedRef<IConcertClientPresenceModeFactory> InFactory) = 0;

	/** 
	 * Set whether presence is currently enabled and should be shown (unless hidden by other settings) 
	 */
	virtual void SetPresenceEnabled(const bool bIsEnabled = true) = 0;

	/** 
	 * Set presence visibility by name
	 * @param InDisplayName The display name of the presences we want to set the visibility on
	 * @param bVisibility The visibility flag to set
	 * @param bPropagateToAll Propagate the visibility change to other user.
	 * @note setting presence by name will persist through connection of client using this display name
	 */
	virtual void SetPresenceVisibility(const FString& InDisplayName, bool bVisibility, bool bPropagateToAll = false) = 0;

	/**
	 * Set presence visibility of the specified client endpoint id
	 * @param EndpointId the endpoint id of the client we we want to set the visibility on
	 * @param bVisibility The visibility flag to set
	 * @param bPropagateToAll Propagate the visibility change to other user.
	 */
	virtual void SetPresenceVisibility(const FGuid& EndpointId, bool bVisibility, bool bPropagateToAll = false) = 0;

	/**
	 * Get the presence actor transform for the specified client endpoint id
	 * @param EndpointId the endpoint id of the client we we want to retrieve the presence transform
	 * @return the transform of the client or the identity transform if the client wasn't found.
	 */
	virtual FTransform GetPresenceTransform(const FGuid& EndpointId) const = 0;

	/** 
	 * Jump (teleport) to another presence 
	 * @param InEndpointId The presence client endpoint id
	 */
	virtual void InitiateJumpToPresence(const FGuid& InEndpointId, const FTransform& InTransformOffet = FTransform::Identity) = 0;

	/**
	 * Returns the path to the UWorld object opened in the editor of the specified client endpoint.
	 * The information may be unavailable if the client was disconnected, the information hasn't replicated yet
	 * or the code was not compiled as part of the UE Editor. This function always returns the editor context world
	 * path, never the PIE/SIE context world path (which contains a UEDPIE_%d_ decoration embedded) even if the user
	 * is in PIE or SIE. For example, the PIE/SIE context world path would look like '/Game/UEDPIE_10_FooMap.FooMap'
	 * while the editor context world returned by this function looks like '/Game/FooMap.FooMap'.
	 * @param[in] EndpointId The endpoint of a client connected to the session (local or remote).
	 * @param[out] OutEditorPlayMode Indicates if the client corresponding to the end point is in PIE, SIE or simply editing.
	 * @return The path to the world being opened in the specified end point editor or an empty string if the information is not available.
	 */
	virtual FString GetPresenceWorldPath(const FGuid& EndpointId, EEditorPlayMode& OutEditorPlayMode) const = 0;
};