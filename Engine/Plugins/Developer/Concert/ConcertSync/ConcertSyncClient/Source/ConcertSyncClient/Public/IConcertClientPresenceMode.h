// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FReferenceCollector;
class IConcertClientSession;
struct FConcertClientInfo;

/** 
 * Base class for Multi-User PresenceMode.
 * PresenceMode are used to send avatar-specific presence events and cache 
 * avatar-related state for the current client. 
 * 
 * Adding a new presence avatar type requires the following:
 *
 *  1) Add a presence mode class inherited from IConcertClientBasePresenceMode to 
 *     send events and cache state, if needed, for the current client.
 *  2) Add a presence actor class to handle events and display the avatar for
 *     remote clients.
 */
class IConcertClientBasePresenceMode
{
public:
	virtual ~IConcertClientBasePresenceMode() = default;

	/** 
	 * Send events for this presence mode
	 * This will be called by the presence manager during tick.
	 * @param Session the session to send the mode events on.
	 */
	virtual void SendEvents(IConcertClientSession& Session) = 0;

	/** 
	 * Get the current presence transform
	 * @return the presence mode transform
	 */
	virtual FTransform GetTransform() = 0;

	/**
	 * Get the VR device type represented by this presence mode. 
	 * NAME_None for desktop or non vr presence mode.
	 * @return the vr device type
	 */
	virtual FName GetVRDeviceType() const = 0;

	/**
	 * Collects any reference to UObject the mode may have
	 * Called by the presence manager in its own AddReferencedObjects
	 */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) = 0;
};

/**
 * Interface for the presence mode factory
 * The presence manager uses a default implementation
 * but it is possible to change the factory to get different presence mode behavior than the default one.
 * @see IConcertClientPresenceManager
 */
class IConcertClientPresenceModeFactory
{
public:
	virtual ~IConcertClientPresenceModeFactory() = default;

	/**
	 * Indicate if the presence mode should be reset for recreation
	 * Called by the presence manager on end frame tick.
	 * @return true if the current presence mode in the presence manager should be reset
	 */
	virtual bool ShouldResetPresenceMode() const = 0;

	/**
	 * Create a presence mode.
	 * Called by the presence manager on end frame tick after resetting the presence mode if `ShouldResetPresenceMode` returned true.
	 * @return a unique pointer to a IConcertClientBasePresenceMode presence mode.
	 * @see ShouldResetPresenceMode
	 */
	virtual TUniquePtr<IConcertClientBasePresenceMode> CreatePresenceMode() = 0;

	/**
	 * Collects any reference to UObject the factory may have
	 * Called by the presence manager in its own AddReferencedObjects
	 */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) = 0;
};
