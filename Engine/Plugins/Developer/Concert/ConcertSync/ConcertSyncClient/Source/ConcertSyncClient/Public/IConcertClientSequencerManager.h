// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Sequencer manager that is held by the client sync module that keeps track of open sequencer UIs, regardless of whether a session is open or not
 * Events are registered to client sessions that will then operate on any tracked sequencer UIs
 */
class IConcertClientSequencerManager
{
public:
	virtual ~IConcertClientSequencerManager() = default;
	
	/**
	 * @return true if playback syncing across opened sequencer is enabled
	 */
	virtual bool IsSequencerPlaybackSyncEnabled() const = 0;

	/**
	 * Set the playback syncing option in Multi-User which syncs playback across user for opened sequencer.
	 * 
	 * @param bEnable The value to set for playback syncing of opened sequencer
	 */
	virtual void SetSequencerPlaybackSync(bool bEnable) = 0;

	/**
	 * @return true if unrelated timeline syncing across opened sequencer is enabled
	 */
	virtual bool IsUnrelatedSequencerTimelineSyncEnabled() const = 0;

	/**
	 * Set the unrelated timeline syncing option in Multi-User which syncs time from any remote sequence.
	 *
	 * @param bEnable The value to set for unrelated timeline syncing.
	 */
	virtual void SetUnrelatedSequencerTimelineSync(bool bEnable) = 0;

	/**
	 * @return true if the remote open option is enabled.
	 */
	virtual bool IsSequencerRemoteOpenEnabled() const = 0;

	/**
	 * @return true if the remote close option is enabled.
	 */
	virtual bool IsSequencerRemoteCloseEnabled() const = 0;

	/**
	 * @return true if the sequencer pacer is enabled.
	 */
	virtual bool IsSequencerPacingEnabled() const = 0;

	/**
	 * @return the current duration for the sequencer pacer.
	 */
	virtual float SequencerPacingDuration() const = 0;

	/**
	 * Set the remote open option in Multi-User
	 * which opens Sequencer for other users when this option is enabled on both user machines.
	 * 
	 * @param bEnable The value to set for the remote open option
	 */
	virtual void SetSequencerRemoteOpen(bool bEnable) = 0;

	/**
	 * Set the remote close option in Multi-User
	 * which closes Sequencer for this user when the sequence is closed by a remote user.
	 *
	 * @param bEnable The value to set for the remote close option
	 */
	virtual void SetSequencerRemoteClose(bool bEnable) = 0;

	/**
	 * Temporarily suspend sequencer pacing on this client.
	 */
	virtual void SuspendSequencerPacing() = 0;

	/**
	 * Resume sequencer pacing on this client if the console variable is enabled.
	 */
	virtual void ResumeSequencerPacing() = 0;

	/**
	 * Specifies the sequencer pacing behavior while in a session.
	 *
	 * @param bValue The value to be set for the sequencer pacing option
	 */
	virtual void SetSequencerPacingEnabled(bool bEnable) = 0;

	/**
	 * Sets the sequencer pacing duration value.
	 *
	 * @param Duration The spacing between state event messages.
	 */
	virtual void SetSequencerPacingDuration(float Duration) = 0;
};
