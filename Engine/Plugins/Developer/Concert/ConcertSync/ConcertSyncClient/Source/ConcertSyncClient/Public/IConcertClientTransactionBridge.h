// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "UObject/StructOnScope.h"
#include "ConcertTransactionEvents.h"
#include "IdentifierTable/ConcertIdentifierTable.h"

struct FConcertSessionVersionInfo;
class UObject;
class UPackage;

/**
 * Common data for a transaction.
 */
struct FConcertClientLocalTransactionCommonData
{
	FConcertClientLocalTransactionCommonData(const FText InTransactionTitle, const FGuid& InTransactionId, const FGuid& InOperationId, UObject* InPrimaryObject)
		: TransactionTitle(InTransactionTitle)
		, TransactionId(InTransactionId)
		, OperationId(InOperationId)
		, PrimaryObject(InPrimaryObject)
	{
	}

	FText TransactionTitle;
	FGuid TransactionId;
	FGuid OperationId;
	FWeakObjectPtr PrimaryObject;
	TArray<FName> ModifiedPackages;
	TArray<FConcertObjectId> ExcludedObjectUpdates;
	bool bIsExcluded = false;
};

/**
 * Snapshot data for a transaction.
 */
struct FConcertClientLocalTransactionSnapshotData
{
	TArray<FConcertExportedObject> SnapshotObjectUpdates;
};

/**
 * Finalized data for a transaction.
 */
struct FConcertClientLocalTransactionFinalizedData
{
	FConcertLocalIdentifierTable FinalizedLocalIdentifierTable;
	TArray<FConcertExportedObject> FinalizedObjectUpdates;
	bool bWasCanceled = false;
};

enum class ETransactionNotification
{
	Begin,
	End,
};

/**
 * Data holding information about conflict that occurred while processing an inbound transaction.
 */
struct FConcertConflictDescriptionBase
{
	FConcertConflictDescriptionBase() = default;
	virtual ~FConcertConflictDescriptionBase() = default;

	virtual FText GetConflictDetails() const
	{
		return FText();
	};
	virtual FText GetConflictTitle() const
	{
		return FText();
	};
};

struct FConcertTransactionFilterArgs
{
	UObject* ObjectToFilter;
	UPackage* Package;
	const FTransactionObjectEvent& TransactionEvent;

	FConcertTransactionFilterArgs(UObject* InObjectToFilter, UPackage* InPackage, const FTransactionObjectEvent& InTransactionEvent)
		: ObjectToFilter(InObjectToFilter)
		, Package(InPackage)
		, TransactionEvent(InTransactionEvent)
	{}
};

DECLARE_MULTICAST_DELEGATE_TwoParams(FOnApplyTransaction, ETransactionNotification, const bool bIsSnapshot);
UE_DEPRECATED(5.5, "Use FOnFilterTransactionDelegate instead.")
DECLARE_DELEGATE_RetVal_TwoParams(ETransactionFilterResult, FTransactionFilterDelegate, UObject*, UPackage*);
DECLARE_DELEGATE_RetVal_OneParam(ETransactionFilterResult, FOnFilterTransactionDelegate, const FConcertTransactionFilterArgs&);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnConcertClientLocalTransactionSnapshot, const FConcertClientLocalTransactionCommonData&, const FConcertClientLocalTransactionSnapshotData&);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnConcertClientLocalTransactionFinalized, const FConcertClientLocalTransactionCommonData&, const FConcertClientLocalTransactionFinalizedData&);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnConcertConflictResolutionForPendingSend, const FConcertConflictDescriptionBase& ConflictDescription);

/**
 * Bridge between the editor transaction system and Concert.
 * Deals with converting local ongoing transactions to Concert transaction data, 
 * and applying remote Concert transaction data onto this local instance.
 */
class IConcertClientTransactionBridge
{
public:
	/**
	 * Create a new instance of the concrete implementation of this class. 
	 */
	CONCERTSYNCCLIENT_API static TUniquePtr<IConcertClientTransactionBridge> NewInstance();

	/** Scoped struct to ignore a local transaction */
	struct FScopedIgnoreLocalTransaction : private TGuardValue<bool>
	{
		FScopedIgnoreLocalTransaction(IConcertClientTransactionBridge& InTransactionBridge)
			: TGuardValue(InTransactionBridge.GetIgnoreLocalTransactionsRef(), true)
		{
		}
	};

	virtual ~IConcertClientTransactionBridge() = default;

	/**
	 * Set whether or not to include editor-only properties when serializing object and property changes.
	 * @note This is set to true by default.
	 */
	virtual void SetIncludeEditorOnlyProperties(const bool InIncludeEditorOnlyProperties) = 0;

	/**
	 * Set whether to include non-property object data in updates, or whether to only include property changes.
	 * @note This is set to true by default.
	 */
	virtual void SetIncludeNonPropertyObjectData(const bool InIncludeNonPropertyObjectData) = 0;

	/**
	 * Set whether to include object changes that have been generated via a transaction annotation 
	 * (where possible), or whether to send the entire transaction annotation blob instead.
	 * @note This is set to UConcertSyncConfig::bIncludeAnnotationObjectChanges by default.
	 */
	virtual void SetIncludeAnnotationObjectChanges(const bool InIncludeAnnotationObjectChanges) = 0;

	/**
	 * Called when an ongoing transaction is updated via a snapshot.
	 * @note This is called during end-frame processing.
	 */
	virtual FOnConcertClientLocalTransactionSnapshot& OnLocalTransactionSnapshot() = 0;

	/**
	 * Called when an transaction is finalized.
	 * @note This is called during end-frame processing.
	 */
	virtual FOnConcertClientLocalTransactionFinalized& OnLocalTransactionFinalized() = 0;

	/** Callback when a conflict occurs on pending transactions. */
	virtual FOnConcertConflictResolutionForPendingSend& OnConflictResolutionForPendingSend() = 0;

	/**
	 * Can we currently apply a remote transaction event to this local instance?
	 * @return True if we can apply a remote transaction, false otherwise.
	 */
	virtual bool CanApplyRemoteTransaction() const = 0;

	/**
	 * Notification of an application of a transaction. This will tell the user if the transaction
	 * originates as a snapshot or is a finalized snapshot message.
	 */
	virtual FOnApplyTransaction& OnApplyTransaction() = 0;

	/** Extended information that describes what changed for a single object within a remote transaction */
	struct FRemoteTransactionObjectResult
	{
		/** Final path of the object after the given remote transaction was applied to it */
		FSoftObjectPath ObjectPath;

		/** Index into the FConcertTransactionEventBase::ExportedObjects array in the remote transaction that was applied to this object */
		int32 ObjectIndex = INDEX_NONE;

		/** True if the object was created by this remote transaction, or false if it already existed */
		bool bNewlyCreated : 1 = false;
	};

	/** Extended information that describes what changed after applying a remote transaction */
	struct FRemoteTransactionResult
	{
		TArray<FRemoteTransactionObjectResult> Objects;
	};

	/**
	 * Apply a remote transaction event to this local instance.
	 * @param InEvent					The event to apply.
	 * @param InVersionInfo				The version information for the serialized data in the event, or null if the event should be serialized using the compiled in version info.
	 * @param InPackagesToProcess		The list of packages to apply changes for, or an empty array to apply all changes.
	 * @param InLocalIdentifierTablePtr The local identifier table for the event data (if any).
	 * @param bIsSnapshot				True if this transaction event was a snapshot rather than a finalized transaction.
	 * @param ResultInfo				Optional extended result information to fill.
	 */
	virtual void ApplyRemoteTransaction(const FConcertTransactionEventBase& InEvent, const FConcertSessionVersionInfo* InVersionInfo, const TArray<FName>& InPackagesToProcess, const FConcertLocalIdentifierTable* InLocalIdentifierTablePtr, const bool bIsSnapshot, FRemoteTransactionResult* ResultInfo = nullptr) = 0;

	/**
	 * Apply a remote transaction event to this local instance.
	 * @param InEvent					The event to apply.
	 * @param InVersionInfo				The version information for the serialized data in the event, or null if the event should be serialized using the compiled in version info.
	 * @param InPackagesToProcess		The list of packages to apply changes for, or an empty array to apply all changes.
	 * @param InLocalIdentifierTablePtr The local identifier table for the event data (if any).
	 * @param bIsSnapshot				True if this transaction event was a snapshot rather than a finalized transaction.
	 * @param ConcertSyncWorldRemapper	Remapper to use in the case the current world is different from the one sent in the transaction.
	 * @param ResultInfo				Optional extended result information to fill.
	 */
	virtual void ApplyRemoteTransaction(const FConcertTransactionEventBase& InEvent, const FConcertSessionVersionInfo* InVersionInfo, const TArray<FName>& InPackagesToProcess, const FConcertLocalIdentifierTable* InLocalIdentifierTablePtr, const bool bIsSnapshot, const class FConcertSyncWorldRemapper& ConcertSyncWorldRemapper, FRemoteTransactionResult* ResultInfo = nullptr) = 0;

	/** Callback to register delegate for handling transaction events */
	UE_DEPRECATED(5.5, "Use the version of RegisterTransactionFilter accepting FOnFilterTransactionDelegate instead.")
	void RegisterTransactionFilter(
		FName FilterName,
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FTransactionFilterDelegate FilterHandle
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		);
	/** Callback to register delegate for handling transaction events */
	virtual void RegisterTransactionFilter(FName FilterName, FOnFilterTransactionDelegate FilterDelegate) = 0;

	/** Callback to register delegate for handling transaction events */
	virtual void UnregisterTransactionFilter(FName FilterName) = 0;

	/**
	 * Bind to the transaction system to receive transaction events.  In past versions of IConcertClientTransactionBridge,
	 * this binding happened automatically.  To prevent possible performance impact from handling editor transactions when the bridge is
	 * not needed, users now have to indicate when they want to start the bridge which binds to the underlying transaction system.
	 */
	virtual void StartBridge() = 0;

	/**
	 * Stop briding the transaction system.  After this call the bridge will unbind from any handlers to the underlying transaction system and
	 * transaction events will not be processed. 
	 */
	virtual void StopBridge() = 0;

protected:
	/**
	 * Function to access the internal bool controlling whether local transactions are currently being tracked.
	 * @note Exists to implement FScopedIgnoreLocalTransaction.
	 */
	virtual bool& GetIgnoreLocalTransactionsRef() = 0;
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
inline void IConcertClientTransactionBridge::RegisterTransactionFilter(
	FName FilterName,
	FTransactionFilterDelegate FilterHandle
	)
{
	RegisterTransactionFilter(FilterName, FOnFilterTransactionDelegate::CreateLambda([FilterHandle = MoveTemp(FilterHandle)](const FConcertTransactionFilterArgs& FilterArgs)
	{
		return FilterHandle.Execute(FilterArgs.ObjectToFilter, FilterArgs.Package);
	}));
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
