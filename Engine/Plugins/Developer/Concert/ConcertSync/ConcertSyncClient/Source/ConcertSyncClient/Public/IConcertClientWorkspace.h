// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Async/Future.h"
#include "UObject/StructOnScope.h"
#include "ConcertWorkspaceMessages.h"
#include "ConcertSyncSessionTypes.h"

#include "ConcertClientPersistData.h"

class ISourceControlProvider;
class IConcertClientSession;
class IConcertClientDataStore;

struct FConcertSyncReplicationEvent;

DECLARE_DELEGATE_RetVal(bool, FCanFinalizeWorkspaceDelegate);
DECLARE_DELEGATE_RetVal(bool, FCanProcessPendingPackages);

DECLARE_MULTICAST_DELEGATE(FOnWorkspaceSynchronized);
DECLARE_MULTICAST_DELEGATE(FOnFinalizeWorkspaceSyncCompleted);
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnActivityAddedOrUpdated, const FConcertClientInfo&/*InClientInfo*/, const FConcertSyncActivity&/*InActivity*/, const FStructOnScope&/*InActivitySummary*/);

class IConcertClientWorkspace
{
public:
	/**
	 * Get the associated session.
	 */
	virtual IConcertClientSession& GetSession() const = 0;

	/**
	 * @return the client id this workspace uses to lock resources.
	 */
	virtual FGuid GetWorkspaceLockId() const = 0;

	/**
	 * @return a valid client id of the owner of this resource lock or an invalid id if unlocked
	 */
	virtual FGuid GetResourceLockId(const FName InResourceName) const = 0;

	/**
	 * Verify if resources are locked by a particular client
	 * @param ResourceNames list of resources path to verify
	 * @param ClientId the client id to verify
	 * @return true if all resources in ResourceNames are locked by ClientId
	 * @note passing an invalid client id will return true if all resources are unlocked
	 */
	virtual bool AreResourcesLockedBy(TArrayView<const FName> ResourceNames, const FGuid& ClientId) = 0;

	/**
	 * Attempt to lock the given resource.
	 * @note Passing force will always assign the lock to the given endpoint, even if currently locked by another.
	 * @return True if the resource was locked (or already locked by the given endpoint), false otherwise.
	 */
	virtual TFuture<FConcertResourceLockResponse> LockResources(TArray<FName> InResourceName) = 0;

	/**
	 * Attempt to unlock the given resource.
	 * @note Passing force will always clear, even if currently locked by another endpoint.
	 * @return True if the resource was unlocked, false otherwise.
	 */
	virtual TFuture<FConcertResourceLockResponse> UnlockResources(TArray<FName> InResourceName) = 0;

	/**
	 * Tell if a workspace contains session changes.
	 * @return True if the session contains any changes.
	 */
	virtual bool HasSessionChanges() const = 0;

	/**
	 * Gather assets changes that happened on the workspace in this session.
	 * @param IgnorePersisted if true will not return packages which have already been persisted in their current state.
	 * @return a list of package names that were modified during the session.
	 */
	virtual TArray<FName> GatherSessionChanges(bool IgnorePersisted = true) = 0;

	/**
	 * Returns the full path to the package if it is a valid package session change.
	 * @param PackageName the package to check.
	 * @return the full path to the package if it is valid.
	 */
	virtual TOptional<FString> GetValidPackageSessionPath(FName PackageName) const = 0;

	/** Persist the session changes from the package list and prepare it for source control submission */
	virtual FPersistResult PersistSessionChanges(FPersistParameters InParam) = 0;

	/**
	 * Get Activities from the session.
	 * @param FirstActivityIdToFetch The ID at which to start fetching activities.
	 * @param MaxNumActivities The maximum number of activities to fetch.
	 * @param OutEndpointClientInfoMap The client info for the activities fetched.
	 * @param OutActivities the activities fetched.
	 */
	virtual void GetActivities(const int64 FirstActivityIdToFetch, const int64 MaxNumActivities, TMap<FGuid, FConcertClientInfo>& OutEndpointClientInfoMap, TArray<FConcertSessionActivity>& OutActivities) const = 0;

	/**
	 * Get the ID of the last activity in the session.
	 */
	virtual int64 GetLastActivityId() const = 0;

	/**
	 * @return the delegate called every time an activity is added to or updated in the session.
	 */
	virtual FOnActivityAddedOrUpdated& OnActivityAddedOrUpdated() = 0;

	/**
	 * Indicate if an asset package is supported for live transactions.
	 *
	 * @param InAssetPackage The package to check
	 * @return true if we support live transactions for objects inside the package
	 */
	virtual bool HasLiveTransactionSupport(class UPackage* InPackage) const = 0;

	/**
	 * Indicate if package dirty event should be ignored for a package
	 * @param InPackage The package to check
	 * @return true if dirty event should be ignored for said package.
	 */
	virtual bool ShouldIgnorePackageDirtyEvent(class UPackage* InPackage) const = 0;
	
	/**
	 * Lookup the specified transaction event.
	 * @param[in] TransactionEventId ID of the transaction to look for.
	 * @param[out] OutTransactionEvent The transaction corresponding to TransactionEventId if found.
	 * @param[in] bMetaDataOnly True to extract the event meta-data only (title, ids, etc), false to get the full transaction data (to reapply/inspect it).
	 * @return Whether the transaction event was found and requested data available. If bMetaDataOnly is false and the event was partially
	 *         synced (because it was superseded by another one), the function returns false.
	 * @see FindOrRequestTransactionEvent() if the full transaction data is required.
	 * @note This function is more efficient to use if only the meta data is required.
	 */
	virtual bool FindTransactionEvent(const int64 TransactionEventId, FConcertSyncTransactionEvent& OutTransactionEvent, const bool bMetaDataOnly) const = 0;

	/**
	 * Lookup the specified transaction events. By default, some transaction activities are partially synced (only the summary) when the system detects that
	 * the transaction was superseded by another one.
	 * @param[in] TransactionEventId ID of the transaction to look for.
	 * @param[in] bMetaDataOnly True to extract the event meta-data only (title, IDs, etc), false to get the full transaction data (to reapply/inspect it).
	 * @return A future containing the event, if found. If bMetaDataOnly is false and the event was partially synced (because it was superseded
	 *         by another one) the function will perform an asynchronous server request to get the transaction data.
	 * @note If only the meta data is required, consider using FindTransactionEvent() as it is more efficient.
	 */
	virtual TFuture<TOptional<FConcertSyncTransactionEvent>> FindOrRequestTransactionEvent(const int64 TransactionEventId, const bool bMetaDataOnly) = 0;

	/**
	 * Lookup the specified package event.
	 * @param[in] PackageEventId ID of the package to look for.
	 * @param[out] OutPackageEvent Information about the package except the package data itself.
	 * @return Whether the package event meta data was found.
	 */
	virtual bool FindPackageEvent(const int64 PackageEventId, FConcertSyncPackageEventMetaData& OutPackageEvent) const = 0;

	/**
	 * Lookup the specified replication event.
	 * @param[in] ReplicationEventId ID of the replication event to look for.
	 * @param[out] OutReplicationEvent The replication event corresponding to ReplicationEventId if found.
	 * @return Whether the replication event was found and requested data available. 
	 */
	virtual bool FindReplicationEvent(const int64 ReplicationEventId, FConcertSyncReplicationEvent& OutReplicationEvent) const = 0;

	/**
	 * @return the delegate called every time the workspace is synced. This is when the server has indicated that all
	 * activities have been sent and ready to be finalized in the current workspace.
	 */
	virtual FOnWorkspaceSynchronized& OnWorkspaceSynchronized() = 0;

	/**
	 * @return The delegate called after a workspace has been completely synced and finalized.  All transactions are posted and packages have been loaded.
	 */
	virtual FOnFinalizeWorkspaceSyncCompleted& OnFinalizeWorkspaceSyncCompleted() = 0;

	/**
	 * This delegate allows user to defer the finalization of a sync workspace. This is for situtations where multiple
	 * client nodes need to be finalized at the same point in time.  The delegate function should return true when workspace
	 * synchronization is allowed and it will be called on OnEndFrame() of the tick loop.
	 *
	 * @param InDelegateName the identifier for the provided delegate.
	 * @param InDelegate the delegate to use to query if finalize workspace
	 */
	virtual void AddWorkspaceFinalizeDelegate(FName InDelegateName, FCanFinalizeWorkspaceDelegate InDelegate) = 0;

	/**
	   Remove the attached named delegate for workspace finalization.
	   @param InDelegateName identifying name for the delegate.
	 */
	virtual void RemoveWorkspaceFinalizeDelegate(FName InDelegateName) = 0;

	/**
	 * This delegate allows user to defer the finalization of a package updates. This is for situtations where multiple
	 * client nodes need finalize their package updates together. The delegate function should return true when workspace
	 * synchronization is allowed and it will be called on OnEndFrame() of the tick loop.
	 *
	 * @param InDelegateName the identifier for the provided delegate.
	 * @param InDelegate the delegate to use to query.
	 */
	virtual void AddWorkspaceCanProcessPackagesDelegate(FName InDelegateName, FCanProcessPendingPackages Delegate) = 0;

	/**
	   Remove the attached named delegate for package synchronization.
	   @param InDelegateName identifying name for the delegate.
	 */
	virtual void RemoveWorkspaceCanProcessPackagesDelegate(FName InDelegateName) = 0;

	/**
	 * Returns true if the current named package is being reloaded by the package manager.
	 */
	virtual bool IsReloadingPackage(FName PackageName) const = 0;

	/**
	 * @return the key/value store shared by all clients.
	 */
	virtual IConcertClientDataStore& GetDataStore() = 0;

	/**
	 * Returns true if the specified asset has unsaved modifications from any other client than the one corresponding
	 * to this workspace client and possibly returns more information about those other clients.
	 * @param[in] AssetName The asset name.
	 * @param[out] OutOtherClientsWithModifNum If not null, will contain how many other client(s) have modified the specified package.
	 * @param[out] OutOtherClientsWithModifInfo If not null, will contain the other client(s) who modified the packages, up to OtherClientsWithModifMaxFetchNum.
	 * @param[in] OtherClientsWithModifMaxFetchNum The maximum number of client info to store in OutOtherClientsWithModifInfo if the latter is not null.
	 */
	virtual bool IsAssetModifiedByOtherClients(const FName& AssetName, int32* OutOtherClientsWithModifNum = nullptr, TArray<FConcertClientInfo>* OutOtherClientsWithModifInfo = nullptr, int32 OtherClientsWithModifMaxFetchNum = 0) const = 0;

	/**
	 * Controls whether the activities emitted to the server through this workspace are marked as 'ignored on restore'. By default, all activities are marked as 'restorable'. When
	 * the workspace 'ignore' state is true, the events emitted are recorded by the server, but are marked as 'should not restore'. Non-restorable activities are used for inspection.
	 * @note This was implemented to prevent multi-users transactions from being restored by disaster recovery in case of a crash during as multi-user session.
	 * @param bIgnore Whether all further events emitted are marked as 'ignored on restore'.
	 */
	virtual void SetIgnoreOnRestoreFlagForEmittedActivities(bool bIgnore) = 0;
};
