// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IConcertClientTransactionBridge.h"
#include "IConcertModule.h"
#include "IConcertClientModule.h"
#include "ConcertActionDefinition.h"
#include "ConcertSyncSessionFlags.h"

class IConcertFileSharingService;
class IConcertClientPackageBridge;
class IConcertClientPresenceManager;
class IConcertClientReplicationManager;
class IConcertClientReplicationBridge;
class IConcertClientSequencerManager;
class IConcertClientTransactionBridge;
class IConcertClientWorkspace;
class IConcertSyncClient;
class UConcertClientConfig;
struct FConcertSessionClientInfo;

DECLARE_MULTICAST_DELEGATE_OneParam(FOnConcertClientWorkspaceStartupOrShutdown, const TSharedPtr<IConcertClientWorkspace>& /** InClientWorkspace */ );
DECLARE_MULTICAST_DELEGATE_OneParam(FOnConcertClientSyncSessionStartupOrShutdown, const IConcertSyncClient* /* InSyncClient */ );

/** Defines the supported editor play modes. */
enum class EEditorPlayMode : uint8
{
	/** The editor is not in any play mode. */
	None,

	/** The editor is in "Play In Editor" play mode. */
	PIE,

	/** The editor is in "Simulate In Editor" play mode. */
	SIE,
};

/**
 * Interface for a Concert Sync Client.
 */
class IConcertSyncClient
{
public:
	virtual ~IConcertSyncClient() = default;

	/** Start this Concert Sync Client using the given config */
	virtual void Startup(const UConcertClientConfig* InClientConfig, const EConcertSyncSessionFlags InSessionFlags) = 0;

	/** Stop this Concert Sync Client */
	virtual void Shutdown() = 0;

	/** Get the current client */
	virtual IConcertClientRef GetConcertClient() const = 0;

	/** Get the current session client workspace, if any. */
	virtual TSharedPtr<IConcertClientWorkspace> GetWorkspace() const = 0;


	
	/** 
	 * Get the current session presence manager, if any.
	 * @note that pointer shouldn't be stored and always accessed through this client.
	 */
	virtual IConcertClientPresenceManager* GetPresenceManager() const = 0;

	/**
	 * Get the current session sequencer manager, if any.
	 * @note that pointer shouldn't be stored and always accessed through this client.
	 */
	virtual IConcertClientSequencerManager* GetSequencerManager() const = 0;

	/**
	 * Get the current session replication manager, if any.
	 * @note that pointer shouldn't be stored and always accessed through this client.
	 */
	virtual IConcertClientReplicationManager* GetReplicationManager() const = 0;


	
	/** Get the delegate called on every workspace startup. */
	virtual FOnConcertClientWorkspaceStartupOrShutdown& OnWorkspaceStartup() = 0;

	/** Get the delegate called on every workspace shutdown. */
	virtual FOnConcertClientWorkspaceStartupOrShutdown& OnWorkspaceShutdown() = 0;

	/** Get the delegate called just before the workspace gets created. */
	virtual FOnConcertClientSyncSessionStartupOrShutdown& OnSyncSessionStartup() = 0;

	/** Get the delegate called just after the workspace gets deleted. */
	virtual FOnConcertClientSyncSessionStartupOrShutdown& OnSyncSessionShutdown() = 0;


	
	/** Persist specific packages */
	virtual void PersistSpecificChanges(TArrayView<const FName> Packages) = 0;

	/** Persist all session changes and prepare the files for source control submission. */
	virtual void PersistAllSessionChanges() = 0;

	/**
	 * Queries the list of opaque actions that could be performed on the specified client session, like turning the client presence visibily on/off in the map.
	 * The actions are usually mapped to buttons and are a way to provide extra functionalities while keeping the implementation hidden.
	 * @param InClientInfo The client for which the actions are queried.
	 * @param OutActions The available actions, if any.
	 */
	virtual void GetSessionClientActions(const FConcertSessionClientInfo& InClientInfo, TArray<FConcertActionDefinition>& OutActions) const = 0;

	/** Set the file sharing service, enabling the server to work with large files. The client and the server must use compatible sharing services. The service must be set prior establishing a connection. */
	virtual void SetFileSharingService(TSharedPtr<IConcertFileSharingService> InFileSharingService) = 0;


	
	/** Get the current transaction bridge for this client */
	virtual IConcertClientTransactionBridge* GetTransactionBridge() const = 0;

	/** Get the current package bridge for this client. */
	virtual IConcertClientPackageBridge* GetPackageBridge() const = 0;

	/** Get the current replication bridge for this client. */
	virtual IConcertClientReplicationBridge* GetReplicationBridge() const = 0;
};
