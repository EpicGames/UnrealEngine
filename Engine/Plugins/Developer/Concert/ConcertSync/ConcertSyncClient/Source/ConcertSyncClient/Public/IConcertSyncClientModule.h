// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class IConcertClientPackageBridge;
class IConcertClientReplicationBridge;
class IConcertClientTransactionBridge;
class IConcertSyncClient;
class UConcertClientConfig;

DECLARE_MULTICAST_DELEGATE_OneParam(FOnConcertClientCreated, TSharedRef<IConcertSyncClient>);

class IConcertClientReplicationBridge;

/**
 * Interface for the Concert Sync Client module.
 */
class IConcertSyncClientModule : public IModuleInterface
{
public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IConcertSyncClientModule& Get()
	{
		static const FName ModuleName = "ConcertSyncClient";
		return FModuleManager::LoadModuleChecked<IConcertSyncClientModule>(ModuleName);
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() during shutdown if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		static const FName ModuleName = "ConcertSyncClient";
		return FModuleManager::Get().IsModuleLoaded(ModuleName);
	}

	/**
	 * Parse command line client settings and apply them
	 * @param CommandLine the application command line arguments
	 * @return The client settings, modified or not by the command line
	 */
	virtual UConcertClientConfig* ParseClientSettings(const TCHAR* CommandLine) = 0;

	/**
	 * Create a sync client that will perform a certain role (eg, MultiUser, DisasterRecovery, etc)
	 * @param InRole The role to create
	 * @return The client
	 */
	virtual TSharedRef<IConcertSyncClient> CreateClient(const FString& InRole) = 0;

	/** Get the bridge between the editor package system and Concert. */
	virtual IConcertClientPackageBridge& GetPackageBridge() = 0;

	/** Get the bridge between the editor transaction system and Concert. */
	virtual IConcertClientTransactionBridge& GetTransactionBridge() = 0;

	/** Gets the bridge between the editor UObjects and Concert. */
	virtual IConcertClientReplicationBridge& GetReplicationBridge() = 0;

	/** Returns the list of active clients. */
	virtual TArray<TSharedRef<IConcertSyncClient>> GetClients() const = 0;

	/** Find a concert sync client. */
	virtual TSharedPtr<IConcertSyncClient> GetClient(const FString& InRole) const = 0;

	/** Delegate invoked when a new concert sync client is created. */
	virtual FOnConcertClientCreated& OnClientCreated() = 0;
};
