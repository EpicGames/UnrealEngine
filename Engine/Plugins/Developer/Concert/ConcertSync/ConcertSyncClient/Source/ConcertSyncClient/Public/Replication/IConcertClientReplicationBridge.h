// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"

class UObject;
struct FConcertReplication_ObjectReplicationEvent;
struct FSoftObjectPath;

DECLARE_MULTICAST_DELEGATE_OneParam(FConcertClientReplicationBridgeObjectEvent, UObject&);
DECLARE_MULTICAST_DELEGATE_OneParam(FConcertClientReplicationBridgeObjectPathEvent, const FSoftObjectPath&);

/**
 * A session-independent object bridging Unreal Engine with Concert.
 * Responsible for tracking the lifetime of replicated objects, i.e. level (un)load, etc.
 *
 * An UObject is called "available" if it is in a state suitable for replicating; otherwise it is "unavailable".
 * It is "tracked" if this bridge is keep track of whether is available; otherwise is is "untracked".
 * The moment a tracked object is determined to be available it is said to "have been discovered". The moment is
 * determined to be unavailable, it is said to "have become hidden".
 * 
 * In the editor an UObject should be considered
 *	- available:
 *		- in an UWorld the client currently has open,
 *		- in an active data layer,
 *	- unavailable 
 *		- in a non-loaded UWorld,
 *		- in an unloaded or loaded data layer (remember that loaded means not visible),
 *		- does not have an UWorld as outer (TODO: in the future this restriction could be lifted - there is no technical reason for it - VP workflows just do not need it right now)
 */
class IConcertClientReplicationBridge
{
public:

	virtual ~IConcertClientReplicationBridge() = default;

	/**
	 * Starts tracking a list of objects, possibly triggering OnObjectDiscovered immediately.
	 * 
	 * Increments a tracking counter for how often this object was requested to be tracked;
	 * there may be multiple independent systems requiring tracking.
	 */
	virtual void PushTrackedObjects(TArrayView<const FSoftObjectPath> TrackedObjects) = 0;
	/**
	 * Stops tracking a list of objects, possibly triggering OnObjectRemoved immediately.
	 * Decrements the tracking counter. If it reaches 0, the object is no longer tracked;
	 * there may be multiple independent systems requiring tracking.
	 */
	virtual void PopTrackedObjects(TArrayView<const FSoftObjectPath> TrackedObjects) = 0;

	/**
	 * More lightweight version of FindObjectIfAvailable which does not force the UObject to be loaded, if it is
	 * determined to be available.
	 * This function should be called instead of FindAndTrackObjectIfAvailable if the UObject instance is not needed.
	 *
	 * This function will not discover nor hide any UObject (that means broadcast OnObjectDiscovered or OnObjectHidden),
	 * unless it is tracked, in which case it can.
	 * 
	 * @param Path The object to find
	 * @return Whether a call to FindObjectIfAvailable would return a valid object.
	 */
	virtual bool IsObjectAvailable(const FSoftObjectPath& Path) = 0;

	/**
	 * Obtains the object if it is available. This does not load the object. 
	 * 
	 * This function will not discover nor hide any UObject (that means broadcast OnObjectDiscovered or OnObjectHidden),
	 * unless it is tracked, in which case it can.
	 * 
	 * @param Path The object to find
	 * @return The object if it was valid or nullptr
	 */
	virtual UObject* FindObjectIfAvailable(const FSoftObjectPath& Path) = 0;
	
	/** Event for when a replicatable object becomes available, e.g. because its level is loaded. */
	virtual FConcertClientReplicationBridgeObjectEvent& OnObjectDiscovered() = 0;
	/** Event for when an object that was previously discovered becomes unavailable, e.g. because it is unloaded. */
	virtual FConcertClientReplicationBridgeObjectPathEvent& OnObjectHidden() = 0;
};
