// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/EBreakBehavior.h"
#include "Replication/Messages/ChangeAuthority.h"
#include "Replication/Messages/ChangeClientEvent.h"
#include "Replication/Messages/PutState.h"
#include "Replication/Messages/ChangeStream.h"
#include "Replication/Messages/ClientQuery.h"
#include "Replication/Messages/Handshake.h"
#include "Replication/Messages/Muting.h"
#include "Replication/Messages/RestoreContent.h"

#define UE_API CONCERTSYNCCLIENT_API

template<typename ResultType>
class TFuture;

struct FConcertReplication_ChangeSyncControl;
struct FConcertReplicationStream;

namespace UE::ConcertSyncClient::Replication
{
	struct FJoinReplicatedSessionArgs
	{
		/** The streams this client offers. */
		TArray<FConcertReplicationStream> Streams;
	};

	struct FJoinReplicatedSessionResult
	{
		/** Error code sent by the server. */
		EJoinReplicationErrorCode ErrorCode;
		/** Optional error message to help human user resolve the error. */
		FString DetailedErrorMessage;

		FJoinReplicatedSessionResult(EJoinReplicationErrorCode ErrorCode, FString DetailedErrorMessage = TEXT(""))
			: ErrorCode(ErrorCode)
			, DetailedErrorMessage(MoveTemp(DetailedErrorMessage))
		{}
	};

	struct FRemoteEditEvent
	{
		const EConcertReplicationChangeClientReason Reason;
		const FConcertReplication_ClientChangeData& ChangeData;
	};
}

/**
 * Handles all communication with the server regarding replication.
 * 
 * Keeps a list of properties to send along with their send rules.
 * Tells the server which properties this client is interested in receiving.
 */
class IConcertClientReplicationManager
{
public:

	/**
	 * Joins a replication session.
	 * Subsequent calls to JoinReplicationSession will fail until either the resulting TFuture returns or LeaveReplicationSession is called.
	 * @note This future can finish on any thread (e.g. when message endpoint times out); usually it finishes on the game thread.  
	 */
	virtual TFuture<UE::ConcertSyncClient::Replication::FJoinReplicatedSessionResult> JoinReplicationSession(UE::ConcertSyncClient::Replication::FJoinReplicatedSessionArgs Args) = 0;
	/** Leaves the current replication session. */
	virtual void LeaveReplicationSession() = 0;

	/**
	 * Whether it is valid to call JoinReplicationSession right now. Returns false if it was called and its future has not yet concluded.
	 * If this returns true, you can call JoinReplicationSession to attempt joining the session.
	 */
	virtual bool CanJoin() = 0;
	/** Whether JoinReplicationSession completed successfully and LeaveReplicationSession has not yet been called. */
	virtual bool IsConnectedToReplicationSession() = 0;

	enum class EStreamEnumerationResult { NoRegisteredStreams, Iterated };
	/**
	 * Iterates the streams the client has registered with the server.
	 * It only makes sense to call this function the manager has joined a replication session.
	 * @return Whether this manager is connected to a session (Iterated) or not (NoRegisteredStreams).
	 * @note This future can finish on any thread (e.g. when message endpoint times out); usually it finishes on the game thread.  
	 */
	virtual EStreamEnumerationResult ForEachRegisteredStream(TFunctionRef<EBreakBehavior(const FConcertReplicationStream& Stream)> Callback) const = 0;
	/** @return Whether this manager has any registered streams (basically whether ForEachRegisteredStream returns EStreamEnumerationResult::Iterated). */
	UE_API bool HasRegisteredStreams() const;
	/** @return The streams registered with the server. */
	UE_API TArray<FConcertReplicationStream> GetRegisteredStreams() const;
	
	/**
	 * Requests from the server to change the authority over some objects.
	 * @note This future can finish on any thread (e.g. when message endpoint times out); usually it finishes on the game thread.  
	 */
	virtual TFuture<FConcertReplication_ChangeAuthority_Response> RequestAuthorityChange(FConcertReplication_ChangeAuthority_Request Args) = 0;
	/** Util function that will request authority for all streams for the given objects. */
	UE_API TFuture<FConcertReplication_ChangeAuthority_Response> TakeAuthorityOver(TConstArrayView<FSoftObjectPath> Objects);
	/** Util function that will let go over all authority of the given objects. */
	UE_API TFuture<FConcertReplication_ChangeAuthority_Response> ReleaseAuthorityOf(TConstArrayView<FSoftObjectPath> Objects);

	enum class EAuthorityEnumerationResult { NoAuthorityAvailable, Iterated };
	/**
	 * Iterates through all objects that this client has authority over.
	 * @return Whether this manager is connected to a session (Iterated) or not (NoAuthorityAvailable)
	 */
	virtual EAuthorityEnumerationResult ForEachClientOwnedObject(TFunctionRef<EBreakBehavior(const FSoftObjectPath& Object, TSet<FGuid>&& OwningStreams)> Callback) const = 0;
	/** @return All the streams this client has registered that have authority for the given ObjectPath. */
	virtual TSet<FGuid> GetClientOwnedStreamsForObject(const FSoftObjectPath& ObjectPath) const = 0;
	/** @return Whether this client has authority over ObjectPath for some stream. */
	virtual bool HasAuthorityOver(const FSoftObjectPath& ObjectPath) const = 0;
	/** @return All owned objects and associated owning streams. */
	UE_API TMap<FSoftObjectPath, TSet<FGuid>> GetClientOwnedObjects() const;

	enum class ESyncControlEnumerationResult { NoneAvailable, Iterated };
	/** Iterates through all objects the server told the client to replicate, which happens when this client has taken authority and another client wants to receive the data. */
	virtual ESyncControlEnumerationResult ForEachSyncControlledObject(TFunctionRef<EBreakBehavior(const FConcertObjectInStreamID& Object)> Callback) const = 0;
	/** @return Number of items ForEachSyncControlledObject would iterate. */
	virtual uint32 NumSyncControlledObjects() const = 0;
	/** @return Whether this client has sync control for a specific object in a stream. */
	virtual bool HasSyncControl(const FConcertObjectInStreamID& Object) const = 0;
	/** Util for converting all sync controlled objects into a TSet. */
	UE_API TSet<FConcertObjectInStreamID> GetSyncControlledObjects() const;

	/**
	 * Requests replication info about other clients, including the streams registered and which objects they have authority over (i.e. are sending).
	 * @note This future can finish on any thread (e.g. when message endpoint times out); usually it finishes on the game thread.  
	 */
	virtual TFuture<FConcertReplication_QueryReplicationInfo_Response> QueryClientInfo(FConcertReplication_QueryReplicationInfo_Request Args) = 0;

	/**
	 * Requests to change the client's registered stream.
	 * @note This future can finish on any thread (e.g. when message endpoint times out); usually it finishes on the game thread.  
	 */
	virtual TFuture<FConcertReplication_ChangeStream_Response> ChangeStream(FConcertReplication_ChangeStream_Request Args) = 0;

	/**
	 * Requests to change the global mute state of objects.
	 * Can only be done if the session has the EConcertSyncSessionFlags::ShouldAllowGlobalMuting flag.
	 */
	virtual TFuture<FConcertReplication_ChangeMuteState_Response> ChangeMuteState(FConcertReplication_ChangeMuteState_Request Request) = 0;
	/** Util function that will mute all of Objects. */
	UE_API TFuture<FConcertReplication_ChangeMuteState_Response> MuteObjects(TConstArrayView<FSoftObjectPath> Objects, EConcertReplicationMuteOption Flags = EConcertReplicationMuteOption::ObjectAndSubobjects);
	/** Util function that will unmute all of Objects. */
	UE_API TFuture<FConcertReplication_ChangeMuteState_Response> UnmuteObjects(TSet<FSoftObjectPath> Objects, EConcertReplicationMuteOption Flags = EConcertReplicationMuteOption::ObjectAndSubobjects);
	/** Gets the global mute state. */
	virtual TFuture<FConcertReplication_QueryMuteState_Response> QueryMuteState(FConcertReplication_QueryMuteState_Request Request = {}) = 0;
	UE_API TFuture<FConcertReplication_QueryMuteState_Response> QueryMuteState(TSet<FSoftObjectPath> Objects);

	/** Restore this client's stream content and authority to what a client had when they left. Only works if session has EConcertSyncSessionFlags::ShouldEnableReplicationActivities set. */
	virtual TFuture<FConcertReplication_RestoreContent_Response> RestoreContent(FConcertReplication_RestoreContent_Request Request = {}) = 0;

	/**
	 * Changes multiple clients' stream, authority, and optionally the global mute state. Useful for applying a preset.
	 * Only works if session has EConcertSyncSessionFlags::ShouldEnableRemoteEditing set.
	 */
	virtual TFuture<FConcertReplication_PutState_Response> PutClientState(FConcertReplication_PutState_Request Request) = 0;

	DECLARE_MULTICAST_DELEGATE(FOnPreStreamsChanged);
	/** Called right before the result of GetRegisteredStreams changes. */
	virtual FOnPreStreamsChanged& OnPreStreamsChanged() = 0;
	
	DECLARE_MULTICAST_DELEGATE(FOnPostStreamsChanged);
	/** Called right after the result of GetRegisteredStreams has changed. */
	virtual FOnPostStreamsChanged& OnPostStreamsChanged() = 0;

	DECLARE_MULTICAST_DELEGATE(FOnPreAuthorityChanged);
	/** Called right before GetClientOwnedObjects changes. */
	virtual FOnPreAuthorityChanged& OnPreAuthorityChanged() = 0;
	
	DECLARE_MULTICAST_DELEGATE(FOnPostAuthorityChanged);
	/** Called right after GetClientOwnedObjects has changed. */
	virtual FOnPostAuthorityChanged& OnPostAuthorityChanged() = 0;

	DECLARE_MULTICAST_DELEGATE(FSyncControlChanged);
	/** Called just before sync control change is applied. */
	virtual FSyncControlChanged& OnPreSyncControlChanged() = 0;
	/** Called just after sync control change is applied. */
	virtual FSyncControlChanged& OnPostSyncControlChanged() = 0;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnRemoteEditApplied, const UE::ConcertSyncClient::Replication::FRemoteEditEvent&);
	/**
	 * Called before updating any local state in response to the server notifying us that the local client's content was remotely edited.
	 * This executes before OnPreStreamsChanged, OnPreAuthorityChanged, and OnPreSyncControlChanged, which may also get triggered depending on the change.
	 */
	virtual FOnRemoteEditApplied& OnPreRemoteEditApplied() = 0;
	/**
	 * Called after updating any local state in response to the server notifying us that the local client's content was remotely edited.
	 * This executes after OnPreStreamsChanged, OnPreAuthorityChanged, and OnPreSyncControlChanged, which may also get triggered depending on the change.
	 */
	virtual FOnRemoteEditApplied& OnPostRemoteEditApplied() = 0;
	
	virtual ~IConcertClientReplicationManager() = default;
};
	

#undef UE_API
