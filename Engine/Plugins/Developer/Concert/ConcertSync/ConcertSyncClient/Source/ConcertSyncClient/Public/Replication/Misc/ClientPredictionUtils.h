// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "HAL/Platform.h"

class IConcertClientWorkspace;

struct FConcertBaseStreamInfo;
struct FConcertClientInfo;
struct FConcertObjectInStreamID;

template<typename OptionalType> struct TOptional;

namespace UE::ConcertSyncClient::Replication
{
	/**
	 * Leverages IConcertClientWorkspace::GetActivities to get chunks from the back of the activity history and passes them to
	 * BacktrackActivityHistoryForSettingActivities until an activity is found that sets the specified client's stream and authority content.
	 * 
	 * @param Workspace The workspace to retrieve the activity history chunks from 
	 * @param ClientInfo Only activities produced by clients that are logically equivalent to this info are considered (@see AreLogicallySameClients)
	 * @param OutStreams The streams the client had, if successful
	 * @param OutAuthority The authority the client had, if successful
	 * @param MaxToFetch Maximum number of activities to fetch with each IConcertClientWorkspace::GetActivities call
	 * @param MinActivityIdCutoff This is the min activity id an activity must have to be considered. Activities before this are not considered.
	 *
	 * @return The activity ID that contained the state. Unset if not found.
	 */
	CONCERTSYNCCLIENT_API TOptional<int64> IncrementalBacktrackActivityHistoryForActivityThatSetsContent(
		const IConcertClientWorkspace& Workspace,
		const FConcertClientInfo& ClientInfo,
		TArray<FConcertBaseStreamInfo>& OutStreams,
		TArray<FConcertObjectInStreamID>& OutAuthority,
		int64 MaxToFetch = 50,
		int64 MinActivityIdCutoff = 1
		);
}
