// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertMessages.h"
#include "CoreMinimal.h"
#include "ConcertSyncSessionTypes.h"
#include "Misc/AutomationTest.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "ConcertClientLiveTransactionAuthors.h"
#include "IConcertSession.h"
#include "ConcertSyncClientLiveSession.h"
#include "ConcertSyncSessionDatabase.h"
#include "Scratchpad/ConcertScratchpad.h"

/** Flags used for the tests. */
static const EAutomationTestFlags ConcertClientLiveTransactionAuthorsTestFlags = EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter;

namespace ConcertLiveTransactionAuthorsTestUtils
{
// Utility functions used to detect when a non-mocked function is called, so that we can mock it properly when required.
template<typename T> T NotMocked(T Ret) { check(false); return Ret; }
template<typename T> T NotMocked()      { check(false); return T(); }

/** Implements a not-working IConcertClientSession. It must be further overridden to implement just what is required by the tests */
class FConcertClientSessionMock : public IConcertClientSession
{
public:
	FConcertClientSessionMock(FConcertSyncEndpointIdAndData InLocalClientInfo)
		: Id(FGuid::NewGuid())
		, Name("FConcertClientSessionMock")
		, LocalClientInfo(MoveTemp(InLocalClientInfo))
	{ }

	// IConcertSession Begin.
	virtual const FGuid& GetId() const override																								{ return Id; }
	virtual const FString& GetName() const override																							{ return NotMocked<const FString&>(Name); }
	virtual const FConcertSessionInfo& GetSessionInfo() const override																		{ return NotMocked<const FConcertSessionInfo&>(SessionInfo); }
	virtual FString GetSessionWorkingDirectory() const override																				{ return FPaths::ProjectIntermediateDir() / TEXT("ConcertClientLiveTransactionAuthorsTest") / Id.ToString(); }
	virtual TArray<FGuid> GetSessionClientEndpointIds() const override																		{ return NotMocked<TArray<FGuid>>(); }
	virtual TArray<FConcertSessionClientInfo> GetSessionClients() const override															{ return NotMocked<TArray<FConcertSessionClientInfo>>(); }
	virtual bool FindSessionClient(const FGuid& InGuid, FConcertSessionClientInfo& OutInfo) const override									{ return GetClient(InGuid, OutInfo); }
	virtual void Startup() override																											{ return NotMocked<void>(); }
	virtual void Shutdown() override																										{ return NotMocked<void>(); };
	virtual FConcertScratchpadRef GetScratchpad() const override																			{ return NotMocked<FConcertScratchpadRef>(); }
	virtual FConcertScratchpadPtr GetClientScratchpad(const FGuid&) const override															{ return NotMocked<FConcertScratchpadRef>(); }
	virtual FDelegateHandle InternalRegisterCustomEventHandler(const FName&, const TSharedRef<IConcertSessionCustomEventHandler>&) override { return NotMocked<FDelegateHandle>(); }
	virtual void InternalUnregisterCustomEventHandler(const FName& EventMessageType, const FDelegateHandle EventHandle) override			{ return NotMocked<void>(); }
	virtual void InternalUnregisterCustomEventHandler(const FName& EventMessageType, const void* EventHandler) override						{ return NotMocked<void>(); }
	virtual void InternalClearCustomEventHandler(const FName& EventMessageType) override													{ return NotMocked<void>(); }
	virtual void InternalSendCustomEvent(const UScriptStruct*, const void*, const TArray<FGuid>&, EConcertMessageFlags, TOptional<FConcertSequencedCustomEvent> = {}) override
	{
		return NotMocked<void>();
	}
	virtual void InternalRegisterCustomRequestHandler(const FName&, const TSharedRef<IConcertSessionCustomRequestHandler>&) override		{ return NotMocked<void>(); }
	virtual void InternalUnregisterCustomRequestHandler(const FName&) override																{ return NotMocked<void>(); }
	virtual void InternalSendCustomRequest(const UScriptStruct*, const void*, const FGuid&, const TSharedRef<IConcertSessionCustomResponseHandler>&) override { NotMocked<void>(); }
	// IConcertSession End.

	// IConcertClientSession Begin
	virtual EConcertConnectionStatus GetConnectionStatus() const override            { return NotMocked<EConcertConnectionStatus>(EConcertConnectionStatus::Connected); }
	virtual FGuid GetSessionClientEndpointId() const override                        { return LocalClientInfo.EndpointId; }
	virtual FGuid GetSessionServerEndpointId() const override                        { return NotMocked<FGuid>(); }
	virtual const FConcertClientInfo& GetLocalClientInfo() const override            { return LocalClientInfo.EndpointData.ClientInfo; }
	virtual void UpdateLocalClientInfo(const FConcertClientInfoUpdate&) override     { return NotMocked<void>(); }
	virtual void Connect() override                                                  { return NotMocked<void>(); }
	virtual void Disconnect() override                                               { return NotMocked<void>(); }

	virtual EConcertSendReceiveState GetSendReceiveState() const override
	{
		return NotMocked<EConcertSendReceiveState>(EConcertSendReceiveState::Default);
	}

	virtual void SetSendReceiveState(EConcertSendReceiveState InSendReceiveState) override
	{
		return NotMocked<void>();
	}

	virtual FOnConcertClientSessionTick& OnTick() override                           { return NotMocked<FOnConcertClientSessionTick&>(Tick); }
	virtual FOnConcertClientSessionConnectionChanged& OnConnectionChanged() override { return NotMocked<FOnConcertClientSessionConnectionChanged&>(ConnectionChanged); }
	virtual FOnConcertClientSessionClientChanged& OnSessionClientChanged() override  { return NotMocked<FOnConcertClientSessionClientChanged&>(ClientChanged); }
	virtual FOnConcertSessionRenamed& OnSessionRenamed() override                    { return NotMocked<FOnConcertSessionRenamed&>(SessionRenamed); }
	// IConcertClientSession End

	void AddClient(FConcertSessionClientInfo Client) { OtherClientsInfo.Add(Client.ClientEndpointId, MoveTemp(Client)); }

	bool GetClient(const FGuid& InGuid, FConcertSessionClientInfo& OutInfo) const
	{
		if (const FConcertSessionClientInfo* ClientInfo = OtherClientsInfo.Find(InGuid))
		{
			OutInfo = *ClientInfo;
			return true;
		}
		return false;
	}
	virtual FConcertSequencedCustomEventManager& GetSequencedEventManager() override
	{
		return NotMocked<FConcertSequencedCustomEventManager&>(CustomEventSequenceManager);
	}

protected:
	FGuid Id;

	// Those below are unused mocked data member, but required to get the code compiling.
	FString Name;
	FConcertSessionInfo SessionInfo;
	FOnConcertClientSessionTick Tick;
	FOnConcertClientSessionConnectionChanged ConnectionChanged;
	FOnConcertClientSessionClientChanged ClientChanged;
	FOnConcertSessionRenamed SessionRenamed;
	FConcertSyncEndpointIdAndData LocalClientInfo;
	FConcertSequencedCustomEventManager CustomEventSequenceManager;
	TMap<FGuid, FConcertSessionClientInfo> OtherClientsInfo;
};

void AddTransactionActivity(FConcertSyncClientLiveSession& InLiveSession, FConcertClientLiveTransactionAuthors& InLiveTransactionAuthors, const FName InPackageName, const FConcertSyncEndpointIdAndData& InSourceClient)
{
	InLiveSession.GetSessionDatabase().SetEndpoint(InSourceClient.EndpointId, InSourceClient.EndpointData);

	FConcertSyncTransactionActivity TransactionActivity;
	TransactionActivity.EndpointId = InSourceClient.EndpointId;
	TransactionActivity.EventData.Transaction.ModifiedPackages.Add(InPackageName);

	int64 ActivityId = 0;
	int64 EventId = 0;
	if (InLiveSession.GetSessionDatabase().AddTransactionActivity(TransactionActivity, ActivityId, EventId))
	{
		// Need to get back the resolved transaction activity
		InLiveSession.GetSessionDatabase().GetTransactionActivity(ActivityId, TransactionActivity);
		InLiveTransactionAuthors.AddLiveTransactionActivity(TransactionActivity.EndpointId, TransactionActivity.EventData.Transaction.ModifiedPackages);
	}
}

void FenceTransactions(FConcertSyncClientLiveSession& InLiveSession, FConcertClientLiveTransactionAuthors& InLiveTransactionAuthors, const FName InPackageName, const FConcertSyncEndpointIdAndData& InSourceClient, int64* InTransactionIdOverride = nullptr)
{
	InLiveSession.GetSessionDatabase().SetEndpoint(InSourceClient.EndpointId, InSourceClient.EndpointData);

	FConcertSyncActivity PackageActivityBasePart;
	FConcertPackageInfo PackageInfo;
	FConcertPackageDataStream PackageDataStream;
	PackageActivityBasePart.EndpointId = InSourceClient.EndpointId;
	PackageInfo.PackageName = InPackageName;
	PackageInfo.PackageUpdateType = EConcertPackageUpdateType::Dummy;
	if (InTransactionIdOverride)
	{
		PackageInfo.TransactionEventIdAtSave = *InTransactionIdOverride;
	}
	else
	{
		InLiveSession.GetSessionDatabase().GetTransactionMaxEventId(PackageInfo.TransactionEventIdAtSave);
	}

	int64 ActivityId = 0;
	int64 EventId = 0;
	InLiveSession.GetSessionDatabase().AddPackageActivity(PackageActivityBasePart, PackageInfo, PackageDataStream, ActivityId, EventId);

	InLiveTransactionAuthors.ResolveLiveTransactionAuthorsForPackage(PackageInfo.PackageName);
}

/** Ensures the live transaction authors works correctly when there is no other clients connected. */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FConcertLiveTransactionAuthorsSingleClient, "VirtualProduction.Concert.LiveTransactionAuthors.SingleClient", ConcertClientLiveTransactionAuthorsTestFlags)

bool FConcertLiveTransactionAuthorsSingleClient::RunTest(const FString& Parameters)
{
	FConcertSyncEndpointIdAndData ThisClient;
	ThisClient.EndpointId = FGuid::NewGuid();
	ThisClient.EndpointData.ClientInfo.Initialize();

	TSharedRef<IConcertClientSession> Session = MakeShared<FConcertClientSessionMock>(ThisClient);
	TSharedRef<FConcertSyncClientLiveSession> LiveSession = MakeShared<FConcertSyncClientLiveSession>(Session, EConcertSyncSessionFlags::None);
	FConcertClientLiveTransactionAuthors LiveTransactionAuthors(LiveSession);

	// Test without any transaction.
	FName PackageName(TEXT("MyLevel"));
	TestTrueExpr(!LiveTransactionAuthors.IsPackageAuthoredByOtherClients(PackageName));

	// Add a live transaction from this client. Ensure it doesn't affect the authored by others.
	AddTransactionActivity(*LiveSession, LiveTransactionAuthors, PackageName, ThisClient);
	TestTrueExpr(!LiveTransactionAuthors.IsPackageAuthoredByOtherClients(PackageName));

	// Add a live transaction on another package.
	AddTransactionActivity(*LiveSession, LiveTransactionAuthors, FName(TEXT("OtherPackage")), ThisClient);
	TestTrueExpr(!LiveTransactionAuthors.IsPackageAuthoredByOtherClients(PackageName));

	// Trim all transactions. Ensure it doesn't affect the package authored by others.
	FenceTransactions(*LiveSession, LiveTransactionAuthors, PackageName, ThisClient);

	int32 OtherClientCount = 0;
	TArray<FConcertClientInfo> OtherClients;
	TestTrueExpr(!LiveTransactionAuthors.IsPackageAuthoredByOtherClients(PackageName, &OtherClientCount, &OtherClients, 10));
	TestTrueExpr(OtherClientCount == 0);
	TestTrueExpr(OtherClients.Num() == 0);

	IFileManager::Get().DeleteDirectory(*Session->GetSessionWorkingDirectory(), false, true);

	return true;
}

/** Ensures the live transaction authors works correctly when there are many clients connected. */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FConcertLiveTransactionAuthorsManyClients, "VirtualProduction.Concert.LiveTransactionAuthors.ManyClients", ConcertClientLiveTransactionAuthorsTestFlags)

bool FConcertLiveTransactionAuthorsManyClients::RunTest(const FString& Parameters)
{
	// Represents the local client.
	FConcertSyncEndpointIdAndData ThisClient;
	ThisClient.EndpointId = FGuid::NewGuid();
	ThisClient.EndpointData.ClientInfo.Initialize();

	// Represents the other clients.
	FConcertSyncEndpointIdAndData OtherClient1;
	OtherClient1.EndpointId = FGuid::NewGuid();
	OtherClient1.EndpointData.ClientInfo.Initialize();
	OtherClient1.EndpointData.ClientInfo.InstanceInfo.InstanceId.A += 1; // Make the InstanceId unique, Initialize() uses the AppId which is the same across the app.

	FConcertSyncEndpointIdAndData OtherClient2;
	OtherClient2.EndpointId = FGuid::NewGuid();
	OtherClient2.EndpointData.ClientInfo.Initialize();
	OtherClient2.EndpointData.ClientInfo.InstanceInfo.InstanceId.B += 1; // Make the InstanceId unique, Initialize() uses the AppId which is the same across the app.

	// Ensure each client has a unique instance Id. The value is not important, but they must be different for the tests to work.
	TestTrueExpr(ThisClient.EndpointData.ClientInfo.InstanceInfo.InstanceId != OtherClient1.EndpointData.ClientInfo.InstanceInfo.InstanceId);
	TestTrueExpr(ThisClient.EndpointData.ClientInfo.InstanceInfo.InstanceId != OtherClient2.EndpointData.ClientInfo.InstanceInfo.InstanceId);
	TestTrueExpr(OtherClient1.EndpointData.ClientInfo.InstanceInfo.InstanceId != OtherClient2.EndpointData.ClientInfo.InstanceInfo.InstanceId);

	// Create the session.
	TSharedRef<FConcertClientSessionMock> Session = MakeShared<FConcertClientSessionMock>(ThisClient);
	TSharedRef<FConcertSyncClientLiveSession> LiveSession = MakeShared<FConcertSyncClientLiveSession>(Session, EConcertSyncSessionFlags::None);

	// Add other clients to the session.
	Session->AddClient(FConcertSessionClientInfo{ OtherClient1.EndpointId, OtherClient1.EndpointData.ClientInfo });
	Session->AddClient(FConcertSessionClientInfo{ OtherClient2.EndpointId, OtherClient2.EndpointData.ClientInfo });

	// Create the live transaction author tracker.
	FConcertClientLiveTransactionAuthors LiveTransactionAuthors(LiveSession);

	// An hypothetical package.
	FName PackageName(TEXT("MyLevel"));
	FName OtherPackageName(TEXT("OtherLevel"));
	
	// Test without any transaction.
	TestTrueExpr(!LiveTransactionAuthors.IsPackageAuthoredByOtherClients(PackageName));

	int32 OtherClientCount = 0;
	TArray<FConcertClientInfo> OtherClients;

	// Add a live transaction from client 1. Ensure it is tracked.
	AddTransactionActivity(*LiveSession, LiveTransactionAuthors, PackageName, OtherClient1);
	TestTrueExpr(LiveTransactionAuthors.IsPackageAuthoredByOtherClients(PackageName));
	TestTrueExpr(LiveTransactionAuthors.IsPackageAuthoredByOtherClients(PackageName, &OtherClientCount, &OtherClients, 10));
	TestTrueExpr(OtherClientCount == 1);
	TestTrueExpr(OtherClients[0].InstanceInfo.InstanceId == OtherClient1.EndpointData.ClientInfo.InstanceInfo.InstanceId);
	OtherClients.Empty();

	// Add a live transaction from client 2. Ensure it is tracked.
	AddTransactionActivity(*LiveSession, LiveTransactionAuthors, PackageName, OtherClient2);
	TestTrueExpr(LiveTransactionAuthors.IsPackageAuthoredByOtherClients(PackageName));
	TestTrueExpr(LiveTransactionAuthors.IsPackageAuthoredByOtherClients(PackageName, &OtherClientCount, &OtherClients, 10));
	TestTrueExpr(OtherClientCount == 2);
	TestTrueExpr(OtherClients[0].InstanceInfo.InstanceId == OtherClient1.EndpointData.ClientInfo.InstanceInfo.InstanceId || OtherClients[0].InstanceInfo.InstanceId == OtherClient1.EndpointData.ClientInfo.InstanceInfo.InstanceId);
	TestTrueExpr(OtherClients[1].InstanceInfo.InstanceId == OtherClient2.EndpointData.ClientInfo.InstanceInfo.InstanceId || OtherClients[1].InstanceInfo.InstanceId == OtherClient2.EndpointData.ClientInfo.InstanceInfo.InstanceId);
	TestTrueExpr(OtherClients[0].InstanceInfo.InstanceId != OtherClients[1].InstanceInfo.InstanceId);
	OtherClients.Empty();

	// Ensure the API only returns 1 client out of 2 if only 1 is requested.
	LiveTransactionAuthors.IsPackageAuthoredByOtherClients(PackageName, &OtherClientCount, &OtherClients, 1);
	TestTrueExpr(OtherClients.Num() == 1);
	OtherClients.Empty();

	// Trim all transactions.
	FenceTransactions(*LiveSession, LiveTransactionAuthors, PackageName, ThisClient);
	TestTrueExpr(!LiveTransactionAuthors.IsPackageAuthoredByOtherClients(PackageName));

	// Add a live transaction on another package just to make noise.
	AddTransactionActivity(*LiveSession, LiveTransactionAuthors, OtherPackageName, ThisClient);
	AddTransactionActivity(*LiveSession, LiveTransactionAuthors, OtherPackageName, OtherClient2);

	// Add more transactions from client 1.
	AddTransactionActivity(*LiveSession, LiveTransactionAuthors, PackageName, OtherClient1);
	AddTransactionActivity(*LiveSession, LiveTransactionAuthors, PackageName, OtherClient1);
	AddTransactionActivity(*LiveSession, LiveTransactionAuthors, PackageName, OtherClient1);
	TestTrueExpr(LiveTransactionAuthors.IsPackageAuthoredByOtherClients(PackageName, &OtherClientCount));
	TestTrueExpr(OtherClientCount == 1);

	int64 TransactionEventIdAtClient1Save = 0;
	LiveSession->GetSessionDatabase().GetTransactionMaxEventId(TransactionEventIdAtClient1Save);

	// Add more transactions from client 2.
	AddTransactionActivity(*LiveSession, LiveTransactionAuthors, PackageName, OtherClient2);
	AddTransactionActivity(*LiveSession, LiveTransactionAuthors, PackageName, OtherClient2);
	TestTrueExpr(LiveTransactionAuthors.IsPackageAuthoredByOtherClients(PackageName, &OtherClientCount));
	TestTrueExpr(OtherClientCount == 2);

	// Trim the transaction from client 1 only.
	FenceTransactions(*LiveSession, LiveTransactionAuthors, PackageName, ThisClient, &TransactionEventIdAtClient1Save);
	TestTrueExpr(LiveTransactionAuthors.IsPackageAuthoredByOtherClients(PackageName, &OtherClientCount, &OtherClients, 1));
	TestTrueExpr(OtherClientCount == 1);
	TestTrueExpr(OtherClients[0].InstanceInfo.InstanceId == OtherClient2.EndpointData.ClientInfo.InstanceInfo.InstanceId);

	// Trim all remaining transactions.
	FenceTransactions(*LiveSession, LiveTransactionAuthors, PackageName, ThisClient);
	TestTrueExpr(!LiveTransactionAuthors.IsPackageAuthoredByOtherClients(PackageName, &OtherClientCount));
	TestTrueExpr(OtherClientCount == 0);

	// Ensure trim only trimmed for "PackageName", not "OtherPackageName". Client2 has a transaction on OtherPackageName.
	TestTrueExpr(LiveTransactionAuthors.IsPackageAuthoredByOtherClients(OtherPackageName, &OtherClientCount));
	TestTrueExpr(OtherClientCount == 1);

	IFileManager::Get().DeleteDirectory(*Session->GetSessionWorkingDirectory(), false, true);

	return true;
}

/** Ensures the live transaction authors works correctly when they are some transaction owned by a disconnected client. */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FConcertLiveTransactionAuthorsDisconnectedClient, "VirtualProduction.Concert.LiveTransactionAuthors.DisconnectedClient", ConcertClientLiveTransactionAuthorsTestFlags)

bool FConcertLiveTransactionAuthorsDisconnectedClient::RunTest(const FString& Parameters)
{
	// Represents the current local client. Let say it represents a person named 'Joe Smith' currently connected.
	FConcertSyncEndpointIdAndData CurrentInstanceOfJoeSmith;
	CurrentInstanceOfJoeSmith.EndpointId = FGuid::NewGuid();
	CurrentInstanceOfJoeSmith.EndpointData.ClientInfo.Initialize();

	// Represents a previous editor instance used by 'Joe Smith'. In that previous instance, Joe had another InstanceId, but he
	// closed (or crashed) the editor without saving. So the previous instance of Joe has live transaction pending. He now has
	// launched a new editor and rejoined the session from the same computer. Below, we simulate its previous instance id.
	FConcertSyncEndpointIdAndData PreviousInstanceOfJoeSmith;
	PreviousInstanceOfJoeSmith.EndpointId = FGuid::NewGuid();
	PreviousInstanceOfJoeSmith.EndpointData.ClientInfo.Initialize();
	PreviousInstanceOfJoeSmith.EndpointData.ClientInfo.InstanceInfo.InstanceId.A += 1; // Make the InstanceId unique, Initialize() uses the AppId which is the same across the app.

	// Represents a disconnected user name Jane Doe who left the session without saving her modifications.
	FConcertSyncEndpointIdAndData DisconnectedInstanceOfJaneDoe;
	DisconnectedInstanceOfJaneDoe.EndpointId = FGuid::NewGuid();
	DisconnectedInstanceOfJaneDoe.EndpointData.ClientInfo.Initialize();
	DisconnectedInstanceOfJaneDoe.EndpointData.ClientInfo.InstanceInfo.InstanceId.B += 1; // Make the InstanceId unique, Initialize() uses the AppId which is the same across the app.
	DisconnectedInstanceOfJaneDoe.EndpointData.ClientInfo.DeviceName = TEXT("ThisIsJaneDoeComputer");
	DisconnectedInstanceOfJaneDoe.EndpointData.ClientInfo.UserName = TEXT("jane.doe");
	DisconnectedInstanceOfJaneDoe.EndpointData.ClientInfo.DisplayName = TEXT("Jane Doe");

	// Create the session and transaction author tracker. Don't add the disconnected client to the session.
	TSharedRef<FConcertClientSessionMock> Session = MakeShared<FConcertClientSessionMock>(CurrentInstanceOfJoeSmith);
	TSharedRef<FConcertSyncClientLiveSession> LiveSession = MakeShared<FConcertSyncClientLiveSession>(Session, EConcertSyncSessionFlags::None);
	FConcertClientLiveTransactionAuthors LiveTransactionAuthors(LiveSession);

	// An hypothetical package.
	FName PackageName(TEXT("MyLevel"));

	// Add live transactions from the disconnected client, just like when a client connects, it gets all live transactions from the transaction ledger,
	// resolve their author using the activity ledger, then populate the live transaction author tracker. During that process, some live transactions may be
	// resolved to author that are now disconnected. The code below simulate that.
	AddTransactionActivity(*LiveSession, LiveTransactionAuthors, PackageName, PreviousInstanceOfJoeSmith);
	AddTransactionActivity(*LiveSession, LiveTransactionAuthors, PackageName, PreviousInstanceOfJoeSmith);

	// We expect the LiveTransactionAuthors to map the previous instance of Joe Smith to the actual instance of Joe Smith because the instance are not
	// run simultaneously, but rather run one after the other (When the same person runs 2 editors in parallel, the person is recognized as 2 different clients).
	// In that case, the live transaction performed by 'PreviousInstanceOfJoeSmith' should be assigned to 'CurrentInstanceOfJoeSmith' instance.
	TestTrueExpr(!LiveTransactionAuthors.IsPackageAuthoredByOtherClients(PackageName));

	// Jane Doe is not connected anymore and she doesn't match Joe Smith identity signature (user name, display name, device name, etc). She should be recognize as a different user.
	AddTransactionActivity(*LiveSession, LiveTransactionAuthors, PackageName, DisconnectedInstanceOfJaneDoe);
	TestTrueExpr(LiveTransactionAuthors.IsPackageAuthoredByOtherClients(PackageName));

	IFileManager::Get().DeleteDirectory(*Session->GetSessionWorkingDirectory(), false, true);

	return true;
}

/** Ensures the live transaction authors works correctly when the same person is editing a package from two editors, on the same machine, concurrently. */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FConcertLiveTransactionAuthorsClientUsingTwoEditors, "VirtualProduction.Concert.LiveTransactionAuthors.SamePersonUsingTwoEditorsConcurrently", ConcertClientLiveTransactionAuthorsTestFlags)

bool FConcertLiveTransactionAuthorsClientUsingTwoEditors::RunTest(const FString& Parameters)
{
	// Represents the current local client. Let say it represents a person named 'Joe Smith' currently connected.
	FConcertSyncEndpointIdAndData ThisJoeSmithInstance;
	ThisJoeSmithInstance.EndpointId = FGuid::NewGuid();
	ThisJoeSmithInstance.EndpointData.ClientInfo.Initialize();

	// Represents also the person 'Joe Smith' but from another editor instance, on the same machine, running concurrently with 'ThisJoeSmithInstance'.
	// Both editors used by Joe are connected to the same session.
	FConcertSyncEndpointIdAndData AnotherInstanceOfJoeSmith;
	AnotherInstanceOfJoeSmith.EndpointId = FGuid::NewGuid();
	AnotherInstanceOfJoeSmith.EndpointData.ClientInfo.Initialize();
	AnotherInstanceOfJoeSmith.EndpointData.ClientInfo.InstanceInfo.InstanceId.A += 1; // Make the InstanceId unique, Initialize() uses the AppId which is the same across the app.

	// Create the session and transaction author tracker.
	TSharedRef<FConcertClientSessionMock> Session = MakeShared<FConcertClientSessionMock>(ThisJoeSmithInstance);
	TSharedRef<FConcertSyncClientLiveSession> LiveSession = MakeShared<FConcertSyncClientLiveSession>(Session, EConcertSyncSessionFlags::None);
	FConcertClientLiveTransactionAuthors LiveTransactionAuthors(LiveSession);

	// Add the other Joe instance to the session.
	Session->AddClient(FConcertSessionClientInfo{ AnotherInstanceOfJoeSmith.EndpointId, AnotherInstanceOfJoeSmith.EndpointData.ClientInfo });

	// An hypothetical package.
	FName PackageName(TEXT("MyLevel"));

	// Add transaction from the local instance of JoeSmith. He should be recognized as himself.
	AddTransactionActivity(*LiveSession, LiveTransactionAuthors, PackageName, ThisJoeSmithInstance);
	TestTrueExpr(!LiveTransactionAuthors.IsPackageAuthoredByOtherClients(PackageName));

	// Add transaction from the other instance of JoeSmith. He should be recognized as a different client.
	AddTransactionActivity(*LiveSession, LiveTransactionAuthors, PackageName, AnotherInstanceOfJoeSmith);
	TestTrueExpr(LiveTransactionAuthors.IsPackageAuthoredByOtherClients(PackageName));

	IFileManager::Get().DeleteDirectory(*Session->GetSessionWorkingDirectory(), false, true);

	return true;
}

}

